[CoreRedirects]

+ClassRedirects=(OldName="PaperAnimatedRenderComponent", NewName="/Script/Paper2D.PaperFlipbookComponent")
+ClassRedirects=(OldName="PaperRenderActor", NewName="/Script/Paper2D.PaperSpriteActor")
+ClassRedirects=(OldName="PaperRenderComponent", NewName="/Script/Paper2D.PaperSpriteComponent")
+ClassRedirects=(OldName="PaperTileMapRenderComponent", NewName="/Script/Paper2D.PaperTileMapComponent")
+ClassRedirects=(OldName="PaperSpriteSheet", NewName="/Script/PaperSpritesheetImporter.PaperSpriteSheet")
	
+PropertyRedirects=(OldName="PaperTileSet.ExperimentalPerTileData", NewName="PaperTileSet.PerTileData")
+PropertyRedirects=(OldName="PaperRenderComponent.TestMaterial", NewName="PaperRenderComponent.MaterialOverride")
