{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Paper2D",
	"Description" : "Paper2D adds tools and assets to help create 2D games including animated sprite assets, tilesets (experimental), 2D level editing tools, and more.",
	"Category" : "2D",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "Paper2D",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "Paper2DEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "PaperSpriteSheetImporter",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "PaperTiledImporter",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "SmartSnapping",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}