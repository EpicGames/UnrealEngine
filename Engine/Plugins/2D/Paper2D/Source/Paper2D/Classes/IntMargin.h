// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IntMargin.generated.h"

/**
 * Describes the space around a 2D area on an integer grid.
 */
USTRUCT(BlueprintType)
struct FIntMargin
{
	GENERATED_USTRUCT_BODY()

	/** Holds the margin to the left. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	int32 Left;

	/** Holds the margin to the top. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	int32 Top;
	
	/** Holds the margin to the right. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	int32 Right;
	
	/** Holds the margin to the bottom. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	int32 Bottom;

public:

	/**
	 * Default constructor.
	 *
	 * The default margin size is zero on all four sides..
	 */
	FIntMargin( )
		: Left(0)
		, Top(0)
		, Right(0)
		, Bottom(0)
	{ }

	/** Construct a Margin with uniform space on all sides */
	FIntMargin(int32 UniformMargin)
		: Left(UniformMargin)
		, Top(UniformMargin)
		, Right(UniformMargin)
		, Bottom(UniformMargin)
	{ }
	
	/** Construct a Margin where Horizontal describes Left and Right spacing while Vertical describes Top and Bottom spacing */
	FIntMargin(int32 Horizontal, int32 Vertical)
		: Left(Horizontal)
		, Top(Vertical)
		, Right(Horizontal)
		, Bottom(Vertical)
	{ }
	
	/** Construct a Margin where the spacing on each side is individually specified. */
	FIntMargin(int32 InLeft, int32 InTop, int32 InRight, int32 InBottom)
		: Left(InLeft)
		, Top(InTop)
		, Right(InRight)
		, Bottom(InBottom)
	{ }
	
public:

	/**
	 * Adds another margin to this margin.
	 *
	 * @param Other The margin to add.
	 * @return A margin that represents this margin plus the other margin.
	 */
	FIntMargin operator+(const FIntMargin& InDelta) const
	{
		return FIntMargin(Left + InDelta.Left, Top + InDelta.Top, Right + InDelta.Right, Bottom + InDelta.Bottom);
	}

	/**
	 * Subtracts another margin from this margin.
	 *
	 * @param Other The margin to subtract.
	 * @return A margin that represents this margin minus the other margin.
	 */
	FIntMargin operator-(const FIntMargin& Other) const
	{
		return FIntMargin(Left - Other.Left, Top - Other.Top, Right - Other.Right, Bottom - Other.Bottom);
	}

	/**
	 * Compares this margin with another for equality.
	 *
	 * @param Other The other margin.
	 * @return true if the two margins are equal, false otherwise.
	 */
	bool operator==(const FIntMargin& Other) const
	{
		return (Left == Other.Left) && (Right == Other.Right) && (Top == Other.Top) && (Bottom == Other.Bottom);
	}

	/**
	 * Compares this margin with another for inequality.
	 *
	 * @param Other The other margin.
	 * @return true if the two margins are not equal, false otherwise.
	 */
	bool operator!=(const FIntMargin& Other) const
	{
		return (Left != Other.Left) || (Right != Other.Right) || (Top != Other.Top) || (Bottom != Other.Bottom);
	}

public:

	/**
	 * Gets the margin's total size.
	 *
	 * @return Cumulative margin size.
	 */
	FIntPoint GetDesiredSize() const
	{
		return FIntPoint(Left + Right, Top + Bottom);
	}
};
