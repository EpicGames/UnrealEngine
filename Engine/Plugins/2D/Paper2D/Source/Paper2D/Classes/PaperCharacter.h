// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Character.h"
#include "PaperCharacter.generated.h"

#define UE_API PAPER2D_API

class UPaperFlipbookComponent;

// APaperCharacter behaves like ACharacter, but uses a UPaperFlipbookComponent instead of a USkeletalMeshComponent as a visual representation
// Note: The variable named Mesh will not be set up on this actor!
UCLASS(MinimalAPI)
class APaperCharacter : public ACharacter
{
	GENERATED_UCLASS_BODY()
	
	// Name of the Sprite component
	static UE_API FName SpriteComponentName;

private:
	/** The main skeletal mesh associated with this Character (optional sub-object). */
	UPROPERTY(Category = Character, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UPaperFlipbookComponent> Sprite;
public:

	UE_API virtual void PostInitializeComponents() override;

	/** Returns Sprite subobject **/
	FORCEINLINE class UPaperFlipbookComponent* GetSprite() const { return Sprite; }
};

#undef UE_API
