// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperFlipbook.generated.h"

#define UE_API PAPER2D_API

struct FComponentSocketDescription;
struct FPropertyChangedEvent;

class UMaterialInterface;
class UPaperSprite;

USTRUCT()
struct FPaperFlipbookKeyFrame
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category=Sprite, EditAnywhere)
	TObjectPtr<UPaperSprite> Sprite;

	UPROPERTY(Category=Sprite, EditAnywhere, meta=(ClampMin=1))
	int32 FrameRun;

	FPaperFlipbookKeyFrame()
		: Sprite(nullptr)
		, FrameRun(1)
	{
	}
};

UENUM()
namespace EFlipbookCollisionMode
{
	enum Type : int
	{
		/** The flipbook has no collision */
		NoCollision,

		/** The flipbook has non-animated collision based on the first frame of the animation */
		FirstFrameCollision,

		/** The flipbook changes collision each frame based on the animation (Note: This setting is not recommended and is very expensive, recreating the physics state every frame) */
		EachFrameCollision,
	};
}

/**
 * Contains an animation sequence of sprite frames
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayThumbnail = "true"))
class UPaperFlipbook : public UObject
{
	GENERATED_UCLASS_BODY()

protected:
	// The nominal frame rate to play this flipbook animation back at
	UPROPERTY(Category=Sprite, EditAnywhere, BlueprintReadOnly, meta=(ClampMin=0, ClampMax=1000))
	float FramesPerSecond;

	// The set of key frames for this flipbook animation (each one has a duration and a sprite to display)
	UPROPERTY(Category=Sprite, EditAnywhere)
	TArray<FPaperFlipbookKeyFrame> KeyFrames;

	// The material to use on a flipbook player instance if not overridden
	UPROPERTY(Category=Sprite, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UMaterialInterface> DefaultMaterial;

	// Collision source
	UPROPERTY(Category=Sprite, EditAnywhere, BlueprintReadOnly)
	TEnumAsByte<EFlipbookCollisionMode::Type> CollisionSource;

public:
	// Returns the collision source of this flipbook animation (if any)
	TEnumAsByte<EFlipbookCollisionMode::Type> GetCollisionSource() const { return CollisionSource; }

	// Returns the nominal frame rate to play this flipbook animation back at
	UE_API float GetFramesPerSecond() const;

	// Returns the total number of frames
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API int32 GetNumFrames() const;

	// Returns the total duration in seconds
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API float GetTotalDuration() const;

	// Returns the keyframe index that covers the specified time (in seconds), or INDEX_NONE if none exists.
	// When bClampToEnds is true, it will choose the first or last keyframe if the time is out of range.
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API int32 GetKeyFrameIndexAtTime(float Time, bool bClampToEnds = false) const;

	// Returns the sprite at the specified time (in seconds), or nullptr if none exists.
	// When bClampToEnds is true, it will choose the first or last sprite if the time is out of range.
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API UPaperSprite* GetSpriteAtTime(float Time, bool bClampToEnds = false) const;

	// Returns the sprite at the specified keyframe index, or nullptr if none exists
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API UPaperSprite* GetSpriteAtFrame(int32 FrameIndex) const;

	// Returns the render bounds of this sprite
	UE_API FBoxSphereBounds GetRenderBounds() const;

	// Returns the number of key frames
	UFUNCTION(BlueprintCallable, Category="Sprite")
	int32 GetNumKeyFrames() const
	{
		return KeyFrames.Num();
	}

	// Is the specified Index within the valid range of key frames?
	UFUNCTION(BlueprintCallable, Category="Sprite")
	bool IsValidKeyFrameIndex(int32 Index) const
	{
		return KeyFrames.IsValidIndex(Index);
	}

	// Returns the key frame at the specified index, make sure the index is valid before use
	const FPaperFlipbookKeyFrame& GetKeyFrameChecked(int32 Index) const
	{
		return KeyFrames[Index];
	}

	// Search for a socket at the specified frame
	UE_API bool FindSocket(FName SocketName, int32 KeyFrameIndex, FTransform& OutLocalTransform);

	// Returns true if the flipbook has any sockets
	UE_API bool HasAnySockets() const;

	// Returns true if the flipbook has a specific named socket
	UE_API bool DoesSocketExist(FName SocketName) const;

	// Returns a list of all of the sockets
	UE_API void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const;

	// Return the default material for this flipbook
	UMaterialInterface* GetDefaultMaterial() const { return DefaultMaterial; }

	// Rebuilds cached data about the animation (such as total number of frames that the keyframes span, etc...)
	UE_API void InvalidateCachedData();

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	// Returns true if the flipbook asset contains the specified sprite asset in any frames
	UE_API bool ContainsSprite(UPaperSprite* SpriteAsset) const;

private:
	friend class FScopedFlipbookMutator;
};


// Helper class to edit properties of a UPaperFlipbook while ensuring that cached data remains up to date
class FScopedFlipbookMutator
{
public:
	float& FramesPerSecond;
	TArray<FPaperFlipbookKeyFrame>& KeyFrames;

private:
	UPaperFlipbook* SourceFlipbook;

public:
	FScopedFlipbookMutator(UPaperFlipbook* InFlipbook)
		: FramesPerSecond(InFlipbook->FramesPerSecond)
		, KeyFrames(InFlipbook->KeyFrames)
		, SourceFlipbook(InFlipbook)
	{
	}

	~FScopedFlipbookMutator()
	{
		InvalidateCachedData();
	}

	void InvalidateCachedData()
	{
		SourceFlipbook->InvalidateCachedData();
	}

	UPaperFlipbook* GetSourceFlipbook() const
	{
		return SourceFlipbook;
	}
};

#undef UE_API
