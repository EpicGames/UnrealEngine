// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "PaperFlipbookActor.generated.h"

#define UE_API PAPER2D_API

class UPaperFlipbookComponent;

/**
 * An instance of a UPaperFlipbook in a level.
 *
 * This actor is created when you drag a flipbook asset from the content browser into the level, and
 * it is just a thin wrapper around a UPaperFlipbookComponent that actually references the asset.
 */
UCLASS(MinimalAPI, ComponentWrapperClass)
class APaperFlipbookActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category=Sprite, VisibleAnywhere, BlueprintReadOnly, meta=(ExposeFunctionCategories="Sprite,Rendering,Physics,Components|Flipbook", AllowPrivateAccess="true"))
	TObjectPtr<class UPaperFlipbookComponent> RenderComponent;

public:

	// AActor interface
#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif
	// End of AActor interface

	/** Returns RenderComponent subobject **/
	FORCEINLINE class UPaperFlipbookComponent* GetRenderComponent() const { return RenderComponent; }
};

#undef UE_API
