// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"
#include "PaperFlipbookComponent.generated.h"

#define UE_API PAPER2D_API

class FStreamingTextureLevelContext;
struct FComponentSocketDescription;
struct FStreamingRenderAssetPrimitiveInfo;

class FPrimitiveSceneProxy;
class UBodySetup;
class UPaperFlipbook;
class UPaperSprite;
class UTexture;

// Event for a non-looping flipbook finishing play
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FFlipbookFinishedPlaySignature);

UCLASS(MinimalAPI, ShowCategories=(Mobility, ComponentReplication), ClassGroup=Paper2D, meta=(BlueprintSpawnableComponent))
class UPaperFlipbookComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

protected:
	/** Flipbook currently being played */
	UPROPERTY(Category=Sprite, EditAnywhere, meta=(DisplayThumbnail = "true"), ReplicatedUsing=OnRep_SourceFlipbook)
	TObjectPtr<UPaperFlipbook> SourceFlipbook;

	// DEPRECATED in 4.5: The material override for this flipbook component (if any); replaced by the Materials array inherited from UMeshComponent
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material_DEPRECATED;

	/** Current play rate of the flipbook */
	UPROPERTY(Category=Sprite, EditAnywhere)
	float PlayRate;

	/** Whether the flipbook should loop when it reaches the end, or stop */
	UPROPERTY()
	uint32 bLooping:1;

	/** If playback should move the current position backwards instead of forwards */
	UPROPERTY()
	uint32 bReversePlayback:1;

	/** Are we currently playing (moving Position) */
	UPROPERTY()
	uint32 bPlaying:1;

	/** Current position in the timeline */
	UPROPERTY()
	float AccumulatedTime;

	/** Last frame index calculated */
	UPROPERTY()
	int32 CachedFrameIndex;

	/** Vertex color to apply to the frames */
	UPROPERTY(BlueprintReadOnly, Interp, Category=Sprite)
	FLinearColor SpriteColor;

	/** The cached body setup */
	UPROPERTY(Transient)
	TObjectPtr<UBodySetup> CachedBodySetup;

public:
	/** Event called whenever a non-looping flipbook finishes playing (either reaching the beginning or the end, depending on the play direction) */
	UPROPERTY(BlueprintAssignable)
	FFlipbookFinishedPlaySignature OnFinishedPlaying;

public:
	/** Change the flipbook used by this instance (will reset the play time to 0 if it is a new flipbook). */
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API virtual bool SetFlipbook(class UPaperFlipbook* NewFlipbook);

	/** Gets the flipbook used by this instance. */
	UFUNCTION(BlueprintPure, Category="Sprite")
	UE_API virtual UPaperFlipbook* GetFlipbook();

	/** Returns the current color of the sprite */
	UFUNCTION(BlueprintPure, Category="Sprite")
	FLinearColor GetSpriteColor() const { return SpriteColor; }
	
	/** Set color of the sprite */
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void SetSpriteColor(FLinearColor NewColor);

	/** Start playback of flipbook */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void Play();

	/** Start playback of flipbook from the start */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void PlayFromStart();

	/** Start playback of flipbook in reverse */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void Reverse();

	/** Start playback of flipbook in reverse from the end */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void ReverseFromEnd();

	/** Stop playback of flipbook */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void Stop();

	/** Get whether this flipbook is playing or not. */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API bool IsPlaying() const;

	/** Get whether we are reversing or not */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API bool IsReversing() const;

	/** Jump to a position in the flipbook (expressed in frames). If bFireEvents is true, event functions will fire, otherwise they will not. */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void SetPlaybackPositionInFrames(int32 NewFramePosition, bool bFireEvents);

	/** Get the current playback position (in frames) of the flipbook */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API int32 GetPlaybackPositionInFrames() const;

	/** Jump to a position in the flipbook (expressed in seconds). If bFireEvents is true, event functions will fire, otherwise they will not. */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void SetPlaybackPosition(float NewPosition, bool bFireEvents);

	/** Get the current playback position (in seconds) of the flipbook */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API float GetPlaybackPosition() const;

	/** true means we should loop, false means we should not. */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void SetLooping(bool bNewLooping);

	/** Get whether we are looping or not */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API bool IsLooping() const;

	/** Sets the new play rate for this flipbook */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void SetPlayRate(float NewRate);

	/** Get the current play rate for this flipbook */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API float GetPlayRate() const;

	/** Set the new playback position time to use */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API void SetNewTime(float NewTime);

	/** Get length of the flipbook (in seconds) */
	UFUNCTION(BlueprintCallable, Category="Components|Flipbook")
	UE_API float GetFlipbookLength() const;

	/** Get length of the flipbook (in frames) */
	UFUNCTION(BlueprintCallable, Category = "Components|Flipbook")
	UE_API int32 GetFlipbookLengthInFrames() const;

	/** Get the nominal framerate that the flipbook will be played back at (ignoring PlayRate), in frames per second */
	UFUNCTION(BlueprintCallable, Category = "Components|Flipbook")
	UE_API float GetFlipbookFramerate() const;

protected:
	UFUNCTION()
	UE_API void OnRep_SourceFlipbook(class UPaperFlipbook* OldFlipbook);

	UE_API void CalculateCurrentFrame();
	UE_API UPaperSprite* GetSpriteAtCachedIndex() const;

	UE_API void TickFlipbook(float DeltaTime);
	UE_API void FlipbookChangedPhysicsState();

private:
		//disable parallel add to scene for paper2d
		void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override
		{
			Super::CreateRenderState_Concurrent(nullptr);
		}

public:
	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	// End of UObject interface

	// UActorComponent interface
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual void SendRenderDynamicData_Concurrent() override;
	UE_API virtual const UObject* AdditionalStatObject() const override;
#if WITH_EDITOR
	UE_API virtual void CheckForErrors() override;
#endif
	// End of UActorComponent interface

	// USceneComponent interface
	UE_API virtual bool HasAnySockets() const override;
	UE_API virtual bool DoesSocketExist(FName InSocketName) const override;
	UE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	UE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	// End of USceneComponent interface

	// UPrimitiveComponent interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel) override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	UE_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures) const override;
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UBodySetup* GetBodySetup() override;
	// End of UPrimitiveComponent interface
};

#undef UE_API
