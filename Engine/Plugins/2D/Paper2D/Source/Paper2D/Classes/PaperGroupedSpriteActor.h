// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "PaperGroupedSpriteActor.generated.h"

#define UE_API PAPER2D_API

class UPaperGroupedSpriteComponent;

/**
 * A group of sprites that will be rendered and culled as a single unit
 *
 * This actor is created when you Merge several sprite components together.
 * it is just a thin wrapper around a UPaperGroupedSpriteComponent.
 */
UCLASS(MinimalAPI, ComponentWrapperClass)
class APaperGroupedSpriteActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category=Sprite, VisibleAnywhere, BlueprintReadOnly, meta=(ExposeFunctionCategories = "Sprite,Rendering,Physics,Components|Sprite", AllowPrivateAccess="true"))
	TObjectPtr<UPaperGroupedSpriteComponent> RenderComponent;

public:
	// AActor interface
#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif
	// End of AActor interface

	/** Returns RenderComponent subobject **/
	FORCEINLINE UPaperGroupedSpriteComponent* GetRenderComponent() const { return RenderComponent; }
};

#undef UE_API
