// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"

#include "PaperGroupedSpriteComponent.generated.h"

#define UE_API PAPER2D_API

class FGroupedSpriteSceneProxy;
class FPrimitiveSceneProxy;
class UPaperSprite;
class UTexture;
struct FNavigableGeometryExport;
struct FNavigationRelevantData;

USTRUCT()
struct FSpriteInstanceData
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Instances)
	FMatrix Transform;

	UPROPERTY(EditAnywhere, Category=Instances)
	TObjectPtr<UPaperSprite> SourceSprite;

	UPROPERTY(EditAnywhere, Category=Instances)
	FColor VertexColor;

	UPROPERTY(EditAnywhere, Category=Instances)
	int32 MaterialIndex;

	bool IsValidInstance() const
	{
		return SourceSprite != nullptr;
	}

public:
	FSpriteInstanceData()
		: Transform(FMatrix::Identity)
		, SourceSprite(nullptr)
		, VertexColor(FColor::White)
		, MaterialIndex(INDEX_NONE)
	{
	}
};

/**
 * A component that handles rendering and collision for many instances of one or more UPaperSprite assets.
 *
 * @see UPrimitiveComponent, UPaperSprite
 */

UCLASS(MinimalAPI, ShowCategories=(Mobility), ClassGroup=Paper2D, meta=(BlueprintSpawnableComponent), EarlyAccessPreview)
class UPaperGroupedSpriteComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

protected:
	/** Array of materials used by the instances */
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> InstanceMaterials;

	/** Array of instances */
	UPROPERTY(EditAnywhere, DisplayName="Instances", Category=Instances, meta=(MakeEditWidget=true))
	TArray<FSpriteInstanceData> PerInstanceSpriteData;

	/** Physics representation of the instance bodies */
	TArray<FBodyInstance*> InstanceBodies;

public:

	/** Add an instance to this component. Transform can be given either in the local space of this component or world space.  */
	UFUNCTION(BlueprintCallable, Category="Components|Sprite")
	UE_API int32 AddInstance(const FTransform& Transform, UPaperSprite* Sprite, bool bWorldSpace = false, FLinearColor Color = FLinearColor::White);

	UE_API virtual int32 AddInstanceWithMaterial(const FTransform& Transform, UPaperSprite* Sprite, UMaterialInterface* MaterialOverride = nullptr, bool bWorldSpace = false, FLinearColor Color = FLinearColor::White);

	/** Get the transform for the instance specified. Instance is returned in local space of this component unless bWorldSpace is set.  Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|Sprite")
	UE_API bool GetInstanceTransform(int32 InstanceIndex, FTransform& OutInstanceTransform, bool bWorldSpace = false) const;
	
	UE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;

	/** Update the transform for the instance specified. Instance is given in local space of this component unless bWorldSpace is set.  Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|Sprite")
	UE_API virtual bool UpdateInstanceTransform(int32 InstanceIndex, const FTransform& NewInstanceTransform, bool bWorldSpace=false, bool bMarkRenderStateDirty=true, bool bTeleport=false);

	/** Update the color for the instance specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|Sprite")
	UE_API virtual bool UpdateInstanceColor(int32 InstanceIndex, FLinearColor NewInstanceColor, bool bMarkRenderStateDirty = true);

	/** Remove the instance specified. Returns True on success. */
	UFUNCTION(BlueprintCallable, Category = "Components|Sprite")
	UE_API virtual bool RemoveInstance(int32 InstanceIndex);
	
	/** Clear all instances being rendered by this component */
	UFUNCTION(BlueprintCallable, Category="Components|Sprite")
	UE_API virtual void ClearInstances();
	
	/** Get the number of instances in this component */
	UFUNCTION(BlueprintCallable, Category = "Components|Sprite")
	UE_API int32 GetInstanceCount() const;

	/** Sort all instances by their world space position along the specified axis */
	UFUNCTION(BlueprintCallable, Category = "Components|Sprite")
	UE_API void SortInstancesAlongAxis(FVector WorldSpaceSortAxis);

	// UActorComponent interface
	UE_API virtual bool ShouldCreatePhysicsState() const override;
protected:
	UE_API virtual void OnCreatePhysicsState() override;
	UE_API virtual void OnDestroyPhysicsState() override;
public:
	UE_API virtual const UObject* AdditionalStatObject() const override;
#if WITH_EDITOR
	UE_API virtual void CheckForErrors() override;
#endif
	// End of UActorComponent interface 

	// UPrimitiveComponent interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual bool CanEditSimulatePhysics() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& BoundTransform) const override;
	UE_API virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel) override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	UE_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	// End of UPrimitiveComponent interface

	// UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif
	// End of UObject interface

	// Returns true if this component references the specified sprite asset
	UE_API bool ContainsSprite(UPaperSprite* SpriteAsset) const;

	// Adds all referenced sprite assets to the specified list
	UE_API void GetReferencedSpriteAssets(TArray<UObject*>& InOutObjects) const;

	/** Handles request from navigation system to gather instance transforms in a specific area box */
	UE_API void GetNavigationPerInstanceTransforms(const FBox& AreaBox, TArray<FTransform>& OutInstanceTransforms) const;

	const TArray<FSpriteInstanceData>& GetPerInstanceSpriteData() const
	{
		return PerInstanceSpriteData;
	}

protected:
	/**
	 * Transfers ownership of instance render data to a render thread
	 * instance render data will be released in scene proxy dtor or on render thread task
	 */
	UE_API void ReleasePerInstanceRenderData();

	/** Creates body instances for all instances owned by this component */
	UE_API void CreateAllInstanceBodies();

	/** Terminate all body instances owned by this component */
	UE_API void ClearAllInstanceBodies();

	/** Sets up new instance data to sensible defaults, creates physics counterparts if possible */
	UE_API void SetupNewInstanceData(FSpriteInstanceData& InOutNewInstanceData, int32 InInstanceIndex, const FTransform& InInstanceTransform, UPaperSprite* InSprite, UMaterialInterface* MaterialOverride, const FColor& InColor);

	/** Creates a body instance for the specified instance data if that sprite has defined collision */
	UE_API FBodyInstance* InitInstanceBody(int32 InstanceIndex, const FSpriteInstanceData& InstanceData, FPhysScene* PhysScene);

	/** Invalidates the render and collision state */
	UE_API void RebuildInstances();

	/** Creates the material list from the instances */
	UE_API void RebuildMaterialList();

	/** Adds to the material list from a single sprite */
	UE_API int32 UpdateMaterialList(UPaperSprite* Sprite, UMaterialInterface* MaterialOverride);

	friend FGroupedSpriteSceneProxy;
};

#undef UE_API
