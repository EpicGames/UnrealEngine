// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperRuntimeSettings.generated.h"

/**
 * Implements the settings for the Paper2D plugin.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UPaperRuntimeSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	// Enables experimental *incomplete and unsupported* texture atlas groups that sprites can be assigned to
	UPROPERTY(EditAnywhere, config, Category=Experimental)
	bool bEnableSpriteAtlasGroups;

	// Enables experimental *incomplete and unsupported* 2D terrain spline editing. Note: You need to restart the editor when enabling this setting for the change to fully take effect.
	UPROPERTY(EditAnywhere, config, Category=Experimental, meta=(ConfigRestartRequired=true))
	bool bEnableTerrainSplineEditing;

	// Enables automatic resizing of various sprite data that is authored in texture space if the source texture gets resized (sockets, the pivot, render and collision geometry, etc...)
	UPROPERTY(EditAnywhere, config, Category=Settings)
	bool bResizeSpriteDataToMatchTextures;
};
