// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/Interface_CollisionDataProvider.h"
#include "SpriteEditorOnlyTypes.h"
#include "Slate/SlateTextureAtlasInterface.h"
#include "PaperSprite.generated.h"

#define UE_API PAPER2D_API

class UTexture2D;
struct FComponentSocketDescription;

class UMaterialInterface;
class UPaperSpriteAtlas;

//@TODO: Should have some nice UI and enforce unique names, etc...
USTRUCT(BlueprintType)
struct FPaperSpriteSocket
{
	GENERATED_USTRUCT_BODY()

	// Transform in pivot space (*not* texture space)
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Sockets)
	FTransform LocalTransform;

	// Name of the socket
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Sockets)
	FName SocketName;
};

typedef TArray<class UTexture*, TInlineAllocator<4>> FAdditionalSpriteTextureArray;

/**
 * Sprite Asset
 *
 * Stores the data necessary to render a single 2D sprite (from a region of a texture)
 * Can also contain collision shapes for the sprite.
 *
 * @see UPaperSpriteComponent
 */

UCLASS(MinimalAPI, BlueprintType, meta=(DisplayThumbnail = "true"))
class UPaperSprite : public UObject, public IInterface_CollisionDataProvider, public ISlateTextureAtlasInterface
{
	GENERATED_UCLASS_BODY()

protected:

#if WITH_EDITORONLY_DATA
	// Origin within SourceImage, prior to atlasing
	UPROPERTY(Category=Sprite, EditAnywhere, AdvancedDisplay, meta=(EditCondition="bTrimmedInSourceImage"))
	FVector2D OriginInSourceImageBeforeTrimming;

	// Dimensions of SourceImage
	UPROPERTY(Category=Sprite, EditAnywhere, AdvancedDisplay, meta=(EditCondition="bTrimmedInSourceImage"))
	FVector2D SourceImageDimensionBeforeTrimming;

	// This texture is trimmed, consider the values above
	UPROPERTY(Category=Sprite, EditAnywhere, AdvancedDisplay)
	bool bTrimmedInSourceImage;

	// This texture is rotated in the atlas
	UPROPERTY(Category=Sprite, EditAnywhere, AdvancedDisplay)
	bool bRotatedInSourceImage;

	// Dimension of the texture when this sprite was created
	// Used when the sprite is resized at some point
	UPROPERTY(Category=Sprite, EditAnywhere, AdvancedDisplay)
	FVector2D SourceTextureDimension;
#endif

#if WITH_EDITORONLY_DATA
	// Position within SourceTexture (in pixels)
	UPROPERTY(Category=Sprite, EditAnywhere, AssetRegistrySearchable)
	FVector2D SourceUV;

	// Dimensions within SourceTexture (in pixels)
	UPROPERTY(Category=Sprite, EditAnywhere, AssetRegistrySearchable)
	FVector2D SourceDimension;

	// The source texture that the sprite comes from
	UPROPERTY(Category=Sprite, EditAnywhere, AssetRegistrySearchable)
	TSoftObjectPtr<UTexture2D> SourceTexture;

	UPROPERTY(Transient)
	mutable TObjectPtr<UTexture2D> SourceTextureCacheNeverSerialized;
#endif

	// Additional source textures for other slots
	UPROPERTY(Category=Sprite, EditAnywhere, AssetRegistrySearchable, meta=(DisplayName="Additional Textures"))
	TArray<TObjectPtr<UTexture>> AdditionalSourceTextures;

	// Position within BakedSourceTexture (in pixels)
	UPROPERTY()
	FVector2D BakedSourceUV;

	// Dimensions within BakedSourceTexture (in pixels)
	UPROPERTY()
	FVector2D BakedSourceDimension;

	UPROPERTY()
	TObjectPtr<UTexture2D> BakedSourceTexture;

	// The material to use on a sprite instance if not overridden (this is the default material when only one is being used, and is the translucent/masked material for Diced render geometry, slot 0)
	UPROPERTY(Category=Sprite, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UMaterialInterface> DefaultMaterial;

	// The alternate material to use on a sprite instance if not overridden (this is only used for Diced render geometry, and will be the opaque material in that case, slot 1)
	UPROPERTY(Category=Rendering, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UMaterialInterface> AlternateMaterial;

	// List of sockets on this sprite
	UPROPERTY(Category=Sprite, EditAnywhere)
	TArray<FPaperSpriteSocket> Sockets;

	// Collision domain (no collision, 2D, or 3D)
	UPROPERTY(Category=Collision, EditAnywhere)
	TEnumAsByte<ESpriteCollisionMode::Type> SpriteCollisionDomain;

	// The scaling factor between pixels and Unreal units (cm) (e.g., 0.64 would make a 64 pixel wide sprite take up 100 cm)
	UPROPERTY(Category=Sprite, EditAnywhere, meta = (DisplayName = "Pixels per unit"))
	float PixelsPerUnrealUnit;

public:
	// Baked physics data.
	UPROPERTY(EditAnywhere, Category=NeverShown)
	TObjectPtr<class UBodySetup> BodySetup;

#if WITH_EDITORONLY_DATA

protected:
	// Pivot mode
	UPROPERTY(Category=Sprite, EditAnywhere)
	TEnumAsByte<ESpritePivotMode::Type> PivotMode;

	// Custom pivot point (relative to the sprite rectangle)
	UPROPERTY(Category=Sprite, EditAnywhere)
	FVector2D CustomPivotPoint;

	// Should the pivot be snapped to a pixel boundary?
	UPROPERTY(Category=Sprite, EditAnywhere, AdvancedDisplay)
	bool bSnapPivotToPixelGrid;

	// Custom collision geometry polygons (in texture space)
	UPROPERTY(Category=Collision, EditAnywhere)
	FSpriteGeometryCollection CollisionGeometry;

	// The extrusion thickness of collision geometry when using a 3D collision domain
	UPROPERTY(Category=Collision, EditAnywhere)
	float CollisionThickness;

	// Custom render geometry polygons (in texture space)
	UPROPERTY(Category=Rendering, EditAnywhere)
	FSpriteGeometryCollection RenderGeometry;

	// Spritesheet group that this sprite belongs to
	UPROPERTY(Category=Sprite, EditAnywhere, AssetRegistrySearchable)
	TObjectPtr<class UPaperSpriteAtlas> AtlasGroup;

	// The previous spritesheet group this belonged to
	// To make sure we remove ourselves from it if changed or nulled out
	TSoftObjectPtr<class UPaperSpriteAtlas> PreviousAtlasGroup;

#endif

public:
	// The point at which the alternate material takes over in the baked render data (or INDEX_NONE)
	UPROPERTY()
	int32 AlternateMaterialSplitIndex;

	// Baked render data (triangle vertices, stored as XY UV tuples)
	//   XY is the XZ position in world space, relative to the pivot
	//   UV is normalized (0..1)
	//   There should always be a multiple of three elements in this array
	UPROPERTY()
	TArray<FVector4> BakedRenderData;

public:

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End of UObject interface

	// ISlateTextureAtlasInterface interface
	UE_API virtual FSlateAtlasData GetSlateAtlasData() const override;
	// End of ISlateTextureAtlasInterface interface

#if WITH_EDITOR
	UE_API FVector2D ConvertTextureSpaceToPivotSpace(FVector2D Input) const;
	UE_API FVector2D ConvertPivotSpaceToTextureSpace(FVector2D Input) const;
	UE_API FVector ConvertTextureSpaceToPivotSpace(FVector Input) const;
	UE_API FVector ConvertPivotSpaceToTextureSpace(FVector Input) const;
	UE_API FVector2D ConvertWorldSpaceToTextureSpace(const FVector& WorldPoint) const;
	UE_API FVector2D ConvertWorldSpaceDeltaToTextureSpace(const FVector& WorldDelta, bool bIgnoreRotation = false) const;

	// World space WRT the sprite editor *only*
	UE_API FVector ConvertTextureSpaceToWorldSpace(const FVector2D& SourcePoint) const;
	UE_API FTransform GetPivotToWorld() const;

	// Returns the raw pivot position (ignoring pixel snapping)
	UE_API FVector2D GetRawPivotPosition() const;

	// Returns the current pivot position in texture space
	UE_API FVector2D GetPivotPosition() const;

	// Returns the extrusion thickness of collision geometry when using a 3D collision domain
	float GetCollisionThickness() const
	{
		return CollisionThickness;
	}

	// Returns the collision domain (no collision, 2D, or 3D)
	ESpriteCollisionMode::Type GetSpriteCollisionDomain() const
	{
		return SpriteCollisionDomain;
	}

	// Rescale properties to handle source texture size change
	UE_API void RescaleSpriteData(UTexture2D* Texture);
	UE_API bool NeedRescaleSpriteData();

	/** This is a generic "rebuild all" function that calls RebuildCollisionData() and then RebuildRenderData(). */
	UE_API void RebuildData();
	UE_API void RebuildCollisionData();
	UE_API void RebuildRenderData();

	UE_API void ExtractSourceRegionFromTexturePoint(const FVector2D& Point);

	// Evaluates the SourceUV/SourceDimensons rectangle, finding the tightest bounds that still include all pixels with alpha above AlphaThreshold.
	// The output box position is the top left corner of the box, not the center.
	UE_API void FindTextureBoundingBox(float AlphaThreshold, /*out*/ FVector2D& OutBoxPosition, /*out*/ FVector2D& OutBoxSize);

	static UE_API void FindContours(const FIntPoint& ScanPos, const FIntPoint& ScanSize, float AlphaThreshold, float Detail, UTexture2D* Texture, TArray< TArray<FIntPoint> >& OutPoints);
	static UE_API void ExtractRectsFromTexture(UTexture2D* Texture, TArray<FIntRect>& OutRects);
	UE_API void BuildGeometryFromContours(FSpriteGeometryCollection& GeomOwner);

	UE_API void CreatePolygonFromBoundingBox(FSpriteGeometryCollection& GeomOwner, bool bUseTightBounds);

	// Reinitializes this sprite (NOTE: Does not register existing components in the world)
	UE_API void InitializeSprite(const FSpriteAssetInitParameters& InitParams, bool bRebuildData = true);

	UE_API void SetTrim(bool bTrimmed, const FVector2D& OriginInSourceImage, const FVector2D& SourceImageDimension, bool bRebuildData = true);
	UE_API void SetRotated(bool bRotated, bool bRebuildData = true);
	UE_API void SetPivotMode(ESpritePivotMode::Type PivotMode, FVector2D CustomTextureSpacePivot, bool bRebuildData = true);
	
	// Returns the Origin within SourceImage, prior to atlasing
	FVector2D GetOriginInSourceImageBeforeTrimming() const { return OriginInSourceImageBeforeTrimming; }

	// Returns the Dimensions of SourceImage prior to trimming
	FVector2D GetSourceImageDimensionBeforeTrimming() const { return SourceImageDimensionBeforeTrimming; }

	// Returns true if this sprite is trimmed from the original texture, meaning that the source image
	// dimensions and origin in the source image may not be the same as the final results for the sprite
	// (empty alpha=0 pixels were trimmed from the exterior region)
	bool IsTrimmedInSourceImage() const { return bTrimmedInSourceImage; }

	// This texture is rotated in the atlas
	bool IsRotatedInSourceImage() const { return bRotatedInSourceImage; }

	ESpritePivotMode::Type GetPivotMode(FVector2D& OutCustomTextureSpacePivot) const { OutCustomTextureSpacePivot = CustomPivotPoint; return PivotMode; }

	FVector2D GetSourceUV() const { return SourceUV; }
	FVector2D GetSourceSize() const { return SourceDimension; }
	UE_API UTexture2D* GetSourceTexture() const;

	const UPaperSpriteAtlas* GetAtlasGroup() const { return AtlasGroup; }
#endif

#if WITH_EDITOR
	// Called when an object is re-imported in the editor
	UE_API void OnObjectReimported(UTexture2D* InObject);
#endif

#if WITH_EDITOR
	// Make sure all socket names are valid
	// All duplicate / empty names will be made unique
	UE_API void ValidateSocketNames();

	// Removes the specified socket
	UE_API void RemoveSocket(FName SocketName);
#endif

	// Return the scaling factor between pixels and Unreal units (cm)
	float GetPixelsPerUnrealUnit() const { return PixelsPerUnrealUnit; }

	// Return the scaling factor between Unreal units (cm) and pixels
	float GetUnrealUnitsPerPixel() const { return 1.0f / PixelsPerUnrealUnit; }

	// Returns the texture this should be rendered with
	UE_API UTexture2D* GetBakedTexture() const;

	// Returns the list of additional source textures this should be rendered with
	UE_API void GetBakedAdditionalSourceTextures(FAdditionalSpriteTextureArray& OutTextureList) const;

	// Return the default material for this sprite
	UMaterialInterface* GetDefaultMaterial() const { return DefaultMaterial; }

	// Return the alternate material for this sprite
	UMaterialInterface* GetAlternateMaterial() const { return AlternateMaterial; }

	// Returns either the default material (index 0) or alternate material (index 1)
	UE_API UMaterialInterface* GetMaterial(int32 MaterialIndex) const;

	// Returns the number of materials (1 or 2, depending on if there is alternate geometry)
	UE_API int32 GetNumMaterials() const;

	// Returns the render bounds of this sprite
	UE_API FBoxSphereBounds GetRenderBounds() const;

	// Search for a socket (note: do not cache this pointer; it's unsafe if the Socket array is edited)
	UE_API FPaperSpriteSocket* FindSocket(FName SocketName);

	// Returns true if the sprite has any sockets
	bool HasAnySockets() const { return Sockets.Num() > 0; }
	
	// Returns a list of all of the sockets
	UE_API void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const;

	// IInterface_CollisionDataProvider interface
	UE_API virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	UE_API virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	// End of IInterface_CollisionDataProvider

#if WITH_EDITOR
	static UE_API FName GetSourceTextureMemberName();
#endif

protected:
	UE_API void RefreshBakedData();

	//@TODO: HACKERY:
	friend class FSpriteEditorViewportClient;
	friend class FSpriteDetailsCustomization;
	friend class FSpriteSelectedVertex;
	friend class FSpriteSelectedShape;
	friend class FSpriteSelectedEdge;
	friend class FSpriteSelectedSourceRegion;
	friend struct FPaperAtlasGenerator;
	friend class FSingleTileEditorViewportClient;
};

#undef UE_API
