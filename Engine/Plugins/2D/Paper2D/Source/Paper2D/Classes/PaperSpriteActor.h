// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "PaperSpriteActor.generated.h"

#define UE_API PAPER2D_API

class UPaperSpriteComponent;

/**
 * An instance of a UPaperSprite in a level.
 *
 * This actor is created when you drag a sprite asset from the content browser into the level, and
 * it is just a thin wrapper around a UPaperSpriteComponent that actually references the asset.
 */
UCLASS(MinimalAPI, ComponentWrapperClass, meta = (ChildCanTick))
class APaperSpriteActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category=Sprite, VisibleAnywhere, BlueprintReadOnly, meta=(ExposeFunctionCategories="Sprite,Rendering,Physics,Components|Sprite", AllowPrivateAccess="true"))
	TObjectPtr<class UPaperSpriteComponent> RenderComponent;
public:

	// AActor interface
#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif
	// End of AActor interface

	/** Returns RenderComponent subobject **/
	FORCEINLINE class UPaperSpriteComponent* GetRenderComponent() const { return RenderComponent; }
};

#undef UE_API
