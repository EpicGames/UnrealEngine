// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperSpriteAtlas.generated.h"

class UTexture;
enum TextureCompressionSettings : int;
enum TextureFilter : int;

USTRUCT()
struct FPaperSpriteAtlasSlot
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TSoftObjectPtr<class UPaperSprite> SpriteRef;

	UPROPERTY()
	int32 AtlasIndex = 0;

	UPROPERTY()
	int32 X = 0;

	UPROPERTY()
	int32 Y = 0;

	UPROPERTY()
	int32 Width = 0;

	UPROPERTY()
	int32 Height = 0;

	bool IsValid() { return AtlasIndex >= 0; }
};

UENUM()
enum class EPaperSpriteAtlasPadding : uint8
{
	/** Dilate the texture to pad the atlas. */
	DilateBorder,
	/** Padding border filled with zeros. */
	PadWithZero,
	//TODO: Tiled padding
	//TODO: Sample outside bounds from source texture (for seamless reconstruction of a larger image)
};



// Groups together a set of sprites that will try to share the same texture atlas (allowing them to be combined into a single draw call)
UCLASS(MinimalAPI, BlueprintType, Experimental, meta=(DisplayThumbnail = "true"))
class UPaperSpriteAtlas : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	// Description of this atlas, which shows up in the content browser tooltip
	UPROPERTY(EditAnywhere, Category=General)
	FString AtlasDescription;

	// Maximum atlas page width (single pages might be smaller)
	UPROPERTY(EditAnywhere, Category = General)
	int32 MaxWidth;

	// Maximum atlas page height (single pages might be smaller)
	UPROPERTY(EditAnywhere, Category = General)
	int32 MaxHeight;

	// Maximum atlas page height (single pages might be smaller)
	UPROPERTY(EditAnywhere, Category = General)
	int32 MipCount;

	// The type of padding performed on this atlas
	UPROPERTY(EditAnywhere, Category = General)
	EPaperSpriteAtlasPadding PaddingType;

	// The number of pixels of padding
	UPROPERTY(EditAnywhere, Category = General)
	int32 Padding;

	// Compression settings to use on atlas texture
	UPROPERTY(EditAnywhere, Category=AtlasTexture)
	TEnumAsByte<TextureCompressionSettings> CompressionSettings;

	// Texture filtering mode when sampling these textures
	UPROPERTY(EditAnywhere, Category=AtlasTexture)
	TEnumAsByte<TextureFilter> Filter;

	// List of generated atlas textures
	UPROPERTY(VisibleAnywhere, Category=AtlasTexture)
	TArray<TObjectPtr<UTexture>> GeneratedTextures;

	// The GUID of the atlas group, used to match up sprites that belong to this group even thru atlas renames
	UPROPERTY(VisibleAnywhere, Category=General)
	FGuid AtlasGUID;

	// Slots in the atlas
	UPROPERTY(EditAnywhere, Category=General)
	bool bRebuildAtlas;

	// Slots in the atlas
	UPROPERTY()
	TArray<FPaperSpriteAtlasSlot> AtlasSlots;

	// Track the number of incremental builds
	UPROPERTY()
	int32 NumIncrementalBuilds;

	// Values used when building this atlas
	UPROPERTY()
	int32 BuiltWidth;

	UPROPERTY()
	int32 BuiltHeight;

	UPROPERTY()
	int32 BuiltPadding;
#endif


public:
	// UObject interface
#if WITH_EDITORONLY_DATA
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
#endif
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// End of UObject interface
};
