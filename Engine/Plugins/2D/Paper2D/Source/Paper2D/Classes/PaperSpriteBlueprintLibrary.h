// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "PaperSpriteBlueprintLibrary.generated.h"

struct FSlateBrush;

class UPaperSprite;

/**
 *
 */
UCLASS(meta=(ScriptName="PaperSpriteLibrary"))
class UPaperSpriteBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static FSlateBrush MakeBrushFromSprite(UPaperSprite* Sprite, int32 Width, int32 Height);
};
