// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"

#include "PaperSpriteComponent.generated.h"

#define UE_API PAPER2D_API

class FStreamingTextureLevelContext;
struct FComponentSocketDescription;
struct FStreamingRenderAssetPrimitiveInfo;

class FPrimitiveSceneProxy;
class UPaperSprite;
class UTexture;

/**
 * A component that handles rendering and collision for a single instance of a UPaperSprite asset.
 *
 * This component is created when you drag a sprite asset from the content browser into a Blueprint, or
 * contained inside of the actor created when you drag one into the level.
 *
 * @see UPrimitiveComponent, UPaperSprite
 */

UCLASS(MinimalAPI, ShowCategories=(Mobility), ClassGroup=Paper2D, meta=(BlueprintSpawnableComponent))
class UPaperSpriteComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

protected:
	// The sprite asset used by this component
	UPROPERTY(Category=Sprite, EditAnywhere, BlueprintReadOnly, meta=(DisplayThumbnail = "true"))
	TObjectPtr<UPaperSprite> SourceSprite;

	// DEPRECATED in 4.4: The material override for this sprite component (if any); replaced by the Materials array inherited from UMeshComponent
	UPROPERTY()
	TObjectPtr<UMaterialInterface> MaterialOverride_DEPRECATED;

	// The color of the sprite (passed to the sprite material as a vertex color)
	UPROPERTY(BlueprintReadOnly, Interp, Category=Sprite)
	FLinearColor SpriteColor;

public:
	/** Change the PaperSprite used by this instance. */
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API virtual bool SetSprite(class UPaperSprite* NewSprite);

	/** Gets the PaperSprite used by this instance. */
	UFUNCTION(BlueprintPure, Category="Sprite")
	UE_API virtual UPaperSprite* GetSprite();

	/** Returns the current color of the sprite */
	FLinearColor GetSpriteColor() const { return SpriteColor; }

	/** Set color of the sprite */
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void SetSpriteColor(FLinearColor NewColor);

	// Returns the wireframe color to use for this component.
	UE_API FLinearColor GetWireframeColor() const;

public:
	// UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	// End of UObject interface

	// UActorComponent interface
	UE_API virtual void SendRenderDynamicData_Concurrent() override;
	UE_API virtual const UObject* AdditionalStatObject() const override;
#if WITH_EDITOR
	UE_API virtual void CheckForErrors() override;
#endif
	// End of UActorComponent interface

	// USceneComponent interface
	UE_API virtual bool HasAnySockets() const override;
	UE_API virtual bool DoesSocketExist(FName InSocketName) const override;
	UE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	UE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	// End of USceneComponent interface

	// UPrimitiveComponent interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual class UBodySetup* GetBodySetup() override;
	UE_API virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel) override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	UE_API virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	UE_API virtual int32 GetNumMaterials() const override;
	// End of UPrimitiveComponent interface

#if WITH_EDITOR
	UE_API void SetTransientTextureOverride(const UTexture* TextureToModifyOverrideFor, UTexture* OverrideTexture);
#endif

protected:
	friend class FPaperSpriteSceneProxy;
};

#undef UE_API
