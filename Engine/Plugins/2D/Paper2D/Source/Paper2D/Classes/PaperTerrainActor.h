// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "PaperTerrainActor.generated.h"

#define UE_API PAPER2D_API

class UPaperTerrainComponent;
class UPaperTerrainSplineComponent;

/**
 * An instance of a piece of 2D terrain in the level
 */

UCLASS(MinimalAPI, BlueprintType, Experimental)
class APaperTerrainActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY()
	TObjectPtr<class USceneComponent> DummyRoot;

	UPROPERTY()
	TObjectPtr<class UPaperTerrainSplineComponent> SplineComponent;

	UPROPERTY(Category=Sprite, VisibleAnywhere, BlueprintReadOnly, meta=(ExposeFunctionCategories="Sprite,Rendering,Physics,Components|Sprite", AllowPrivateAccess="true"))
	TObjectPtr<class UPaperTerrainComponent> RenderComponent;
public:

	// AActor interface
#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif
	// End of AActor interface

	/** Returns DummyRoot subobject **/
	FORCEINLINE class USceneComponent* GetDummyRoot() const { return DummyRoot; }
	/** Returns SplineComponent subobject **/
	FORCEINLINE class UPaperTerrainSplineComponent* GetSplineComponent() const { return SplineComponent; }
	/** Returns RenderComponent subobject **/
	FORCEINLINE class UPaperTerrainComponent* GetRenderComponent() const { return RenderComponent; }
};

#undef UE_API
