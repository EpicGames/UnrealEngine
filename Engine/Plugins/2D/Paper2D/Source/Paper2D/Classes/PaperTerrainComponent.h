// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "PaperTerrainComponent.generated.h"

#define UE_API PAPER2D_API

class UPaperSprite;
namespace ESpriteCollisionMode { enum Type : int; }
struct FSpriteDrawCallRecord;

class FPrimitiveSceneProxy;
class UBodySetup;
class UMaterialInterface;
struct FPaperTerrainMaterialRule;

struct FPaperTerrainSpriteGeometry
{
	TArray<FSpriteDrawCallRecord> Records;
	UMaterialInterface* Material;
	int32 DrawOrder;
};

struct FTerrainSpriteStamp
{
	const UPaperSprite* Sprite;
	float NominalWidth;
	float Time;
	float Scale;
	bool bCanStretch;

	FTerrainSpriteStamp(const UPaperSprite* InSprite, float InTime, bool bIsEndCap);
};

struct FTerrainSegment
{
	float StartTime;
	float EndTime;
	const FPaperTerrainMaterialRule* Rule;
	TArray<FTerrainSpriteStamp> Stamps;

	FTerrainSegment();
	void RepositionStampsToFillSpace();
};

/**
 * The terrain visualization component for an associated spline component.
 * This takes a 2D terrain material and instances sprite geometry along the spline path.
 */

UCLASS(MinimalAPI, BlueprintType, Experimental)
class UPaperTerrainComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

public:
	/** The terrain material to apply to this component (set of rules for which sprites are used on different surfaces or the interior) */
	UPROPERTY(Category=Sprite, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<class UPaperTerrainMaterial> TerrainMaterial;

	UPROPERTY(Category = Sprite, EditAnywhere, BlueprintReadOnly)
	bool bClosedSpline;

	UPROPERTY(Category = Sprite, EditAnywhere, BlueprintReadOnly, meta = (EditCondition = "bClosedSpline"))
	bool bFilledSpline;

	UPROPERTY()
	TObjectPtr<class UPaperTerrainSplineComponent> AssociatedSpline;

	/** Random seed used for choosing which spline meshes to use. */
	UPROPERTY(Category=Sprite, EditAnywhere)
	int32 RandomSeed;

	/** The overlap amount between segments */
	UPROPERTY(Category=Sprite, EditAnywhere)
	float SegmentOverlapAmount;

protected:
	/** The color of the terrain (passed to the sprite material as a vertex color) */
	UPROPERTY(Category=Sprite, BlueprintReadOnly, Interp)
	FLinearColor TerrainColor;

	/** Number of steps per spline segment to place in the reparameterization table */
	UPROPERTY(Category=Sprite, EditAnywhere, meta=(ClampMin=4, UIMin=4), AdvancedDisplay)
	int32 ReparamStepsPerSegment;

	/** Collision domain (no collision, 2D (experimental), or 3D) */
	UPROPERTY(Category=Collision, EditAnywhere)
	TEnumAsByte<ESpriteCollisionMode::Type> SpriteCollisionDomain;

	/** The extrusion thickness of collision geometry when using a 3D collision domain */
	UPROPERTY(Category=Collision, EditAnywhere)
	float CollisionThickness;

public:
	// UObject interface
	UE_API virtual const UObject* AdditionalStatObject() const override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End of UObject interface

	// UActorComponent interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	// End of UActorComponent interface

	// UPrimitiveComponent interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual UBodySetup* GetBodySetup() override;
	// End of UPrimitiveComponent interface

	// Set color of the terrain
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void SetTerrainColor(FLinearColor NewColor);

protected:
	UE_API void SpawnSegments(const TArray<FTerrainSegment>& TerrainSegments, bool bGenerateSegmentColliders);
	
	UE_API void GenerateFillRenderDataFromPolygon(const class UPaperSprite* NewSprite, FSpriteDrawCallRecord& FillDrawCall, const FVector2D& TextureSize, const TArray<FVector2D>& TriangulatedPolygonVertices);
	UE_API void GenerateCollisionDataFromPolygon(const TArray<FVector2D>& SplinePolyVertices2D, const TArray<float>& TerrainOffsets, const TArray<FVector2D>& TriangulatedPolygonVertices);
	UE_API void InsertConvexCollisionDataFromPolygon(const TArray<FVector2D>& ClosedPolyVertices2D);
	UE_API void ConstrainSplinePointsToXZ();

	UE_API void OnSplineEdited();

	/** Description of collision */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<class UBodySetup> CachedBodySetup;

	TArray<FPaperTerrainSpriteGeometry> GeneratedSpriteGeometry;
	
	UE_API FTransform GetTransformAtDistance(float InDistance) const;
};

#undef UE_API
