// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "PaperTerrainMaterial.generated.h"

#define UE_API PAPER2D_API

struct FPropertyChangedEvent;

// Rule for a single section of a terrain material
USTRUCT(BlueprintType)
struct FPaperTerrainMaterialRule
{
	GENERATED_USTRUCT_BODY()

	// The sprite to use at the 'left' (closest to spline start) edge of the terrain segment
	UPROPERTY(Category=Sprite, EditAnywhere, meta=(DisplayThumbnail="true"))
	TObjectPtr<class UPaperSprite> StartCap;

	// A set of sprites to randomly choose to fill up the interior space between the caps in a terrain segment
	UPROPERTY(Category=Sprite, EditAnywhere, meta=(DisplayThumbnail="true"))
	TArray<TObjectPtr<class UPaperSprite>> Body;

	// The sprite to use at the 'right' (closest to spline end) edge of the terrain segment
	UPROPERTY(Category=Sprite, EditAnywhere, meta=(DisplayThumbnail="true"))
	TObjectPtr<class UPaperSprite> EndCap;

	// Minimum slope angle (in degrees) to apply this rule
	UPROPERTY(Category=Sprite, EditAnywhere)
	float MinimumAngle;

	// Maximum slope angle (in degrees) to apply this rule
	UPROPERTY(Category=Sprite, EditAnywhere)
	float MaximumAngle;

	// If true, collision is enabled for sections matching this rule
	UPROPERTY(Category=Sprite, EditAnywhere)
	bool bEnableCollision;

	// How much should the collision be lofted from the spline (positive values go out from the spline, negative values go in to the spline)
	UPROPERTY(Category=Sprite, EditAnywhere)
	float CollisionOffset;

	// Specify a draw order for different materials in a spline. Smaller draw orders are drawn first, negative values are allowed.
	UPROPERTY(Category=Sprite, EditAnywhere)
	int32 DrawOrder;

#if WITH_EDITORONLY_DATA
	// Readable description for the rule (unused anywhere, just for clarity when editing the material)
	UPROPERTY(Category = Sprite, EditAnywhere)
	FText Description;
#endif

	FPaperTerrainMaterialRule()
		: StartCap(nullptr)
		, EndCap(nullptr)
		, MinimumAngle(0.0f)
		, MaximumAngle(360.0f)
		, bEnableCollision(true)
		, CollisionOffset(0.0f)
		, DrawOrder(0)
	{
	}
};

/**
 * Paper Terrain Material
 *
 * 'Material' setup for a 2D terrain spline (stores references to sprites that will be instanced along the spline path, not actually related to UMaterialInterface).
 */

UCLASS(MinimalAPI, BlueprintType)
class UPaperTerrainMaterial : public UDataAsset
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category=Sprite, EditAnywhere)
	TArray<FPaperTerrainMaterialRule> Rules;

	// The sprite to use for an interior region fill
	UPROPERTY(Category = Sprite, EditAnywhere, meta=(DisplayThumbnail="true"))
	TObjectPtr<class UPaperSprite> InteriorFill;

#if WITH_EDITOR
	// UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif
};

#undef UE_API
