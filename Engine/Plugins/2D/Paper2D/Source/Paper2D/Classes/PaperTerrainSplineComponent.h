// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SplineComponent.h"
#include "PaperTerrainSplineComponent.generated.h"

#define UE_API PAPER2D_API

//@TODO: Document
UCLASS(MinimalAPI, BlueprintType, Experimental)
class UPaperTerrainSplineComponent : public USplineComponent
{
	GENERATED_UCLASS_BODY()

public:
	// UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End of UObject interface

public:
	// Triggered when the spline is edited
	FSimpleDelegate OnSplineEdited;
};

#undef UE_API
