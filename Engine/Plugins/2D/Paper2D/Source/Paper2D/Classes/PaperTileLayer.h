// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "PaperTileLayer.generated.h"

#define UE_API PAPER2D_API

class UBodySetup;
class UPaperTileMap;
class UPaperTileSet;

// Flags used in the packed tile index
enum class EPaperTileFlags : uint32
{
	FlipHorizontal = (1U << 31),
	FlipVertical = (1U << 30),
	FlipDiagonal = (1U << 29),

	TileIndexMask = ~(7U << 29),
};

// This is the contents of a tile map cell
USTRUCT(BlueprintType, meta=(HasNativeBreak="/Script/Paper2D.TileMapBlueprintLibrary.BreakTile", HasNativeMake="/Script/Paper2D.TileMapBlueprintLibrary.MakeTile"))
struct FPaperTileInfo
{
	GENERATED_USTRUCT_BODY()

	// The tile set that this tile comes from
	UPROPERTY(EditAnywhere, Category=Sprite)
	TObjectPtr<UPaperTileSet> TileSet;

	// This is the index of the current tile within the tile set
	UPROPERTY(EditAnywhere, Category=Sprite)
	int32 PackedTileIndex;

	FPaperTileInfo()
		: TileSet(nullptr)
		, PackedTileIndex(INDEX_NONE)
	{
	}

	inline bool operator==(const FPaperTileInfo& Other) const
	{
		return (Other.TileSet == TileSet) && (Other.PackedTileIndex == PackedTileIndex);
	}

	inline bool operator!=(const FPaperTileInfo& Other) const
	{
		return !(*this == Other);
	}

	bool IsValid() const
	{
		return (TileSet != nullptr) && (PackedTileIndex != INDEX_NONE);
	}

 	inline int32 GetFlagsAsIndex() const
 	{
		return (int32)(((uint32)PackedTileIndex) >> 29);
 	}

	inline void SetFlagsAsIndex(uint8 NewFlags)
	{
		const uint32 Base = PackedTileIndex & (int32)EPaperTileFlags::TileIndexMask;
		const uint32 WithNewFlags = Base | ((NewFlags & 0x7) << 29);
		PackedTileIndex = (int32)WithNewFlags;
	}

	inline int32 GetTileIndex() const
	{
		return PackedTileIndex & (int32)EPaperTileFlags::TileIndexMask;
	}

	inline bool HasFlag(EPaperTileFlags Flag) const
	{
		return (PackedTileIndex & (int32)Flag) != 0;
	}

	inline void ToggleFlag(EPaperTileFlags Flag)
	{
		if (IsValid())
		{
			PackedTileIndex ^= (int32)Flag;
		}
	}

	inline void SetFlagValue(EPaperTileFlags Flag, bool bValue)
	{
		if (IsValid())
		{
			if (bValue)
			{
				PackedTileIndex |= (int32)Flag;
			}
			else
			{
				PackedTileIndex &= ~(int32)Flag;
			}
		}
	}

	friend uint32 GetTypeHash(const FPaperTileInfo& PaperTileInfo)
	{
		return HashCombine(GetTypeHash(PaperTileInfo.TileSet), PaperTileInfo.PackedTileIndex);
	}
};

// This class represents a single layer in a tile map.  All layers in the map must have the size dimensions.
UCLASS(MinimalAPI)
class UPaperTileLayer : public UObject
{
	GENERATED_UCLASS_BODY()

	// Name of the layer
	UPROPERTY(BlueprintReadOnly, Category=Sprite)
	FText LayerName;

private:
	// Width of the layer (in tiles)
	UPROPERTY(BlueprintReadOnly, Category=Sprite, meta=(AllowPrivateAccess="true"))
	int32 LayerWidth;

	// Height of the layer (in tiles)
	UPROPERTY(BlueprintReadOnly, Category=Sprite, meta=(AllowPrivateAccess="true"))
	int32 LayerHeight;

#if WITH_EDITORONLY_DATA
	// Is this layer currently hidden in the editor?
	UPROPERTY(EditAnywhere, Category=Sprite, meta=(AllowPrivateAccess="true"))
	uint32 bHiddenInEditor:1;
#endif

	// Should this layer be hidden in the game?
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sprite, meta=(AllowPrivateAccess="true"))
	uint32 bHiddenInGame:1;

	// Should this layer generate collision?
	// Note: This only has an effect if the owning tile map has collision enabled
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sprite, meta=(AllowPrivateAccess="true"))
	uint32 bLayerCollides:1;

	// Should this layer use a custom thickness for generated collision instead of the thickness setting in the tile map?
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sprite, meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	uint32 bOverrideCollisionThickness:1;

	// Should this layer use a custom offset for generated collision instead of the layer drawing offset?
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sprite, meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	uint32 bOverrideCollisionOffset:1;

	// The override thickness of the collision for this layer (when bOverrideCollisionThickness is set)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sprite, meta=(EditCondition=bOverrideCollisionThickness, AllowPrivateAccess="true"))
	float CollisionThicknessOverride;

	// The override offset of the collision for this layer (when bOverrideCollisionOffset is set)
	// Note: This is measured in Unreal Units (cm) from the center of the tile map component, not from the previous layer.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sprite, meta=(EditCondition=bOverrideCollisionOffset, AllowPrivateAccess="true"))
	float CollisionOffsetOverride;

	// The color of this layer (multiplied with the tile map color and passed to the material as a vertex color)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sprite, meta=(AllowPrivateAccess="true"))
	FLinearColor LayerColor;

	// The allocated width of the tile data (used to handle resizing without data loss)
	UPROPERTY()
	int32 AllocatedWidth;

	// The allocated height of the tile data (used to handle resizing without data loss)
	UPROPERTY()
	int32 AllocatedHeight;

	// The allocated tile data
	UPROPERTY()
	TArray<FPaperTileInfo> AllocatedCells;

	UPROPERTY()
	TObjectPtr<UPaperTileSet> TileSet_DEPRECATED;

	UPROPERTY()
	TArray<int32> AllocatedGrid_DEPRECATED;

public:
	UE_API void ConvertToTileSetPerCell();

	// UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End of UObject interface

	// Returns the parent tile map
	UE_API UPaperTileMap* GetTileMap() const;

	// Returns the index of this layer in the parent tile map
	UE_API int32 GetLayerIndex() const;

	// Returns whether the specified coordinates are in bounds for the layer
	UE_API bool InBounds(int32 X, int32 Y) const;

	// Returns the tile information about the specified cell
	UE_API FPaperTileInfo GetCell(int32 X, int32 Y) const;

	// Sets the tile information about the specified cell
	UE_API void SetCell(int32 X, int32 Y, const FPaperTileInfo& NewValue);

	// Reallocates the map.  This is a destructive operation that does not copy data across!
	UE_API void DestructiveAllocateMap(int32 NewWidth, int32 NewHeight);
	
	// Reallocates the map.  This tries to preserve contents.
	UE_API void ResizeMap(int32 NewWidth, int32 NewHeight);

	// Adds collision to the specified body setup
	UE_API void AugmentBodySetup(UBodySetup* ShapeBodySetup, float RenderSeparation);

	// Gets the layer-specific color multiplier
	UE_API FLinearColor GetLayerColor() const;

	// Sets the layer-specific color multiplier (Note: does not invalidate any components using this layer!)
	UE_API void SetLayerColor(FLinearColor NewColor);

	// Checks to see if this layer uses the specified tile set
	// Note: This is a slow operation, it scans each tile!
	UE_API bool UsesTileSet(UPaperTileSet* TileSet) const;

#if WITH_EDITORONLY_DATA
	// Should this layer be drawn (in the editor)?
	bool ShouldRenderInEditor() const { return !bHiddenInEditor; }

	// Set whether this layer should be drawn (in the editor)
	void SetShouldRenderInEditor(bool bShouldRender) { bHiddenInEditor = !bShouldRender; }
#endif

	// Should this layer be drawn (in game)?
	bool ShouldRenderInGame() const { return !bHiddenInGame; }

	// Returns the width of the layer (in tiles)
	int32 GetLayerWidth() const { return LayerWidth; }

	// Returns the height of the layer (in tiles)
	int32 GetLayerHeight() const { return LayerHeight; }

	// Returns the transform for the given packed flag index (0..7)
	static UE_API FTransform GetTileTransform(int32 FlagIndex);

	// Returns the number of occupied cells in the layer
	UE_API int32 GetNumOccupiedCells() const;

	// Returns the raw pointer to the allocated cells, only for use when rendering the tile map and should never be held onto
	const FPaperTileInfo* PRIVATE_GetAllocatedCells() const
	{
		return AllocatedCells.GetData();
	}

	void SetLayerCollides(bool bShouldCollide) { bLayerCollides = bShouldCollide; }
	
	// Should this layer generate collision?
	// Note: This only has an effect if the owning tile map has collision enabled
	bool GetLayerCollides() const { return bLayerCollides; }

	void SetLayerCollisionThickness(bool bShouldOverride, float OverrideValue)
	{
		bOverrideCollisionThickness = bShouldOverride;
		CollisionThicknessOverride = OverrideValue;
	}

	void SetLayerCollisionOffset(bool bShouldOverride, float OverrideValue)
	{
		bOverrideCollisionOffset = bShouldOverride;
		CollisionOffsetOverride = OverrideValue;
	}

protected:
	UE_API void ReallocateAndCopyMap();

};

#undef UE_API
