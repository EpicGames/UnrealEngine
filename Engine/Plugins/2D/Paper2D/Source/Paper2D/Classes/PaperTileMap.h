// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperTileMap.generated.h"

#define UE_API PAPER2D_API

namespace ESpriteCollisionMode { enum Type : int; }
struct FPropertyChangedEvent;

class UBodySetup;
class UMaterialInterface;
class UPaperTileLayer;
class UPaperTileSet;

// The different kinds of projection modes supported
UENUM()
namespace ETileMapProjectionMode
{
	enum Type : int
	{
		/** Square tile layout */
		Orthogonal,

		// Isometric tile layout (shaped like a diamond) */
		IsometricDiamond,

		/** Isometric tile layout (roughly in a square with alternating rows staggered).  Warning: Not fully supported yet. */
		IsometricStaggered,

		/** Hexagonal tile layout (roughly in a square with alternating rows staggered).  Warning: Not fully supported yet. */
		HexagonalStaggered
	};
}

// A tile map is a 2D grid with a defined width and height (in tiles).  There can be multiple layers, each of which can specify which tile should appear in each cell of the map for that layer.
UCLASS(MinimalAPI, BlueprintType)
class UPaperTileMap : public UObject
{
	GENERATED_UCLASS_BODY()

	// Width of map (in tiles)
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly, meta=(UIMin=1, ClampMin=1, ClampMax=1024))
	int32 MapWidth;

	// Height of map (in tiles)
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly, meta=(UIMin=1, ClampMin=1, ClampMax=1024))
	int32 MapHeight;

	// Width of one tile (in pixels)
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly, meta=(UIMin=1, ClampMin=1))
	int32 TileWidth;

	// Height of one tile (in pixels)
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly, meta=(UIMin=1, ClampMin=1))
	int32 TileHeight;

	// The scaling factor between pixels and Unreal units (cm) (e.g., 0.64 would make a 64 pixel wide tile take up 100 cm)
	UPROPERTY(Category = Setup, EditAnywhere)
	float PixelsPerUnrealUnit;

	// The Z-separation incurred as you travel in X (not strictly applied, batched tiles will be put at the same Z level) 
	UPROPERTY(Category=Setup, EditAnywhere, AdvancedDisplay)
	float SeparationPerTileX;

	// The Z-separation incurred as you travel in Y (not strictly applied, batched tiles will be put at the same Z level) 
	UPROPERTY(Category=Setup, EditAnywhere, AdvancedDisplay)
	float SeparationPerTileY;
	
	// The Z-separation between each layer of the tile map
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly)
	float SeparationPerLayer;

	// Last tile set that was selected when editing the tile map
	UPROPERTY()
	TSoftObjectPtr<UPaperTileSet> SelectedTileSet;

	// The material to use on a tile map instance if not overridden
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UMaterialInterface> Material;

	// The list of layers
	UPROPERTY(Instanced, Category=Sprite, BlueprintReadOnly, EditAnywhere)
	TArray<TObjectPtr<UPaperTileLayer>> TileLayers;

protected:
	// The extrusion thickness of collision geometry when using a 3D collision domain
	UPROPERTY(Category=Collision, EditAnywhere, BlueprintReadOnly)
	float CollisionThickness;

	// Collision domain (no collision, 2D, or 3D)
	UPROPERTY(Category=Collision, EditAnywhere, BlueprintReadOnly)
	TEnumAsByte<ESpriteCollisionMode::Type> SpriteCollisionDomain;

public:
	// Tile map type
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly)
	TEnumAsByte<ETileMapProjectionMode::Type> ProjectionMode;

	// The vertical height of the sides of the hex cell for a tile.
	// Note: This value should already be included as part of the TileHeight, and is purely cosmetic; it only affects how the tile cursor preview is drawn.
	UPROPERTY(Category=Setup, EditAnywhere, meta=(UIMin=0, ClampMin=0))
	int32 HexSideLength;

	// Baked physics data.
	UPROPERTY()
	TObjectPtr<UBodySetup> BodySetup;

public:
#if WITH_EDITORONLY_DATA
	/** Importing data and options used for this tile map */
	UPROPERTY(Category=ImportSettings, VisibleAnywhere, Instanced)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/** The currently selected layer index */
	UPROPERTY()
	int32 SelectedLayerIndex;

	/** The background color displayed in the tile map editor */
	UPROPERTY(Category=Setup, EditAnywhere, meta=(HideAlphaChannel))
	FLinearColor BackgroundColor;

	/** The color of the tile grid */
	UPROPERTY(Category=Setup, EditAnywhere, meta=(HideAlphaChannel))
	FLinearColor TileGridColor;

	/** The color of the multi tile grid */
	UPROPERTY(Category=Setup, EditAnywhere, meta=(HideAlphaChannel))
	FLinearColor MultiTileGridColor;

	/** Number of tiles the multi tile grid spans horizontally. 0 removes vertical lines */
	UPROPERTY(Category=Setup, EditAnywhere, meta=(ClampMin=0))
	int32 MultiTileGridWidth;

	/** Number of tiles the multi tile grid spans vertically. 0 removes horizontal lines */
	UPROPERTY(Category=Setup, EditAnywhere, meta=(ClampMin=0))
	int32 MultiTileGridHeight;

	/** Number of tiles the multi tile grid is shifted to the right */
	UPROPERTY(Category=Setup, EditAnywhere)
	int32 MultiTileGridOffsetX;

	/** Number of tiles the multi tile grid is shifted downwards */
	UPROPERTY(Category=Setup, EditAnywhere)
	int32 MultiTileGridOffsetY;

	/** The color of the layer grid */
	UPROPERTY(Category=Setup, EditAnywhere, meta=(HideAlphaChannel))
	FLinearColor LayerGridColor;
#endif

	/** The naming index to start at when trying to create a new layer */
	UPROPERTY()
	int32 LayerNameIndex;

public:
	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API void ValidateSelectedLayerIndex();
#endif
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#endif
	// End of UObject interface

	// Returns the tile coordinates of the specified local space position
	UE_API void GetTileCoordinatesFromLocalSpacePosition(const FVector& Position, int32& OutTileX, int32& OutTileY) const;

	// Returns the top left corner of the specified tile in local space
	UE_API FVector GetTilePositionInLocalSpace(float TileX, float TileY, int32 LayerIndex = 0) const;

	// Returns the center of the specified tile in local space
	UE_API FVector GetTileCenterInLocalSpace(float TileX, float TileY, int32 LayerIndex = 0) const;

	// Returns the polygon for the specified tile (will be 4 or 6 vertices as a rectangle, diamond, or hexagon)
	UE_API void GetTilePolygon(int32 TileX, int32 TileY, int32 LayerIndex, TArray<FVector>& LocalSpacePoints) const;

	UE_API void GetTileToLocalParameters(FVector& OutCornerPosition, FVector& OutStepX, FVector& OutStepY, FVector& OutOffsetYFactor) const;
	UE_API void GetLocalToTileParameters(FVector& OutCornerPosition, FVector& OutStepX, FVector& OutStepY, FVector& OutOffsetYFactor) const;

	// Returns the extrusion thickness of collision geometry when using a 3D collision domain
	float GetCollisionThickness() const
	{
		return CollisionThickness;
	}

	// Returns the collision domain (no collision, 2D, or 3D)
	ESpriteCollisionMode::Type GetSpriteCollisionDomain() const
	{
		return SpriteCollisionDomain;
	}

	// Sets the collision thickness
	UE_API void SetCollisionThickness(float Thickness = 50.0f);

	// Sets the collision domain
	UE_API void SetCollisionDomain(ESpriteCollisionMode::Type Domain);

	UE_API FBoxSphereBounds GetRenderBounds() const;

	// Creates and adds a new layer and returns it
	UE_API UPaperTileLayer* AddNewLayer(int32 InsertionIndex = INDEX_NONE);

	// Handles adding an existing layer that does *not* belong to any existing tile map
	UE_API void AddExistingLayer(UPaperTileLayer* NewLayer, int32 InsertionIndex = INDEX_NONE);

	// Creates a reasonable new layer name
	static UE_API FText GenerateNewLayerName(UPaperTileMap* TileMap);

	// Returns true if the specified name is already in use as a layer name
	UE_API bool IsLayerNameInUse(const FText& LayerName) const;

	// Resize the tile map and all layers
	UE_API void ResizeMap(int32 NewWidth, int32 NewHeight, bool bForceResize = true);

	// Return the scaling factor between pixels and Unreal units (cm)
	float GetPixelsPerUnrealUnit() const { return PixelsPerUnrealUnit; }

	// Return the scaling factor between Unreal units (cm) and pixels
	float GetUnrealUnitsPerPixel() const { return 1.0f / PixelsPerUnrealUnit; }

	// Called when a fresh tile map has been created (by factory or otherwise)
	// Adds a default layer and pulls the PixelsPerUnrealUnit from the project settings
	UE_API void InitializeNewEmptyTileMap(UPaperTileSet* DefaultTileSetAsset = nullptr);

	// Creates a clone of this tile map in the specified outer
	UE_API UPaperTileMap* CloneTileMap(UObject* OuterForClone);

	// Checks to see if this tile map uses the specified tile set
	// Note: This is a slow operation, it scans each tile of each layer!
	UE_API bool UsesTileSet(UPaperTileSet* TileSet) const;

	// Rebuild collision and recreate the body setup
	UE_API void RebuildCollision();

protected:
	UE_API virtual void UpdateBodySetup();
};

#undef UE_API
