// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"
#include "PaperTileMapComponent.generated.h"

#define UE_API PAPER2D_API

class UPaperTileLayer;
struct FDynamicMeshVertex;
struct FPaperTileInfo;

class FPaperTileMapRenderSceneProxy;
class FPrimitiveSceneProxy;
class UPaperTileMap;
class UTexture;
struct FSpriteRenderSection;

/**
 * A component that handles rendering and collision for a single instance of a UPaperTileMap asset.
 *
 * This component is created when you drag a tile map asset from the content browser into a Blueprint, or
 * contained inside of the actor created when you drag one into the level.
 *
 * NOTE: This is an beta preview class.  While not considered production-ready, it is a step beyond
 * 'experimental' and is being provided as a preview of things to come:
 *  - We will try to provide forward-compatibility for content you create.
 *  - The classes may change significantly in the future.
 *  - The code is in an early state and may not meet the desired polish / quality bar.
 *  - There is probably no documentation or example content yet.
 *  - They will be promoted out of 'beta' when they are production ready.
 *
 * @see UPrimitiveComponent, UPaperTileMap
 */

UCLASS(MinimalAPI, hideCategories=Object, ClassGroup=Paper2D, EarlyAccessPreview, meta=(BlueprintSpawnableComponent))
class UPaperTileMapComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY()
	int32 MapWidth_DEPRECATED;

	UPROPERTY()
	int32 MapHeight_DEPRECATED;

	UPROPERTY()
	int32 TileWidth_DEPRECATED;

	UPROPERTY()
	int32 TileHeight_DEPRECATED;

	UPROPERTY()
	TObjectPtr<class UPaperTileSet> DefaultLayerTileSet_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<UPaperTileLayer>> TileLayers_DEPRECATED;

	// The color of the tile map (multiplied with the per-layer color and passed to the material as a vertex color)
	UPROPERTY(EditAnywhere, Category=Materials)
	FLinearColor TileMapColor;

	// The index of the single layer to use if enabled
	UPROPERTY(EditAnywhere, Category=Rendering, meta=(EditCondition=bUseSingleLayer))
	int32 UseSingleLayerIndex;

	// Should we draw a single layer?
	UPROPERTY(EditAnywhere, Category=Rendering, meta=(InlineEditConditionToggle))
	bool bUseSingleLayer;

#if WITH_EDITOR
	// The number of batches required to render this tile map
	int32 NumBatches;

	// The number of triangles rendered in this tile map
	int32 NumTriangles;
#endif

public:
	// The tile map used by this component
	UPROPERTY(Category=Setup, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<class UPaperTileMap> TileMap;

#if WITH_EDITORONLY_DATA
	// Should this component show a tile grid when the component is selected?
	UPROPERTY(Category=Rendering, EditAnywhere)
	bool bShowPerTileGridWhenSelected;

	// Should this component show an outline around each layer when the component is selected?
	UPROPERTY(Category=Rendering, EditAnywhere)
	bool bShowPerLayerGridWhenSelected;

	// Should this component show an outline around the first layer when the component is not selected?
	UPROPERTY(Category=Rendering, EditAnywhere)
	bool bShowOutlineWhenUnselected;

	// Should this component show a tile grid when the component is not selected?
	UPROPERTY(Category=Rendering, EditAnywhere)
	bool bShowPerTileGridWhenUnselected;

	// Should this component show an outline around each layer when the component is not selected?
	UPROPERTY(Category=Rendering, EditAnywhere)
	bool bShowPerLayerGridWhenUnselected;
#endif

protected:
	friend FPaperTileMapRenderSceneProxy;

	UE_API void RebuildRenderData(TArray<FSpriteRenderSection>& Sections, TArray<FDynamicMeshVertex>& Vertices);

public:
	// UObject interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End of UObject interface

	// UActorComponent interface
	UE_API virtual const UObject* AdditionalStatObject() const override;
	// End of UActorComponent interface

	// UPrimitiveComponent interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual class UBodySetup* GetBodySetup() override;
	UE_API virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel) override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual int32 GetNumMaterials() const override;
	// End of UPrimitiveComponent interface

	// Creates a new tile map internally, replacing the TileMap reference (or dropping the previous owned one)
	UE_API void CreateNewOwnedTileMap();

	/**
	 * Creates a new tile map of the specified size, replacing the TileMap reference (or dropping the previous owned one)
	 *
	 * @param MapWidth Width of the map (in tiles)
	 * @param MapHeight Height of the map (in tiles)
	 * @param TileWidth Width of one tile (in pixels)
	 * @param TileHeight Height of one tile (in pixels)
	 * @param bCreateLayer Should an empty layer be created?
	 */
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void CreateNewTileMap(int32 MapWidth = 4, int32 MapHeight = 4, int32 TileWidth = 32, int32 TileHeight = 32, float PixelsPerUnrealUnit = 1.0f, bool bCreateLayer = true);

	// Does this component own the tile map (is it instanced instead of being an asset reference)?
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API bool OwnsTileMap() const;

	/** Change the PaperTileMap used by this instance. */
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API virtual bool SetTileMap(UPaperTileMap* NewTileMap);

	// Returns the size of the tile map
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void GetMapSize(int32& MapWidth, int32& MapHeight, int32& NumLayers);

	// Returns the contents of a specified tile cell
	UFUNCTION(BlueprintPure, Category="Sprite", meta=(Layer="0"))
	UE_API FPaperTileInfo GetTile(int32 X, int32 Y, int32 Layer) const;

	// Modifies the contents of a specified tile cell (Note: This will only work on components that own their own tile map (OwnsTileMap returns true), you cannot modify standalone tile map assets)
	// Note: Does not update collision by default, call RebuildCollision after all edits have been done in a frame if necessary
	UFUNCTION(BlueprintCallable, Category="Sprite", meta=(Layer="0"))
	UE_API void SetTile(int32 X, int32 Y, int32 Layer, FPaperTileInfo NewValue);

	// Resizes the tile map (Note: This will only work on components that own their own tile map (OwnsTileMap returns true), you cannot modify standalone tile map assets) 
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void ResizeMap(int32 NewWidthInTiles, int32 NewHeightInTiles);

	// Creates and adds a new layer to the tile map
	// Note: This will only work on components that own their own tile map (OwnsTileMap returns true), you cannot modify standalone tile map assets
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API UPaperTileLayer* AddNewLayer();

	// Gets the tile map global color multiplier (multiplied with the per-layer color and passed to the material as a vertex color)
	UFUNCTION(BlueprintPure, Category="Sprite")
	UE_API FLinearColor GetTileMapColor() const;

	// Sets the tile map global color multiplier (multiplied with the per-layer color and passed to the material as a vertex color)
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void SetTileMapColor(FLinearColor NewColor);

	// Gets the per-layer color multiplier for a specific layer (multiplied with the tile map color and passed to the material as a vertex color)
	UFUNCTION(BlueprintPure, Category = "Sprite")
	UE_API FLinearColor GetLayerColor(int32 Layer = 0) const;

	// Sets the per-layer color multiplier for a specific layer (multiplied with the tile map color and passed to the material as a vertex color)
	// Note: This will only work on components that own their own tile map (OwnsTileMap returns true), you cannot modify standalone tile map assets
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void SetLayerColor(FLinearColor NewColor, int32 Layer = 0);

	// Returns the wireframe color to use for this component.
	UE_API FLinearColor GetWireframeColor() const;

	// Makes the tile map asset pointed to by this component editable.  Nothing happens if it was already instanced, but
	// if the tile map is an asset reference, it is cloned to make a unique instance.
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void MakeTileMapEditable();

	// Returns the position of the top left corner of the specified tile
	UFUNCTION(BlueprintPure, Category="Sprite")
	UE_API FVector GetTileCornerPosition(int32 TileX, int32 TileY, int32 LayerIndex = 0, bool bWorldSpace = false) const;

	// Returns the position of the center of the specified tile
	UFUNCTION(BlueprintPure, Category="Sprite")
	UE_API FVector GetTileCenterPosition(int32 TileX, int32 TileY, int32 LayerIndex = 0, bool bWorldSpace = false) const;

	// Returns the polygon for the specified tile (will be 4 or 6 vertices as a rectangle, diamond, or hexagon)
	UFUNCTION(BlueprintPure, Category="Sprite")
	UE_API void GetTilePolygon(int32 TileX, int32 TileY, TArray<FVector>& Points, int32 LayerIndex = 0, bool bWorldSpace = false) const;

	// Sets the default thickness for any layers that don't override the collision thickness
	// Note: This will only work on components that own their own tile map (OwnsTileMap returns true), you cannot modify standalone tile map assets
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void SetDefaultCollisionThickness(float Thickness, bool bRebuildCollision = true);

	// Sets the collision thickness for a specific layer
	// Note: This will only work on components that own their own tile map (OwnsTileMap returns true), you cannot modify standalone tile map assets
	UFUNCTION(BlueprintCallable, Category="Sprite")
	UE_API void SetLayerCollision(int32 Layer = 0, bool bHasCollision = true, bool bOverrideThickness = true, float CustomThickness = 50.0f, bool bOverrideOffset = false, float CustomOffset = 0.0f, bool bRebuildCollision = true);

	// Rebuilds collision for the tile map
	UFUNCTION(BlueprintCallable, Category = "Sprite")
	UE_API void RebuildCollision();

#if WITH_EDITOR
	// Returns the rendering stats for this component
	UE_API void GetRenderingStats(int32& OutNumTriangles, int32& OutNumBatches) const;
#endif
};

#undef UE_API
