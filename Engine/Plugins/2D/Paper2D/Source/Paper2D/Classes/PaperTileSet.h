// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpriteEditorOnlyTypes.h"
#include "IntMargin.h"

#include "PaperTileSet.generated.h"

#define UE_API PAPER2D_API

struct FPaperTileInfo;

// Information about a single tile in a tile set
USTRUCT(BlueprintType, meta=(ShowOnlyInnerProperties))
struct FPaperTileMetadata
{
public:
	GENERATED_USTRUCT_BODY()

	// A tag that can be used for grouping and categorizing (consider using it as the index into a UDataTable asset).
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Sprite)
	FName UserDataName;

	// Collision data for the tile
	UPROPERTY(EditAnywhere, Category=Sprite)
	FSpriteGeometryCollection CollisionData;

	// Indexes into the Terrains array of the owning tile set, in counterclockwise order starting from top-left
	// 0xFF indicates no membership.
	UPROPERTY()
	uint8 TerrainMembership[4];

public:
	FPaperTileMetadata()
	{
		CollisionData.GeometryType = ESpritePolygonMode::FullyCustom;
		TerrainMembership[0] = 0xFF;
		TerrainMembership[1] = 0xFF;
		TerrainMembership[2] = 0xFF;
		TerrainMembership[3] = 0xFF;
	}

	// Does this tile have collision information?
	bool HasCollision() const
	{
		return CollisionData.Shapes.Num() > 0;
	}

	// Does this tile have user-specified metadata?
	bool HasMetaData() const
	{
		return !UserDataName.IsNone();
	}
};

// Information about a terrain type
USTRUCT()
struct FPaperTileSetTerrain
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Sprite)
	FString TerrainName;

	UPROPERTY()
	int32 CenterTileIndex = 0;
};

/**
 * A tile set is a collection of tiles pulled from a texture that can be used to fill out a tile map.
 *
 * @see UPaperTileMap, UPaperTileMapComponent
 */
UCLASS(MinimalAPI, BlueprintType)
class UPaperTileSet : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	// The width and height of a single tile (in pixels)
	UPROPERTY(Category=TileSet, BlueprintReadOnly, EditAnywhere, meta=(UIMin=1, ClampMin=1, AllowPrivateAccess="true"))
	FIntPoint TileSize;

	// The tile sheet texture associated with this tile set
	UPROPERTY(Category=TileSet, BlueprintReadOnly, EditAnywhere, meta=(DisplayName="Tile Sheet Texture", AllowPrivateAccess="true"))
	TObjectPtr<UTexture2D> TileSheet;

	// Additional source textures for other slots
	UPROPERTY(Category = TileSet, EditAnywhere, AssetRegistrySearchable, meta = (DisplayName = "Additional Textures"))
	TArray<TObjectPtr<UTexture>> AdditionalSourceTextures;

	// The amount of padding around the border of the tile sheet (in pixels)
	UPROPERTY(Category=TileSet, BlueprintReadOnly, EditAnywhere, meta=(UIMin=0, ClampMin=0, AllowPrivateAccess="true"))
	FIntMargin BorderMargin;

	// The amount of padding between tiles in the tile sheet (in pixels)
	UPROPERTY(Category=TileSet, BlueprintReadOnly, EditAnywhere, meta=(UIMin=0, ClampMin=0, DisplayName="Per-Tile Spacing", AllowPrivateAccess="true"))
	FIntPoint PerTileSpacing;

	// The drawing offset for tiles from this set (in pixels)
	UPROPERTY(Category=TileSet, BlueprintReadOnly, EditAnywhere, meta=(AllowPrivateAccess="true"))
	FIntPoint DrawingOffset;

#if WITH_EDITORONLY_DATA
	/** The background color displayed in the tile set viewer */
	UPROPERTY(Category=TileSet, EditAnywhere, meta=(HideAlphaChannel), meta=(AllowPrivateAccess="true"))
	FLinearColor BackgroundColor;
#endif

	// Cached width of this tile set (in tiles)
	UPROPERTY()
	int32 WidthInTiles;

	// Cached height of this tile set (in tiles)
	UPROPERTY()
	int32 HeightInTiles;

	// Allocated width of the per-tile data
	UPROPERTY()
	int32 AllocatedWidth;

	// Allocated height of the per-tile data
	UPROPERTY()
	int32 AllocatedHeight;

	// Per-tile information
	UPROPERTY(EditAnywhere, EditFixedSize, Category=Sprite)
	TArray<FPaperTileMetadata> PerTileData;

	// Terrain information
	UPROPERTY()//@TODO: TileMapTerrains: (EditAnywhere, Category=Sprite)
	TArray<FPaperTileSetTerrain> Terrains;

protected:

	// Reallocates the PerTileData array to the specified size.
	// Note: This is a destructive operation!
	UE_API void DestructiveAllocateTileData(int32 NewWidth, int32 NewHeight);

	// Reallocates the per-tile data to match the current (WidthInTiles, HeightInTiles) size, copying over what it can
	UE_API void ReallocateAndCopyTileData();

public:

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End of UObject interface

public:
	// Returns the number of tiles in this tile set
	UE_API int32 GetTileCount() const;
	
	// Returns the number of tiles per row in this tile set
	UE_API int32 GetTileCountX() const;
	
	// Returns the number of tiles per column in this tile set
	UE_API int32 GetTileCountY() const;

	// Returns editable tile metadata for the specified tile index
	UE_API FPaperTileMetadata* GetMutableTileMetadata(int32 TileIndex);

	// Returns the tile metadata for the specified tile index
	UE_API const FPaperTileMetadata* GetTileMetadata(int32 TileIndex) const;

	// Returns the user data name for the specified tile, or NAME_None if there is no user-specified data
	UE_API FName GetTileUserData(int32 TileIndex) const;

	// Returns the texture-space coordinates of the top left corner of the specified tile index
	UE_API bool GetTileUV(int32 TileIndex, /*out*/ FVector2D& Out_TileUV) const;

	// Returns the texture-space coordinates of the top left corner of the tile at (TileXY.X, TileXY.Y)
	UE_API FIntPoint GetTileUVFromTileXY(const FIntPoint& TileXY) const;

	// Converts the texture-space coordinates into tile coordinates
	UE_API FIntPoint GetTileXYFromTextureUV(const FVector2D& TextureUV, bool bRoundUp) const;

	// Adds a new terrain to this tile set (returns false if the maximum number of terrains has already been reached)
	UE_API bool AddTerrainDescription(FPaperTileSetTerrain NewTerrain);

	// Returns the number of terrains this tile set has
	int32 GetNumTerrains() const
	{
		return Terrains.Num();
	}

	FPaperTileSetTerrain GetTerrain(int32 Index) const { return Terrains.IsValidIndex(Index) ? Terrains[Index] : FPaperTileSetTerrain(); }

	// Returns the terrain type this tile is a member of, or INDEX_NONE if it is not part of a terrain
	UE_API int32 GetTerrainMembership(const FPaperTileInfo& TileInfo) const;

#if WITH_EDITOR
	static FName GetPerTilePropertyName() { return GET_MEMBER_NAME_CHECKED(UPaperTileSet, PerTileData); }
#endif

	// Sets the size of a tile (in pixels)
	// Note: Does not trigger a rebuild of any tile map components using this tile set
	inline void SetTileSize(FIntPoint NewSize)
	{
		TileSize = FIntPoint(FMath::Max(NewSize.X, 1), FMath::Max(NewSize.Y, 1));
	}

	// Returns the size of a tile (in pixels)
	inline FIntPoint GetTileSize() const
	{
		return TileSize;
	}

	// Sets the tile sheet texture associated with this tile set
	// Note: Does not trigger a rebuild of any tile map components using this tile set
	inline void SetTileSheetTexture(UTexture2D* NewTileSheet)
	{
		TileSheet = NewTileSheet;
	}

	// Returns the tile sheet texture associated with this tile set
	inline UTexture2D* GetTileSheetTexture() const
	{
		return TileSheet;
	}

	inline TArray<UTexture*> GetAdditionalTextures() const
	{
		return AdditionalSourceTextures;
	}

	// Returns the imported size of the tile sheet texture (in pixels)
	inline FIntPoint GetTileSheetAuthoredSize() const
	{
		return TileSheet->GetImportedSize();
	}

	// Returns the amount of padding around the border of the tile sheet (in pixels)
	inline FIntMargin GetMargin() const
	{
		return BorderMargin;
	}

	// Sets the amount of padding around the border of the tile sheet (in pixels)
	// Note: Does not trigger a rebuild of any tile map components using this tile set
	inline void SetMargin(FIntMargin NewMargin)
	{
		BorderMargin = NewMargin;
	}

	// Returns the amount of padding between tiles in the tile sheet (in pixels)
	inline FIntPoint GetPerTileSpacing() const
	{
		return PerTileSpacing;
	}

	// Sets the amount of padding between tiles in the tile sheet (in pixels)
	// Note: Does not trigger a rebuild of any tile map components using this tile set
	inline void SetPerTileSpacing(FIntPoint NewSpacing)
	{
		PerTileSpacing = NewSpacing;
	}

	// Returns the drawing offset for tiles from this set (in pixels)
	inline FIntPoint GetDrawingOffset() const
	{
		return DrawingOffset;
	}

	// Sets the drawing offset for tiles from this set (in pixels)
	// Note: Does not trigger a rebuild of any tile map components using this tile set
	inline void SetDrawingOffset(FIntPoint NewDrawingOffset)
	{
		DrawingOffset = NewDrawingOffset;
	}

#if WITH_EDITORONLY_DATA
	// Returns the background color displayed in the tile set viewer
	inline FLinearColor GetBackgroundColor() const
	{
		return BackgroundColor;
	}

	// Returns the background color displayed in the tile set viewer
	inline void SetBackgroundColor(FLinearColor NewColor)
	{
		BackgroundColor = NewColor;
	}
#endif

private:
	UPROPERTY()
	int32 TileWidth_DEPRECATED;

	UPROPERTY()
	int32 TileHeight_DEPRECATED;

	UPROPERTY()
	int32 Margin_DEPRECATED;

	UPROPERTY()
	int32 Spacing_DEPRECATED;
};

#undef UE_API
