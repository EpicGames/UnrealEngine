// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperSprite.h"
#include "SpriteDrawCall.generated.h"

#define UE_API PAPER2D_API

//
USTRUCT()
struct FSpriteDrawCallRecord
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector Destination;

	UPROPERTY()
	TObjectPtr<UTexture> BaseTexture;

	FAdditionalSpriteTextureArray AdditionalTextures;

	UPROPERTY()
	FColor Color;

	//@TODO: The rest of this struct doesn't need to be properties either, but has to be due to serialization for now
	// Render triangle list (stored as loose vertices)
	TArray< FVector4, TInlineAllocator<6> > RenderVerts;

	UE_API void BuildFromSprite(const class UPaperSprite* Sprite);

	FSpriteDrawCallRecord()
		: Destination(ForceInitToZero)
		, BaseTexture(nullptr)
		, Color(FColor::White)
	{
	}

	bool IsValid() const
	{
		return (RenderVerts.Num() > 0) && (BaseTexture != nullptr) && (BaseTexture->GetResource() != nullptr);
	}
};

#undef UE_API
