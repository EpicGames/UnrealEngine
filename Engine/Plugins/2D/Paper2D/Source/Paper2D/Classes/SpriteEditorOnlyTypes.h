// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"
#include "SpriteEditorOnlyTypes.generated.h"

#define UE_API PAPER2D_API

class UMaterialInterface;

// The kind of collision that a Paper2D asset or component might participate in
UENUM()
namespace ESpriteCollisionMode
{
	enum Type : int
	{
		/** Should this have no collison and not participate in physics? */
		None,

		Use2DPhysics UMETA(Hidden, DisplayName = "Use 2D Physics (Deprecated)"),

		/** Should this have 3D collision geometry and participate in the 3D physics world? */
		Use3DPhysics UMETA(DisplayName = "Use 3D Physics")
	};
}

// The type of a shape in a sprite geometry structure
UENUM()
enum class ESpriteShapeType : uint8
{
	/** Box/Rectangular prism (size defined by BoxSize) */
	Box,

	/** Circle/Sphere (major axis is defined by BoxSize.X, minor axis by BoxSize.Y) */
	Circle,

	/** Custom closed polygon */
	Polygon
};

// A single piece of geometry (e.g., a polygon which may be convex or concave, a box, or a circle)
USTRUCT()
struct FSpriteGeometryShape
{
	GENERATED_USTRUCT_BODY()

	FSpriteGeometryShape()
		: ShapeType(ESpriteShapeType::Box)
		, BoxSize(ForceInitToZero)
		, BoxPosition(ForceInitToZero)
		, Rotation(0.0f)
		, bNegativeWinding(false)

	{
	}

	// The type of this piece of geometry
	UPROPERTY(Category=Physics, VisibleAnywhere)
	ESpriteShapeType ShapeType;

	// Vertices for the polygon (valid for Box and Polygon, but empty for Circle)
	UPROPERTY(Category=Physics, EditAnywhere)
	TArray<FVector2D> Vertices;

	// Size of the box or major/minor dimensions of the circle
	// Note: Only valid when GeometryType is Box or Circle
	UPROPERTY(Category=Physics, VisibleAnywhere)
	FVector2D BoxSize;
	
	// Center of the box or circle, acts as the pivot point for polygons (but may not be at the center of them)
	UPROPERTY(Category=Physics, VisibleAnywhere)
	FVector2D BoxPosition;

	// Rotation of the shape (in degrees)
	UPROPERTY(Category=Physics, VisibleAnywhere)
	float Rotation;

	// For Polygon geometry, this tells us if the winding should be negative (CW) regardless of the order in Vertices
	UPROPERTY(Category=Physics, VisibleAnywhere)
	bool bNegativeWinding;

public:
	// Is this shape valid (e.g., at least 3 vertices for a polygon)?
	bool IsShapeValid() const
	{
		return (ShapeType != ESpriteShapeType::Polygon) || (Vertices.Num() > 2);
	}

	FVector2D ConvertTextureSpaceToShapeSpace(const FVector2D& TextureSpacePoint) const
	{
		return (TextureSpacePoint - BoxPosition).GetRotated(Rotation);
	}

	FVector2D ConvertShapeSpaceToTextureSpace(const FVector2D& ShapeSpacePoint) const
	{
		return ShapeSpacePoint.GetRotated(-Rotation) + BoxPosition;
	}

	// Returns the polygon centroid in texture space
	FVector2D GetPolygonCentroid() const
	{
		FBox2D Bounds(ForceInit);

		FVector2D LocalSpaceResult = FVector2D::ZeroVector;

		for (const FVector2D& Vertex : Vertices)
		{
			Bounds += Vertex;
		}

		if (Vertices.Num() > 0)
		{
			LocalSpaceResult = Bounds.GetCenter();
		}

		return ConvertShapeSpaceToTextureSpace(LocalSpaceResult);
	}

	void GetTextureSpaceVertices(TArray<FVector2D>& InOutVertices) const
	{
		InOutVertices.Reserve(InOutVertices.Num() + Vertices.Num());
		for (const FVector2D& Vertex : Vertices)
		{
			InOutVertices.Add(ConvertShapeSpaceToTextureSpace(Vertex));
		}
	}

	// Sets a new pivot and adjusts all vertices to be relative to it
	void SetNewPivot(const FVector2D& NewPosInTextureSpace)
	{
		const FVector2D DeltaTexturePos = BoxPosition - NewPosInTextureSpace;
		for (FVector2D& Vertex : Vertices)
		{
			const FVector2D NewVertexPosTS = ConvertShapeSpaceToTextureSpace(Vertex) + DeltaTexturePos;
			Vertex = ConvertTextureSpaceToShapeSpace(NewVertexPosTS);
		}

		BoxPosition = NewPosInTextureSpace;
	}
};


// Method of specifying polygons for a sprite's render or collision data
UENUM()
namespace ESpritePolygonMode
{
	enum Type : int
	{
		/** Use the bounding box of the source sprite (no optimization) */
		SourceBoundingBox,

		/** Tighten the bounding box around the sprite to exclude fully transparent areas (the default) */
		TightBoundingBox,

		/** Shrink-wrapped geometry */
		ShrinkWrapped,

		/** Fully custom geometry; edited by hand */
		FullyCustom,

		/** Diced (split up into smaller squares, including only non-empty ones in the final geometry).  This option is only supported for Render geometry and will be ignored for Collision geometry. */
		Diced
	};
}

USTRUCT()
struct FSpriteGeometryCollection
{
	GENERATED_USTRUCT_BODY()

	// List of shapes
	UPROPERTY(Category=PolygonData, EditAnywhere, AdvancedDisplay)
	TArray<FSpriteGeometryShape> Shapes;

	// The geometry type (automatic / manual)
	UPROPERTY(Category=PolygonData, EditAnywhere)
	TEnumAsByte<ESpritePolygonMode::Type> GeometryType;

	// Size of a single subdivision (in pixels) in X (for Diced mode)
	UPROPERTY(Category=PolygonData, EditAnywhere)
	int32 PixelsPerSubdivisionX;

	// Size of a single subdivision (in pixels) in Y (for Diced mode)
	UPROPERTY(Category=PolygonData, EditAnywhere)
	int32 PixelsPerSubdivisionY;

	// Experimental: Hint to the triangulation routine that extra vertices should be preserved
	UPROPERTY(Category=PolygonData, EditAnywhere, AdvancedDisplay)
	bool bAvoidVertexMerging;

	// Alpha threshold for a transparent pixel (range 0..1, anything equal or below this value will be considered unimportant)
	UPROPERTY(Category=PolygonData, EditAnywhere, AdvancedDisplay)
	float AlphaThreshold;

	// Amount to detail to consider when shrink-wrapping (range 0..1, 0 = low detail, 1 = high detail)
	UPROPERTY(Category = PolygonData, EditAnywhere, AdvancedDisplay)
	float DetailAmount;

	// This is the threshold below which multiple vertices will be merged together when doing shrink-wrapping.  Higher values result in fewer vertices.
	UPROPERTY(Category=PolygonData, EditAnywhere, AdvancedDisplay)
	float SimplifyEpsilon;

public:
	FSpriteGeometryCollection()
		: GeometryType(ESpritePolygonMode::TightBoundingBox)
		, PixelsPerSubdivisionX(32)
		, PixelsPerSubdivisionY(32)
		, bAvoidVertexMerging(false)
		, AlphaThreshold(0.0f)
		, DetailAmount(0.5f)
		, SimplifyEpsilon(2.0f)
	{
	}

	UE_API void AddRectangleShape(FVector2D Position, FVector2D Size);
	UE_API void AddCircleShape(FVector2D Position, FVector2D Size);

	// Empties this geometry collection, resetting the GeometryType to TightBoundingBox
	UE_API void Reset();

	// Conditions this geometry collection (turning Polygons back to Boxes if they meet the definition of a box, etc...)
	// Returns true if something was modified
	UE_API bool ConditionGeometry();

	// Takes all polygon shapes and generates a list of triangles from them.
	// Output will contain a multiple of 3 points, each set is one triangle.
	// Always ignores circles, but can include or ignore boxes based on bIncludeBoxes.
	UE_API void Triangulate(TArray<FVector2D>& Target, bool bIncludeBoxes) const;
};

// A helper class for converting one or more FSpriteGeometryCollection objects into a UBodySetup object
struct FSpriteGeometryCollisionBuilderBase
{
protected:
	class UBodySetup* MyBodySetup;
	float UnrealUnitsPerPixel;
	float CollisionThickness;
	float ZOffsetAmount;
	ESpriteCollisionMode::Type CollisionDomain;

public:
	FSpriteGeometryCollisionBuilderBase(class UBodySetup* InBodySetup);

	virtual void ProcessGeometry(const FSpriteGeometryCollection& InGeometry);
	virtual void Finalize();
protected:
	void AddBoxCollisionShapesToBodySetup(const FSpriteGeometryCollection& InGeometry);
	void AddPolygonCollisionShapesToBodySetup(const FSpriteGeometryCollection& InGeometry);
	void AddCircleCollisionShapesToBodySetup(const FSpriteGeometryCollection& InGeometry);

	virtual FVector2D ConvertTextureSpaceToPivotSpace(const FVector2D& Input) const;
	virtual FVector2D ConvertTextureSpaceToPivotSpaceNoTranslation(const FVector2D& Input) const;
};

USTRUCT()
struct FSpriteAssetInitParameters
{
	GENERATED_USTRUCT_BODY()

	FSpriteAssetInitParameters()
		: Texture(nullptr)
		, Offset(ForceInitToZero)
		, Dimension(ForceInitToZero)
		, bOverridePixelsPerUnrealUnit(false)
		, PixelsPerUnrealUnit(1.0f)
		, DefaultMaterialOverride(nullptr)
		, AlternateMaterialOverride(nullptr)
	{
	}

	// Set the texture and the offset/dimension to fully match the specified texture 
	void SetTextureAndFill(UTexture2D* InTexture)
	{
		Texture = InTexture;
		if (Texture != nullptr)
		{
			Dimension = Texture->GetImportedSize();
			Offset = FIntPoint::ZeroValue;
		}
		else
		{
			Dimension = FIntPoint::ZeroValue;
			Offset = FIntPoint::ZeroValue;
		}
	}

	void SetPixelsPerUnrealUnit(float NewPixelsPerUU)
	{
		bOverridePixelsPerUnrealUnit = true;
		PixelsPerUnrealUnit = NewPixelsPerUU;
	}
public:
	// The texture to use
	UTexture2D* Texture;

	// Additional textures to use
	TArray<UTexture*> AdditionalTextures;

	// The offset within the texture (in pixels)
	FIntPoint Offset;

	// The dimension of the subregion within the texture (in pixels)
	FIntPoint Dimension;

	// Should we apply the PixelsPerUnrealUnit, or leave the value as it was
	bool bOverridePixelsPerUnrealUnit;

	// The scaling factor between pixels and Unreal units (cm) to use (e.g., 0.64 would make a 64 pixel wide sprite take up 100 cm)
	float PixelsPerUnrealUnit;

	// The material to override the default value with, if set
	UMaterialInterface* DefaultMaterialOverride;

	// The material to override the alternate (opaque) value with, if set
	UMaterialInterface* AlternateMaterialOverride;
};

UENUM()
namespace ESpritePivotMode
{
	enum Type : int
	{
		Top_Left,
		Top_Center,
		Top_Right,
		Center_Left,
		Center_Center,
		Center_Right,
		Bottom_Left,
		Bottom_Center,
		Bottom_Right,
		Custom
	};
}

#undef UE_API
