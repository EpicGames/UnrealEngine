// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "TileMapBlueprintLibrary.generated.h"

struct FPaperTileInfo;

class UPaperTileSet;

/**
 * A collection of utility methods for working with tile map components
 *
 * @see UPaperTileMap, UPaperTileMapComponent
 */
UCLASS(meta=(ScriptName="TileMapLibrary"))
class UTileMapBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Returns the user data name for the specified tile, or NAME_None if there is no user-specified data
	UFUNCTION(BlueprintPure, Category=Sprite)
	static FName GetTileUserData(FPaperTileInfo Tile);

	// Returns the transform for a tile
	UFUNCTION(BlueprintPure, Category=Sprite)
	static FTransform GetTileTransform(FPaperTileInfo Tile);

	// Breaks out the information for a tile
	UFUNCTION(BlueprintPure, Category=Sprite, meta=(AdvancedDisplay=2))
	static void BreakTile(FPaperTileInfo Tile, int32& TileIndex, UPaperTileSet*& TileSet, bool& bFlipH, bool& bFlipV, bool& bFlipD);

	// Creates a tile from the specified information
	UFUNCTION(BlueprintPure, Category=Sprite, meta=(AdvancedDisplay=2))
	static FPaperTileInfo MakeTile(int32 TileIndex, UPaperTileSet* TileSet, bool bFlipH, bool bFlipV, bool bFlipD);
};
