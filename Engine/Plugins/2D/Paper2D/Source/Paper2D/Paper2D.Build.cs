// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Paper2D : ModuleRules
{
	public Paper2D(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"RHI",
        		        "SlateCore",
        		        "Slate",
                		"NavigationSystem"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Renderer",
			}
		);

		if (Target.bBuildEditor == true)
		{
			//@TODO: Needed for the triangulation code used for sprites (but only in editor mode)
			//@TOOD: Try to move the code dependent on the triangulation code to the editor-only module
			PrivateDependencyModuleNames.Add("EditorFramework");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
