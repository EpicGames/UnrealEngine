// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"
#include "AlphaBitmap.h"
#include "Math/IntRect.h"
#include "Misc/ScopedSlowTask.h"

//////////////////////////////////////////////////////////////////////////
// FBitmap

struct FBitmap : public FAlphaBitmap
{
	FBitmap(UTexture2D* SourceTexture, int32 AlphaThreshold, uint8 DefaultValue = 0)
		: FAlphaBitmap(SourceTexture, DefaultValue)
	{
		ThresholdImage(AlphaThreshold);
	}

	// Create an empty bitmap
	FBitmap(int Width, int Height, int DefaultValue = 0)
		: FAlphaBitmap(Width, Height, DefaultValue)
	{
	}

	// Performs a flood fill on the target bitmap, with the boundary defined
	// by the current bitmap.
	void FloodFill(FBitmap& MaskBitmap, int StartX, int StartY)
	{
		TArray<FIntPoint> QueuedPoints;
		QueuedPoints.Reserve(Width);

		QueuedPoints.Add(FIntPoint(StartX, StartY));
		while (QueuedPoints.Num() > 0)
		{
			FIntPoint Point = QueuedPoints.Pop(EAllowShrinking::No);
			if (MaskBitmap.GetPixel(Point.X, Point.Y) == 0 && GetPixel(Point.X, Point.Y) == 1)
			{
				MaskBitmap.SetPixel(Point.X, Point.Y, 1);
				if (Point.X > 0) 
				{
					QueuedPoints.Add(FIntPoint(Point.X - 1, Point.Y));
				}
				if (Point.X < Width - 1) 
				{
					QueuedPoints.Add(FIntPoint(Point.X + 1, Point.Y));
				}
				if (Point.Y > 0) 
				{
					QueuedPoints.Add(FIntPoint(Point.X, Point.Y - 1));
				}
				if (Point.Y < Height - 1) 
				{
					QueuedPoints.Add(FIntPoint(Point.X, Point.Y + 1));
				}
			}
		}
	}

	// Walk the border pixels to find any intersecting islands we haven't already filled in the mask bitmap
	bool HasOverlappingIsland(FBitmap& MaskBitmap, const FIntPoint& Origin, const FIntPoint& Dimension, FIntPoint& OutFill)
	{
		OutFill.X = 0;
		OutFill.Y = 0;
		const int X0 = Origin.X;
		const int Y0 = Origin.Y;
		const int X1 = Origin.X + Dimension.X - 1;
		const int Y1 = Origin.Y + Dimension.Y - 1;

		for (int X = X0; X <= X1; ++X)
		{
			if (MaskBitmap.GetPixel(X, Y0) == 0 && GetPixel(X, Y0) != 0)
			{
				OutFill.X = X;
				OutFill.Y = Y0;
				return true;
			}
			if (MaskBitmap.GetPixel(X, Y1) == 0 && GetPixel(X, Y1) != 0)
			{
				OutFill.X = X;
				OutFill.Y = Y1;
				return true;
			}
		}

		for (int Y = Y0; Y <= Y1; ++Y)
		{
			if (MaskBitmap.GetPixel(X0, Y) == 0 && GetPixel(X0, Y) != 0)
			{
				OutFill.X = X0;
				OutFill.Y = Y;
				return true;
			}
			if (MaskBitmap.GetPixel(X1, Y) == 0 && GetPixel(X1, Y) != 0)
			{
				OutFill.X = X1;
				OutFill.Y = Y;
				return true;
			}
		}

		return false;
	}

	// Finds the rect of the contour of the shape clicked on, extended by rectangles to support
	// separated but intersecting islands.
	bool HasConnectedRect(const int X, const int Y, bool Extend, FIntPoint& OutOrigin, FIntPoint& OutDimension)
	{
		if (GetPixel(X, Y) == 0) 
		{
			// selected an empty pixel
			OutOrigin.X = 0;
			OutOrigin.Y = 0;
			OutDimension.X = 0;
			OutDimension.Y = 0;
			return false;
		}

		// Tmp get rid of this
		// This whole thing can be much more efficiently using the 8 bpp
		FBitmap MaskBitmap(Width, Height, 0);
		if (Extend)
		{
			MaskBitmap.DrawRectOutline(OutOrigin.X, OutOrigin.Y, OutDimension.X, OutDimension.Y);
		}

		// MaxPasses shouldn't be nessecary, but worst case interlocked pixel
		// patterns can cause problems. Dilating the bitmap before processing will
		// reduce these problems, but may not be desirable in all cases.
		int NumPasses = 0;
		int MaxPasses = 40;
		FIntPoint Origin(0, 0);
		FIntPoint Dimension(0, 0);
		FIntPoint FillPoint(X, Y);
		do
		{
			// This is probably going to be a botleneck at larger texture sizes
			// A contour tracing algorithm will probably suffice here.
			FloodFill(MaskBitmap, FillPoint.X, FillPoint.Y);
			MaskBitmap.GetTightBounds(/*out*/Origin, /*out*/Dimension);
		} while (NumPasses++ < MaxPasses && HasOverlappingIsland(MaskBitmap, Origin, Dimension, /*out*/FillPoint));

		checkSlow(Dimension.X > 0 && Dimension.Y > 0);

		OutOrigin = Origin;
		OutDimension = Dimension;

		return true;
	}

	// Detects all valid rects in this bitmap
	void ExtractRects(TArray<FIntRect>& OutRects)
	{
		FScopedSlowTask SlowTask(Height + Height/4, NSLOCTEXT("Paper2D", "Paper2D_AnalyzingTextureForSprites", "Scanning Texture For Sprites"));
		SlowTask.MakeDialog(false);

		SlowTask.EnterProgressFrame(Height/4);
		FBitmap MaskBitmap(Width, Height);

		const int32 ProgressReportInterval = 16;
		int32 NextProgressReportLine = ProgressReportInterval;

		for (int Y = 0; Y < Height; ++Y)
		{
			if (Y == NextProgressReportLine)
			{
				NextProgressReportLine += ProgressReportInterval;
				SlowTask.EnterProgressFrame(ProgressReportInterval);
			}

			for (int X = 0; X < Width; ++X)
			{
				if (MaskBitmap.GetPixel(X, Y) == 0 && GetPixel(X, Y) != 0)
				{
					FIntPoint Origin;
					FIntPoint Dimension;

					// Found something we don't already know of in the mask
					if (HasConnectedRect(X, Y, false, /*out*/Origin, /*out*/Dimension))
					{
						FIntRect NewRect(Origin, Origin + Dimension);
						bool bHasOverlap = false;
						do
						{
							bHasOverlap = false;
							for (int OutRectIndex = 0; OutRectIndex < OutRects.Num(); ++OutRectIndex)
							{
								FIntRect& ExistingRect = OutRects[OutRectIndex];
								bool bRectsOverlapping = NewRect.Max.X > ExistingRect.Min.X
														&& NewRect.Min.X < ExistingRect.Max.X
														&& NewRect.Max.Y > ExistingRect.Min.Y
														&& NewRect.Min.Y < ExistingRect.Max.Y;
								if (bRectsOverlapping)
								{
									NewRect = FIntRect(FMath::Min(NewRect.Min.X, ExistingRect.Min.X), FMath::Min(NewRect.Min.Y, ExistingRect.Min.Y),
													   FMath::Max(NewRect.Max.X, ExistingRect.Max.X), FMath::Max(NewRect.Max.Y, ExistingRect.Max.Y));
									OutRects.RemoveAtSwap(OutRectIndex, EAllowShrinking::No);
									break;
								}
							}
						} while (bHasOverlap); //-V654

						// Mark in the mask, avoid checking for any more sprites within this rect
						MaskBitmap.FillRect(NewRect.Min.X, NewRect.Min.Y, NewRect.Width(), NewRect.Height());
						checkSlow(NewRect.Width() > 0 && NewRect.Height() > 0);

						OutRects.Add(NewRect);
					}
				}
			}
		}
	}
};
