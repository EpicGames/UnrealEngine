// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroupedSpriteSceneProxy.h"
#include "PaperGroupedSpriteComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "SceneInterface.h"

//////////////////////////////////////////////////////////////////////////
// FGroupedSpriteSceneProxy

FSpriteRenderSection& FGroupedSpriteSceneProxy::FindOrAddSection(FSpriteDrawCallRecord& InBatch, UMaterialInterface* InMaterial)
{
	// Check the existing sections, starting with the most recent
	for (int32 SectionIndex = BatchedSections.Num() - 1; SectionIndex >= 0; --SectionIndex)
	{
		FSpriteRenderSection& TestSection = BatchedSections[SectionIndex];

		if (TestSection.Material == InMaterial)
		{
			if (TestSection.BaseTexture == InBatch.BaseTexture)
			{
				if (TestSection.AdditionalTextures == InBatch.AdditionalTextures)
				{
					return TestSection;
				}
			}
		}
	}

	// Didn't find a matching section, create one
	FSpriteRenderSection& NewSection = *(new (BatchedSections) FSpriteRenderSection());
	NewSection.Material = InMaterial;
	NewSection.BaseTexture = InBatch.BaseTexture;
	NewSection.AdditionalTextures = InBatch.AdditionalTextures;
	NewSection.VertexOffset = Vertices.Num();

	return NewSection;
}

SIZE_T FGroupedSpriteSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FGroupedSpriteSceneProxy::FGroupedSpriteSceneProxy(UPaperGroupedSpriteComponent* InComponent)
	: FPaperRenderSceneProxy(InComponent)
	, MyComponent(InComponent)
{
	MaterialRelevance = InComponent->GetMaterialRelevance(GetScene().GetShaderPlatform());

	NumInstances = InComponent->PerInstanceSpriteData.Num();

	const bool bAllowCollisionRendering = AllowDebugViewmodes() && InComponent->IsCollisionEnabled();

	if (bAllowCollisionRendering)
	{
		BodySetupTransforms.Reserve(NumInstances);
		BodySetups.Reserve(NumInstances);
	}

	// Create all the sections first so we can generate indices correctly
	TArray<int32> SectionIndicies;
	SectionIndicies.AddZeroed(NumInstances);

	for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
	{
		const FSpriteInstanceData& InstanceData = InComponent->PerInstanceSpriteData[InstanceIndex];

		if (UPaperSprite* SourceSprite = InstanceData.SourceSprite)
		{
			UTexture2D* BaseTexture = SourceSprite->GetBakedTexture();
			FAdditionalSpriteTextureArray AdditionalTextures;
			SourceSprite->GetBakedAdditionalSourceTextures(/*out*/ AdditionalTextures);
			UMaterialInterface* SpriteMaterial = InComponent->GetMaterial(InstanceData.MaterialIndex);

			FSpriteRenderSection* FoundSection = nullptr;
			int32 FoundSectionIndex = INDEX_NONE;

			for (int32 SectionIndex = BatchedSections.Num() - 1; SectionIndex >= 0; --SectionIndex)
			{
				FSpriteRenderSection& TestSection = BatchedSections[SectionIndex];
				if (TestSection.Material == SpriteMaterial)
				{
					if (TestSection.BaseTexture == BaseTexture)
					{
						if (TestSection.AdditionalTextures == AdditionalTextures)
						{
							FoundSection = &TestSection;
							FoundSectionIndex = SectionIndex;
							break;
						}
					}
				}
			}

			if (FoundSectionIndex == INDEX_NONE)
			{
				FoundSectionIndex = BatchedSections.Num();

				// Didn't find a matching section, create one
				FoundSection = new (BatchedSections) FSpriteRenderSection();
				FoundSection->Material = SpriteMaterial;
				FoundSection->BaseTexture = BaseTexture;
				FoundSection->AdditionalTextures = AdditionalTextures;
			}

			SectionIndicies[InstanceIndex] = FoundSectionIndex;
			FoundSection->NumVertices += SourceSprite->BakedRenderData.Num();
		}
	}

	int32 RunningVertCount = 0;
	for (FSpriteRenderSection& Section : BatchedSections)
	{
		Section.VertexOffset = RunningVertCount;
		RunningVertCount += Section.NumVertices;
		Section.NumVertices = 0;
	}
	check(Vertices.Num() == 0);
	Vertices.AddUninitialized(RunningVertCount);

	int32 InstanceIndex = 0;
	for (const FSpriteInstanceData& InstanceData : InComponent->PerInstanceSpriteData)
	{
		UBodySetup* BodySetup = nullptr;
		if (UPaperSprite* SourceSprite = InstanceData.SourceSprite)
		{
			const int32 SectionIndex = SectionIndicies[InstanceIndex];
			FSpriteRenderSection& Section = BatchedSections[SectionIndex];
			FDynamicMeshVertex* VertexWritePtr = Vertices.GetData() + Section.VertexOffset + Section.NumVertices;
			Section.NumVertices += SourceSprite->BakedRenderData.Num();

			const FPackedNormal TangentX = InstanceData.Transform.GetUnitAxis(EAxis::X);
			FPackedNormal TangentZ = InstanceData.Transform.GetUnitAxis(EAxis::Y);
			TangentZ.Vector.W = (InstanceData.Transform.Determinant() < 0.0f) ? -127 : 127;

			const FColor VertColor(InstanceData.VertexColor);
			for (const FVector4& SourceVert : SourceSprite->BakedRenderData)
			{
				const FVector LocalPos((PaperAxisX * SourceVert.X) + (PaperAxisY * SourceVert.Y));
				const FVector3f ComponentSpacePos = (FVector4f)InstanceData.Transform.TransformPosition(LocalPos);
				const FVector2f UV(SourceVert.Z, SourceVert.W);	// LWC_TODO: Precision loss

				*VertexWritePtr++ = FDynamicMeshVertex(ComponentSpacePos, TangentX.ToFVector3f(), TangentZ.ToFVector3f(), UV, VertColor);
			}

			BodySetup = SourceSprite->BodySetup;
		}

		if (bAllowCollisionRendering)
		{
			BodySetupTransforms.Add(InstanceData.Transform);
			BodySetups.Add(BodySetup);
		}
		++InstanceIndex;
	}
}

void FGroupedSpriteSceneProxy::DebugDrawCollision(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector, bool bDrawSolid) const
{
	for (int32 InstanceIndex = 0; InstanceIndex < BodySetups.Num(); ++InstanceIndex)
	{
		if (UBodySetup* BodySetup = BodySetups[InstanceIndex].Get())
		{
			const FColor CollisionColor = FColor(157, 149, 223, 255);
			const FMatrix GeomTransform = BodySetupTransforms[InstanceIndex] * GetLocalToWorld();
			DebugDrawBodySetup(View, ViewIndex, Collector, BodySetup, GeomTransform, CollisionColor, bDrawSolid);
		}
	}
}
