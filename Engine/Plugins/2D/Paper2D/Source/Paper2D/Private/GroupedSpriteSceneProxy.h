// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperRenderSceneProxy.h"

struct FSpriteDrawCallRecord;

class FMeshElementCollector;
class UBodySetup;
class UPaperGroupedSpriteComponent;
struct FPerInstanceRenderData;

//////////////////////////////////////////////////////////////////////////
// FGroupedSpriteSceneProxy

class FGroupedSpriteSceneProxy final : public FPaperRenderSceneProxy
{
public:
	SIZE_T GetTypeHash() const override;

	FGroupedSpriteSceneProxy(UPaperGroupedSpriteComponent* InComponent);

protected:
	// FPaperRenderSceneProxy interface
	virtual void DebugDrawCollision(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector, bool bDrawSolid) const override;
	// End of FPaperRenderSceneProxy interface

private:
	const UPaperGroupedSpriteComponent* MyComponent;

	/** Per instance render data, could be shared with component */
	TSharedPtr<FPerInstanceRenderData, ESPMode::ThreadSafe> PerInstanceRenderData;

	/** Number of instances */
	int32 NumInstances;

	TArray<FMatrix> BodySetupTransforms;
	TArray<TWeakObjectPtr<UBodySetup>> BodySetups;

private:
	FSpriteRenderSection& FindOrAddSection(FSpriteDrawCallRecord& InBatch, UMaterialInterface* InMaterial);
};
