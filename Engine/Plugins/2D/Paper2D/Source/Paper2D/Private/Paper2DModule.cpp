// Copyright Epic Games, Inc. All Rights Reserved.

#include "Paper2DModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Paper2DPrivate.h"
#include "PaperRenderSceneProxy.h"

DEFINE_STAT(STAT_PaperRender_SetSpriteRT);

//////////////////////////////////////////////////////////////////////////
// FPaper2DModule

class FPaper2DModule : public IPaper2DModuleInterface
{
public:
	virtual void StartupModule() override
	{
		check(GConfig);

		if (!GConfig->GetVector(TEXT("Paper2D"), TEXT("PaperAxisX"), PaperAxisX, GEngineIni))
		{
			PaperAxisX = FVector(1.0f, 0.0f, 0.0f);
		}
		if (!GConfig->GetVector(TEXT("Paper2D"), TEXT("PaperAxisY"), PaperAxisY, GEngineIni))
		{
			PaperAxisY = FVector(0.0f, 0.0f, 1.0f);
		}

		PaperAxisZ = FVector::CrossProduct(PaperAxisX, PaperAxisY);

		FPaperSpriteTangents::SetTangentsFromPaperAxes();
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FPaper2DModule, Paper2D);
DEFINE_LOG_CATEGORY(LogPaper2D);

FVector PaperAxisX(1.0f, 0.0f, 0.0f);
FVector PaperAxisY(0.0f, 0.0f, 1.0f);
FVector PaperAxisZ(0.0f, 1.0f, 0.0f);
