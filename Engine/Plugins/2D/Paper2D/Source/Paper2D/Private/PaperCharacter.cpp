// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperCharacter.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "PaperFlipbookComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperCharacter)

FName APaperCharacter::SpriteComponentName(TEXT("Sprite0"));

//////////////////////////////////////////////////////////////////////////
// APaperCharacter

APaperCharacter::APaperCharacter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(ACharacter::MeshComponentName))
{
	// Try to create the sprite component
	Sprite = CreateOptionalDefaultSubobject<UPaperFlipbookComponent>(APaperCharacter::SpriteComponentName);
	if (Sprite)
	{
		Sprite->AlwaysLoadOnClient = true;
		Sprite->AlwaysLoadOnServer = true;
		Sprite->bOwnerNoSee = false;
		Sprite->bAffectDynamicIndirectLighting = true;
		Sprite->PrimaryComponentTick.TickGroup = TG_PrePhysics;
		Sprite->SetupAttachment(GetCapsuleComponent());
		static FName CollisionProfileName(TEXT("CharacterMesh"));
		Sprite->SetCollisionProfileName(CollisionProfileName);
		Sprite->SetGenerateOverlapEvents(false);
	}
}

void APaperCharacter::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (IsValidChecked(this))
	{
		if (Sprite)
		{
			// force animation tick after movement component updates
			if (Sprite->PrimaryComponentTick.bCanEverTick && GetCharacterMovement())
			{
				Sprite->PrimaryComponentTick.AddPrerequisite(GetCharacterMovement(), GetCharacterMovement()->PrimaryComponentTick);
			}
		}
	}
}

