// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FPaperCustomVersion::GUID(0x11310AED, 0x2E554D61, 0xAF679AA3, 0xC5A1082C);

// Register the custom version with core
FCustomVersionRegistration GRegisterPaperCustomVersion(FPaperCustomVersion::GUID, FPaperCustomVersion::LatestVersion, TEXT("Paper2DVer"));
