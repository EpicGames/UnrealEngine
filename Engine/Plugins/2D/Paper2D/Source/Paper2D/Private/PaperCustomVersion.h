// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for all packages containing Paper2D asset types
struct FPaperCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		// Fixed negative volume in UPaperSprite 3D aggregate geometry (caused collision issues)
		FixedNegativeVolume = 1,

		// Removed FBodyInstance2D and moved 2D physics support into Engine (requires regenerating 2D agg geom)
		RemovedBodyInstance2D = 2,

		// Moved tile map data out of the component into UPaperTileMap
		MovedTileMapDataToSeparateClass = 3,

		// Marked sprite UBodySetup to use simple as complex, allowing 3D raycasts to succeed
		MarkSpriteBodySetupToUseSimpleAsComplex = 4,

		// Added PixelsPerUnrealUnit to move away from a 1:1 relationship between pixels and uu
		AddPixelsPerUnrealUnit = 5,

		// Fixed typo in convex hull generation code for 3D custom collision
		FixTypoIn3DConvexHullCollisionGeneration = 6,

		// Fixed undo issues in several asset types due to missing RF_Transactional flag
		AddTransactionalToClasses = 7,

		// Converted UPaperSpriteComponent to be derived from UMeshComponent
		ConvertPaperSpriteComponentToBeMeshComponent = 8,

		// Added the option to snap pivots to a pixel grid
		AddPivotSnapToPixelGrid = 9,

		// Converted UPaperFlipbookComponent to be derived from UMeshComponent
		ConvertPaperFlipbookComponentToBeMeshComponent = 10,

		// Add default BodyInstance setup in the sprite asset
		AddDefaultCollisionProfileInSpriteAsset = 11,
		
		// Add source texture size
		AddSourceTextureSize = 12,

		// Fix incorrect collision geometry generation when sprites were rotated in the source texture
		FixIncorrectCollisionOnSourceRotatedSprites = 13,

		// Refactor sprite render/collision polygon storage to allow more flexible geometry shapes
		RefactorPolygonStorageToSupportShapes = 14,

		// Change Tile Set margin and padding to allow non-uniform values (also converts TileWidth/TileHeight into TileSize and makes members private)
		AllowNonUniformPaddingInTileSets = 15,

		// Fixed the tangent generation for sprites, tile maps, etc... being incorrect for the front face
		FixTangentGenerationForFrontFace = 16,

		// Converted Paper2D vertex colors to be treated as linear space instead of sRGB space, and the color picker now matches the result on screen
		// rather than sprites appearing far brighter than they should (note: they are still sent down as a FColor so they have limited precision)
		FixVertexColorSpace = 17,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FPaperCustomVersion() {}
};
