// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbook.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/MaterialInterface.h"
#include "PaperSprite.h"
#include "PaperCustomVersion.h"
#include "PaperFlipbookComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperFlipbook)

#if WITH_EDITORONLY_DATA
#include "ComponentReregisterContext.h"
#endif

//////////////////////////////////////////////////////////////////////////
// UPaperFlipbook

UPaperFlipbook::UPaperFlipbook(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FramesPerSecond = 15;

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> MaskedMaterialRef(TEXT("/Paper2D/MaskedUnlitSpriteMaterial"));
	DefaultMaterial = MaskedMaterialRef.Object;
}

float UPaperFlipbook::GetFramesPerSecond() const
{
	return FramesPerSecond;
}

int32 UPaperFlipbook::GetNumFrames() const
{
	int32 SumFrames = 0;
	for (int32 KeyFrameIndex = 0; KeyFrameIndex < KeyFrames.Num(); ++KeyFrameIndex)
	{
		SumFrames += KeyFrames[KeyFrameIndex].FrameRun;
	}

	return SumFrames;
}

float UPaperFlipbook::GetTotalDuration() const
{
	if (FramesPerSecond != 0)
	{
		return GetNumFrames() / FramesPerSecond;
	}

	return 0.0f;
}

int32 UPaperFlipbook::GetKeyFrameIndexAtTime(float Time, bool bClampToEnds) const
{
	if ((Time < 0.0f) && !bClampToEnds)
	{
		return INDEX_NONE;
	}

	if (FramesPerSecond > 0.0f)
	{
		float SumTime = 0.0f;

		for (int32 KeyFrameIndex = 0; KeyFrameIndex < KeyFrames.Num(); ++KeyFrameIndex)
		{
			SumTime += KeyFrames[KeyFrameIndex].FrameRun / FramesPerSecond;

			if (Time <= SumTime)
			{
				return KeyFrameIndex;
			}
		}

		// Return the last frame (note: relies on INDEX_NONE = -1 if there are no key frames)
		return KeyFrames.Num() - 1;
	}
	else
	{
		return (KeyFrames.Num() > 0) ? 0 : INDEX_NONE;
	}
}

UPaperSprite* UPaperFlipbook::GetSpriteAtTime(float Time, bool bClampToEnds) const
{
	const int32 KeyFrameIndex = GetKeyFrameIndexAtTime(Time, bClampToEnds);
	return KeyFrames.IsValidIndex(KeyFrameIndex) ? KeyFrames[KeyFrameIndex].Sprite : nullptr;
}

UPaperSprite* UPaperFlipbook::GetSpriteAtFrame(int32 FrameIndex) const
{
	return KeyFrames.IsValidIndex(FrameIndex) ? KeyFrames[FrameIndex].Sprite : nullptr;
}

void UPaperFlipbook::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);
}

void UPaperFlipbook::PostLoad()
{
	Super::PostLoad();

	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);
	if (PaperVer < FPaperCustomVersion::AddTransactionalToClasses)
	{
		SetFlags(RF_Transactional);
	}
}

#if WITH_EDITOR
void UPaperFlipbook::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (FramesPerSecond < 0.0f)
	{
		FramesPerSecond = 0.0f;
	}

	//@TODO: Determine when this is really needed, as it is seriously expensive!
	TComponentReregisterContext<UPaperFlipbookComponent> ReregisterAnimatedComponents;

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

FBoxSphereBounds UPaperFlipbook::GetRenderBounds() const
{
	FBoxSphereBounds MergedBoundingBox(ForceInit);
	bool bHasValidBounds = false;

	for (const FPaperFlipbookKeyFrame& KeyFrame : KeyFrames)
	{
		if (KeyFrame.Sprite != nullptr)
		{
			const FBoxSphereBounds FrameBounds = KeyFrame.Sprite->GetRenderBounds();

			if (bHasValidBounds)
			{
				MergedBoundingBox = Union(MergedBoundingBox, FrameBounds);
			}
			else
			{
				MergedBoundingBox = FrameBounds;
				bHasValidBounds = true;
			}
		}
	}

	return MergedBoundingBox;
}

bool UPaperFlipbook::FindSocket(FName SocketName, int32 KeyFrameIndex, FTransform& OutLocalTransform)
{
	if (KeyFrames.IsValidIndex(KeyFrameIndex) && (SocketName != NAME_None))
	{
		if (UPaperSprite* SpriteFrame = KeyFrames[KeyFrameIndex].Sprite)
		{
			if (FPaperSpriteSocket* SocketInfo = SpriteFrame->FindSocket(SocketName))
			{
				OutLocalTransform = SocketInfo->LocalTransform;
				OutLocalTransform.ScaleTranslation(SpriteFrame->GetUnrealUnitsPerPixel());

				return true;
			}
		}
	}

	return false;
}

bool UPaperFlipbook::HasAnySockets() const
{
	for (const FPaperFlipbookKeyFrame& KeyFrame : KeyFrames)
	{
		if (KeyFrame.Sprite != nullptr)
		{
			if (KeyFrame.Sprite->HasAnySockets())
			{
				return true;
			}
		}
	}

	return false;
}

bool UPaperFlipbook::DoesSocketExist(FName SocketName) const
{
	for (const FPaperFlipbookKeyFrame& KeyFrame : KeyFrames)
	{
		if (KeyFrame.Sprite != nullptr)
		{
			if (KeyFrame.Sprite->FindSocket(SocketName))
			{
				return true;
			}
		}
	}

	return false;
}

void UPaperFlipbook::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	TSet<FName> SocketNames;
	TArray<FComponentSocketDescription> FrameDescriptions;

	for (const FPaperFlipbookKeyFrame& KeyFrame : KeyFrames)
	{
		if (KeyFrame.Sprite != nullptr)
		{
			FrameDescriptions.Reset();
			KeyFrame.Sprite->QuerySupportedSockets(/*out*/ FrameDescriptions);

			for (const FComponentSocketDescription& FrameSocket : FrameDescriptions)
			{
				if (!SocketNames.Contains(FrameSocket.Name))
				{
					SocketNames.Add(FrameSocket.Name);
					OutSockets.Add(FrameSocket);
				}
			}
		}
	}
}

void UPaperFlipbook::InvalidateCachedData()
{
	// No cached data yet, but the functions that currently have to iterate over all frames can use cached data in the future
}

bool UPaperFlipbook::ContainsSprite(UPaperSprite* SpriteAsset) const
{
	for (const FPaperFlipbookKeyFrame& KeyFrame : KeyFrames)
	{
		if (KeyFrame.Sprite == SpriteAsset)
		{
			return true;
		}
	}

	return false;
}

