// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbookActor.h"
#include "PaperFlipbookComponent.h"
#include "PaperFlipbook.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperFlipbookActor)

//////////////////////////////////////////////////////////////////////////
// APaperFlipbookActor

APaperFlipbookActor::APaperFlipbookActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RenderComponent = CreateDefaultSubobject<UPaperFlipbookComponent>(TEXT("RenderComponent"));

	RootComponent = RenderComponent;
}

#if WITH_EDITOR
bool APaperFlipbookActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (UPaperFlipbook* FlipbookAsset = RenderComponent->GetFlipbook())
	{
		Objects.Add(FlipbookAsset);
	}
	return true;
}
#endif

