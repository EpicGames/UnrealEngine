// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbookComponent.h"
#include "RenderingThread.h"
#include "Engine/CollisionProfile.h"
#include "PaperFlipbookSceneProxy.h"
#include "PaperFlipbook.h"
#include "PaperCustomVersion.h"
#include "Net/UnrealNetwork.h"
#include "ContentStreaming.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "PhysicsEngine/BodySetup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperFlipbookComponent)

DECLARE_CYCLE_STAT(TEXT("Tick Flipbook"), STAT_TickFlipbook, STATGROUP_Paper2D);

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// UPaperFlipbookComponent

UPaperFlipbookComponent::UPaperFlipbookComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
	Material_DEPRECATED = nullptr;

	CastShadow = false;
	bUseAsOccluder = false;
	bCanEverAffectNavigation = false;

	SpriteColor = FLinearColor::White;

	Mobility = EComponentMobility::Movable;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	bTickInEditor = true;
	
	CachedFrameIndex = INDEX_NONE;
	AccumulatedTime = 0.0f;
	PlayRate = 1.0f;

	bLooping = true;
	bReversePlayback = false;
	bPlaying = true;
}

UPaperSprite* UPaperFlipbookComponent::GetSpriteAtCachedIndex() const
{
	UPaperSprite* SpriteToSend = nullptr;
	if ((SourceFlipbook != nullptr) && SourceFlipbook->IsValidKeyFrameIndex(CachedFrameIndex))
	{
		SpriteToSend = SourceFlipbook->GetKeyFrameChecked(CachedFrameIndex).Sprite;
	}
	return SpriteToSend;
}


void UPaperFlipbookComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);
}

void UPaperFlipbookComponent::PostLoad()
{
	Super::PostLoad();

	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);

	if (PaperVer < FPaperCustomVersion::ConvertPaperFlipbookComponentToBeMeshComponent)
	{
		if (Material_DEPRECATED != nullptr)
		{
			SetMaterial(0, Material_DEPRECATED);
		}
	}

	if (PaperVer < FPaperCustomVersion::FixVertexColorSpace)
	{
		const FColor SRGBColor = SpriteColor.ToFColor(/*bSRGB=*/ true);
		SpriteColor = SRGBColor.ReinterpretAsLinear();
	}
}

FPrimitiveSceneProxy* UPaperFlipbookComponent::CreateSceneProxy()
{
	FPaperFlipbookSceneProxy* NewProxy = new FPaperFlipbookSceneProxy(this);

	CalculateCurrentFrame();
	UPaperSprite* SpriteToSend = GetSpriteAtCachedIndex();
	const int32 SplitIndex = SpriteToSend ? SpriteToSend->AlternateMaterialSplitIndex : INDEX_NONE;

	FSpriteDrawCallRecord DrawCall;
	DrawCall.BuildFromSprite(SpriteToSend);
	DrawCall.Color = SpriteColor.ToFColor(/*bSRGB=*/ false);

	FPaperFlipbookSceneProxy* InSceneProxy = (FPaperFlipbookSceneProxy*)NewProxy;
	ENQUEUE_RENDER_COMMAND(FCreatePaperFlipbookProxy_SetSprite)(
		[InSceneProxy, DrawCall, SplitIndex](FRHICommandListImmediate& RHICmdList)
	{
		InSceneProxy->SetSprite_RenderThread(DrawCall, SplitIndex);
	});
	return NewProxy;
}

FBoxSphereBounds UPaperFlipbookComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (SourceFlipbook != nullptr)
	{
		// Graphics bounds.
		FBoxSphereBounds NewBounds = SourceFlipbook->GetRenderBounds().TransformBy(LocalToWorld);

		// Add bounds of collision geometry (if present).
		if (CachedBodySetup != nullptr)
		{
			const FBox AggGeomBox = CachedBodySetup->AggGeom.CalcAABB(LocalToWorld);
			if (AggGeomBox.IsValid)
			{
				NewBounds = Union(NewBounds, FBoxSphereBounds(AggGeomBox));
			}
		}

		// Apply bounds scale
		NewBounds.BoxExtent *= BoundsScale;
		NewBounds.SphereRadius *= BoundsScale;

		return NewBounds;
	}
	else
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}
}

void UPaperFlipbookComponent::GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel)
{
	// Get the texture referenced by each keyframe
	if (SourceFlipbook != nullptr)
	{
		for (int32 Index = 0; Index < SourceFlipbook->GetNumKeyFrames(); ++Index)
		{
			const FPaperFlipbookKeyFrame& KeyFrame = SourceFlipbook->GetKeyFrameChecked(Index);
			if (KeyFrame.Sprite != nullptr)
			{
				if (UTexture* BakedTexture = KeyFrame.Sprite->GetBakedTexture())
				{
					OutTextures.AddUnique(BakedTexture);
				}
			}
		}
	}

	// Get any textures referenced by our materials
	Super::GetUsedTextures(OutTextures, QualityLevel);
}

UMaterialInterface* UPaperFlipbookComponent::GetMaterial(int32 MaterialIndex) const
{
	if (OverrideMaterials.IsValidIndex(MaterialIndex) && (OverrideMaterials[MaterialIndex] != nullptr))
	{
		return OverrideMaterials[MaterialIndex];
	}
	else if (SourceFlipbook != nullptr)
	{
		return SourceFlipbook->GetDefaultMaterial();
	}

	return nullptr;
}

void UPaperFlipbookComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	return Super::GetUsedMaterials(OutMaterials, bGetDebugMaterials);
}

void UPaperFlipbookComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	//@TODO: PAPER2D: Need to support this for proper texture streaming
	return Super::GetStreamingRenderAssetInfo(LevelContext, OutStreamingRenderAssets);
}

int32 UPaperFlipbookComponent::GetNumMaterials() const
{
	return FMath::Max<int32>(OverrideMaterials.Num(), 1);
}

UBodySetup* UPaperFlipbookComponent::GetBodySetup()
{
	CachedBodySetup = nullptr;

	if ((SourceFlipbook != nullptr) && (SourceFlipbook->GetCollisionSource() != EFlipbookCollisionMode::NoCollision))
	{
		const int32 PotentialSpriteIndex = (SourceFlipbook->GetCollisionSource() == EFlipbookCollisionMode::FirstFrameCollision) ? 0 : CachedFrameIndex;
		UPaperSprite* PotentialSpriteSource = SourceFlipbook->GetSpriteAtFrame(PotentialSpriteIndex);

		if (PotentialSpriteSource != nullptr)
		{
			CachedBodySetup = PotentialSpriteSource->BodySetup;
		}
	}

	return CachedBodySetup;
}

void UPaperFlipbookComponent::CalculateCurrentFrame()
{
	const int32 LastCachedFrame = CachedFrameIndex;
	CachedFrameIndex = (SourceFlipbook != nullptr) ? SourceFlipbook->GetKeyFrameIndexAtTime(AccumulatedTime) : INDEX_NONE;

	if (CachedFrameIndex != LastCachedFrame)
	{
		// Update children transforms in case we have anything attached to an animated socket
		UpdateChildTransforms();

		if ((SourceFlipbook != nullptr) && (SourceFlipbook->GetCollisionSource() == EFlipbookCollisionMode::EachFrameCollision))
		{
			FlipbookChangedPhysicsState();
		}

		// Indicate we need to send new dynamic data.
		MarkRenderDynamicDataDirty();
	}
}

void UPaperFlipbookComponent::FlipbookChangedPhysicsState()
{
	// If the frame has changed and we're using animated collision, we need to recreate that state as well
	RecreatePhysicsState();

	// We just totally changed the physics setup so update overlaps too
	UpdateOverlaps();
}

void UPaperFlipbookComponent::TickFlipbook(float DeltaTime)
{
	bool bIsFinished = false;

	if (bPlaying)
	{
		const float TimelineLength = GetFlipbookLength();
		const float EffectiveDeltaTime = DeltaTime * (bReversePlayback ? (-PlayRate) : (PlayRate));

		float NewPosition = AccumulatedTime + EffectiveDeltaTime;

		if (EffectiveDeltaTime > 0.0f)
		{
			if (NewPosition > TimelineLength)
			{
				if (bLooping)
				{
					// If looping, play to end, jump to start, and set target to somewhere near the beginning.
					SetPlaybackPosition(TimelineLength, true);
					SetPlaybackPosition(0.0f, false);

					if (TimelineLength > 0.0f)
					{
						while (NewPosition > TimelineLength)
						{
							NewPosition -= TimelineLength;
						}
					}
					else
					{
						NewPosition = 0.0f;
					}
				}
				else
				{
					// If not looping, snap to end and stop playing.
					NewPosition = TimelineLength;
					Stop();
					bIsFinished = true;
				}
			}
		}
		else
		{
			if (NewPosition < 0.0f)
			{
				if (bLooping)
				{
					// If looping, play to start, jump to end, and set target to somewhere near the end.
					SetPlaybackPosition(0.0f, true);
					SetPlaybackPosition(TimelineLength, false);

					if (TimelineLength > 0.0f)
					{
						while (NewPosition < 0.0f)
						{
							NewPosition += TimelineLength;
						}
					}
					else
					{
						NewPosition = 0.0f;
					}
				}
				else
				{
					// If not looping, snap to start and stop playing.
					NewPosition = 0.0f;
					Stop();
					bIsFinished = true;
				}
			}
		}

		SetPlaybackPosition(NewPosition, true);
	}

	// Notify user that the flipbook finished playing
	if (bIsFinished)
	{
		OnFinishedPlaying.Broadcast();
	}
}

/// @cond DOXYGEN_WARNINGS

void UPaperFlipbookComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UPaperFlipbookComponent, SourceFlipbook);
}

/// @endcond

void UPaperFlipbookComponent::OnRep_SourceFlipbook(class UPaperFlipbook* OldFlipbook)
{
	if (OldFlipbook != SourceFlipbook)
	{
		// Force SetFlipbook to change the animation (by default it won't change)
		UPaperFlipbook* NewFlipbook = SourceFlipbook;
		SourceFlipbook = nullptr;

		SetFlipbook(NewFlipbook);
	}
}

void UPaperFlipbookComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	SCOPE_CYCLE_COUNTER(STAT_TickFlipbook);

	// Advance time
	TickFlipbook(DeltaTime);

	// Update the frame and push it to the renderer if necessary
	CalculateCurrentFrame();
}

void UPaperFlipbookComponent::SendRenderDynamicData_Concurrent()
{
	if (SceneProxy != nullptr)
	{
		UPaperSprite* SpriteToSend = GetSpriteAtCachedIndex();

		FSpriteDrawCallRecord DrawCall;
		DrawCall.BuildFromSprite(SpriteToSend);
		DrawCall.Color = SpriteColor.ToFColor(/*bSRGB=*/ false);
		const int32 SplitIndex = SpriteToSend ? SpriteToSend->AlternateMaterialSplitIndex : INDEX_NONE;

		FPaperFlipbookSceneProxy* InSceneProxy = (FPaperFlipbookSceneProxy*)SceneProxy;
		UBodySetup* InBodySetup = CachedBodySetup;
		ENQUEUE_RENDER_COMMAND(FSendPaperRenderComponentDynamicData)(
			[InSceneProxy, DrawCall, InBodySetup, SplitIndex](FRHICommandListImmediate& RHICmdList)
			{
				InSceneProxy->SetSprite_RenderThread(DrawCall, SplitIndex);
				InSceneProxy->SetBodySetup_RenderThread(InBodySetup);
			});
	}
}

bool UPaperFlipbookComponent::SetFlipbook(class UPaperFlipbook* NewFlipbook)
{
	if (NewFlipbook != SourceFlipbook)
	{
		// Don't allow changing the sprite if we are "static".
		AActor* ComponentOwner = GetOwner();
		if ((ComponentOwner == nullptr) || AreDynamicDataChangesAllowed())
		{
			SourceFlipbook = NewFlipbook;

			// We need to also reset the frame and time also
			AccumulatedTime = 0.0f;
			CalculateCurrentFrame();

			// Need to send this to render thread at some point
			MarkRenderStateDirty();

			// Update physics representation right away
			FlipbookChangedPhysicsState();

			// Update this component streaming data.
			IStreamingManager::Get().NotifyPrimitiveUpdated(this);

			// Since we have new mesh, we need to update bounds
			UpdateBounds();

			return true;
		}
	}

	return false;
}

UPaperFlipbook* UPaperFlipbookComponent::GetFlipbook()
{
	return SourceFlipbook;
}

void UPaperFlipbookComponent::SetSpriteColor(FLinearColor NewColor)
{
	// Can't set color on a static component
	if (AreDynamicDataChangesAllowed() && (SpriteColor != NewColor))
	{
		SpriteColor = NewColor;

		//@TODO: Should we send immediately?
		MarkRenderDynamicDataDirty();
	}
}

const UObject* UPaperFlipbookComponent::AdditionalStatObject() const
{
	return SourceFlipbook;
}

void UPaperFlipbookComponent::Play()
{
	Activate();
	bReversePlayback = false;
	bPlaying = true;
}

void UPaperFlipbookComponent::PlayFromStart()
{
	SetPlaybackPosition(0.0f, /*bFireEvents=*/ false);
	Play();
}

void UPaperFlipbookComponent::Reverse()
{
	Activate();
	bReversePlayback = true;
	bPlaying = true;
}

void UPaperFlipbookComponent::ReverseFromEnd()
{
	SetPlaybackPosition(GetFlipbookLength(), /*bFireEvents=*/ false);
	Reverse();
}

void UPaperFlipbookComponent::Stop()
{
	bPlaying = false;
}

bool UPaperFlipbookComponent::IsPlaying() const
{
	return bPlaying;
}

bool UPaperFlipbookComponent::IsReversing() const
{
	return bPlaying && bReversePlayback;
}

void UPaperFlipbookComponent::SetPlaybackPositionInFrames(int32 NewFramePosition, bool bFireEvents)
{
	const float Framerate = GetFlipbookFramerate();
	const float NewTime = (Framerate > 0.0f) ? (NewFramePosition / Framerate) : 0.0f;
	SetPlaybackPosition(NewTime, bFireEvents);
}

int32 UPaperFlipbookComponent::GetPlaybackPositionInFrames() const
{
	const float Framerate = GetFlipbookFramerate();
	const int32 NumFrames = GetFlipbookLengthInFrames();
	if (NumFrames > 0)
	{
		return FMath::Clamp<int32>(FMath::TruncToInt(AccumulatedTime * Framerate), 0, NumFrames - 1);
	}
	else
	{
		return 0;
	}
}

void UPaperFlipbookComponent::SetPlaybackPosition(float NewPosition, bool bFireEvents)
{
	float OldPosition = AccumulatedTime;
	AccumulatedTime = NewPosition;

	// If we should be firing events for this track...
	if (bFireEvents)
	{
		float MinTime;
		float MaxTime;
		if (!bReversePlayback)
		{
			// If playing sequence forwards.
			MinTime = OldPosition;
			MaxTime = AccumulatedTime;

			// Slight hack here.. if playing forwards and reaching the end of the sequence, force it over a little to ensure we fire events actually on the end of the sequence.
			if (MaxTime == GetFlipbookLength())
			{
				MaxTime += (float)KINDA_SMALL_NUMBER;
			}
		}
		else
		{
			// If playing sequence backwards.
			MinTime = AccumulatedTime;
			MaxTime = OldPosition;

			// Same small hack as above for backwards case.
			if (MinTime == 0.0f)
			{
				MinTime -= (float)KINDA_SMALL_NUMBER;
			}
		}

#if 0
		// See which events fall into traversed region.
		for (int32 i = 0; i < Events.Num(); i++)
		{
			float EventTime = Events[i].Time;

			// Need to be slightly careful here and make behavior for firing events symmetric when playing forwards of backwards.
			bool bFireThisEvent = false;
			if (!bReversePlayback)
			{
				if ((EventTime >= MinTime) && (EventTime < MaxTime))
				{
					bFireThisEvent = true;
				}
			}
			else
			{
				if ((EventTime > MinTime) && (EventTime <= MaxTime))
				{
					bFireThisEvent = true;
				}
			}

			if (bFireThisEvent)
			{
				Events[i].EventFunc.ExecuteIfBound();
			}
		}
#endif
	}

#if 0
	// Execute the delegate to say that all properties are updated
	TimelinePostUpdateFunc.ExecuteIfBound();
#endif

	if (OldPosition != AccumulatedTime)
	{
		CalculateCurrentFrame();
	}
}

float UPaperFlipbookComponent::GetPlaybackPosition() const
{
	return AccumulatedTime;
}

void UPaperFlipbookComponent::SetLooping(bool bNewLooping)
{
	bLooping = bNewLooping;
}

bool UPaperFlipbookComponent::IsLooping() const
{
	return bLooping;
}

void UPaperFlipbookComponent::SetPlayRate(float NewRate)
{
	PlayRate = NewRate;
}

float UPaperFlipbookComponent::GetPlayRate() const
{
	return PlayRate;
}

void UPaperFlipbookComponent::SetNewTime(float NewTime)
{
	// Ensure value is sensible
	//@TODO: PAPER2D: NewTime = FMath::Clamp<float>(NewTime, 0.0f, Length);

	SetPlaybackPosition(NewTime, /*bFireEvents=*/ false);
}

float UPaperFlipbookComponent::GetFlipbookLength() const
{
	return (SourceFlipbook != nullptr) ? SourceFlipbook->GetTotalDuration() : 0.0f;
}

int32 UPaperFlipbookComponent::GetFlipbookLengthInFrames() const
{
	return (SourceFlipbook != nullptr) ? SourceFlipbook->GetNumFrames() : 0;
}

float UPaperFlipbookComponent::GetFlipbookFramerate() const
{
	return (SourceFlipbook != nullptr) ? SourceFlipbook->GetFramesPerSecond() : 15.0f;
}

bool UPaperFlipbookComponent::HasAnySockets() const
{
	if (SourceFlipbook != nullptr)
	{
		return SourceFlipbook->HasAnySockets();
	}

	return false;
}

bool UPaperFlipbookComponent::DoesSocketExist(FName InSocketName) const
{
	if (SourceFlipbook != nullptr)
	{
		FTransform SocketLocalTransform;
		if (SourceFlipbook->DoesSocketExist(InSocketName))
		{
			return true;
		}
	}

	return false;
}

FTransform UPaperFlipbookComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	if (SourceFlipbook != nullptr)
	{
		FTransform SocketLocalTransform;
		if (SourceFlipbook->FindSocket(InSocketName, (CachedFrameIndex != INDEX_NONE) ? CachedFrameIndex : 0, /*out*/ SocketLocalTransform))
		{
			switch (TransformSpace)
			{
			case RTS_World:
				return SocketLocalTransform * GetComponentTransform();

			case RTS_Actor:
				if (const AActor* Actor = GetOwner())
				{
					const FTransform SocketTransform = SocketLocalTransform * GetComponentTransform();
					return SocketTransform.GetRelativeTransform(Actor->GetTransform());
				}
				break;

			case RTS_Component:
			case RTS_ParentBoneSpace:
				return SocketLocalTransform;

			default:
				check(false);
			}
		}
	}

	return Super::GetSocketTransform(InSocketName, TransformSpace);
}

void UPaperFlipbookComponent::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	if (SourceFlipbook != nullptr)
	{
		return SourceFlipbook->QuerySupportedSockets(OutSockets);
	}
}

#if WITH_EDITOR
void UPaperFlipbookComponent::CheckForErrors()
{
	Super::CheckForErrors();

	AActor* Owner = GetOwner();

	for (int32 MaterialIndex = 0; MaterialIndex < GetNumMaterials(); ++MaterialIndex)
	{
		if (UMaterialInterface* Material = GetMaterial(MaterialIndex))
		{
			if (!Material->IsTwoSided())
			{
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(Owner))
					->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_PaperFlipbookMaterialNotTwoSided", "The material applied to the flipbook component is not marked as two-sided, which may cause lighting artifacts.")))
					->AddToken(FUObjectToken::Create(Material))
					->AddToken(FMapErrorToken::Create(FName(TEXT("PaperFlipbookMaterialNotTwoSided"))));
			}
		}
	}
}
#endif

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

