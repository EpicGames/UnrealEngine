// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbookSceneProxy.h"
#include "PaperFlipbookComponent.h"

//////////////////////////////////////////////////////////////////////////
// FPaperFlipbookSceneProxy

FPaperFlipbookSceneProxy::FPaperFlipbookSceneProxy(const UPaperFlipbookComponent* InComponent)
	: FPaperRenderSceneProxy_SpriteBase(InComponent)
{
}

SIZE_T FPaperFlipbookSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}