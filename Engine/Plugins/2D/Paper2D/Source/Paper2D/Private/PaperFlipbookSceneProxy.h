// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperRenderSceneProxy.h"

//////////////////////////////////////////////////////////////////////////
// FPaperFlipbookSceneProxy

class FPaperFlipbookSceneProxy final : public FPaperRenderSceneProxy_SpriteBase
{
public:
	SIZE_T GetTypeHash() const override;

	FPaperFlipbookSceneProxy(const class UPaperFlipbookComponent* InComponent);
};
