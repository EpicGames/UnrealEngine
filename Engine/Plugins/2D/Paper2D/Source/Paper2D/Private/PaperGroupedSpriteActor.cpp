// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperGroupedSpriteActor.h"
#include "PaperGroupedSpriteComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperGroupedSpriteActor)

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// APaperGroupedSpriteActor

APaperGroupedSpriteActor::APaperGroupedSpriteActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RenderComponent = CreateDefaultSubobject<UPaperGroupedSpriteComponent>(TEXT("RenderComponent"));

	RootComponent = RenderComponent;
}

#if WITH_EDITOR
bool APaperGroupedSpriteActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	RenderComponent->GetReferencedSpriteAssets(Objects);

	return true;
}
#endif

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

