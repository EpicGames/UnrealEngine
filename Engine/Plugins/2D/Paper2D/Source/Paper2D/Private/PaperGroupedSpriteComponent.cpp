// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperGroupedSpriteComponent.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "Engine/CollisionProfile.h"

#include "GroupedSpriteSceneProxy.h"
#include "AI/NavigationSystemHelpers.h"

#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperGroupedSpriteComponent)

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// UPaperGroupedSpriteComponent

UPaperGroupedSpriteComponent::UPaperGroupedSpriteComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);

	Mobility = EComponentMobility::Movable;
	BodyInstance.bSimulatePhysics = false;
	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;
}

int32 UPaperGroupedSpriteComponent::AddInstance(const FTransform& Transform, UPaperSprite* Sprite, bool bWorldSpace, FLinearColor Color)
{
	return AddInstanceWithMaterial(Transform, Sprite, nullptr, bWorldSpace, Color);
}

int32 UPaperGroupedSpriteComponent::AddInstanceWithMaterial(const FTransform& Transform, UPaperSprite* Sprite, UMaterialInterface* MaterialOverride, bool bWorldSpace, FLinearColor Color)
{
	const int32 NewInstanceIndex = PerInstanceSpriteData.Num();

	const FTransform LocalTransform(bWorldSpace ? Transform.GetRelativeTransform(GetComponentTransform()) : Transform);

	FSpriteInstanceData& NewInstanceData = *new (PerInstanceSpriteData)FSpriteInstanceData();
	SetupNewInstanceData(NewInstanceData, NewInstanceIndex, LocalTransform, Sprite, MaterialOverride, Color.ToFColor(/*bSRGB=*/ false));

	MarkRenderStateDirty();

	FNavigationSystem::UpdateComponentData(*this);

	return NewInstanceIndex;
}

bool UPaperGroupedSpriteComponent::GetInstanceTransform(int32 InstanceIndex, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	if (!PerInstanceSpriteData.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	const FSpriteInstanceData& InstanceData = PerInstanceSpriteData[InstanceIndex];

	OutInstanceTransform = FTransform(InstanceData.Transform);
	if (bWorldSpace)
	{
		OutInstanceTransform = OutInstanceTransform * GetComponentTransform();
	}

	return true;
}

void UPaperGroupedSpriteComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	// We are handling the physics move below, so don't handle it at higher levels
	Super::OnUpdateTransform(UpdateTransformFlags | EUpdateTransformFlags::SkipPhysicsUpdate, Teleport);

	// Always send new transform to physics
	if (bPhysicsStateCreated && !(EUpdateTransformFlags::SkipPhysicsUpdate & UpdateTransformFlags))
	{
		const bool bTeleport = TeleportEnumToFlag(Teleport);

		const FTransform& ComponentTransform = GetComponentTransform();
		for (int32 i = 0; i < PerInstanceSpriteData.Num(); i++)
		{
			const FTransform InstanceTransform(PerInstanceSpriteData[i].Transform);
			UpdateInstanceTransform(i, InstanceTransform * ComponentTransform, /* bWorldSpace= */true, /* bMarkRenderStateDirty= */false, bTeleport);
		}
	}
}

bool UPaperGroupedSpriteComponent::UpdateInstanceTransform(int32 InstanceIndex, const FTransform& NewInstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	if (!PerInstanceSpriteData.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	// Request navigation update
	FNavigationSystem::UpdateComponentData(*this);

	FSpriteInstanceData& InstanceData = PerInstanceSpriteData[InstanceIndex];

	// Render data uses local transform of the instance
	FTransform LocalTransform = bWorldSpace ? NewInstanceTransform.GetRelativeTransform(GetComponentTransform()) : NewInstanceTransform;
	InstanceData.Transform = LocalTransform.ToMatrixWithScale();

	if (bPhysicsStateCreated)
	{
		// Physics uses world transform of the instance
		const FTransform WorldTransform = bWorldSpace ? NewInstanceTransform : (LocalTransform * GetComponentTransform());
		if (FBodyInstance* InstanceBodyInstance = InstanceBodies[InstanceIndex])
		{
			// Update transform.
			InstanceBodyInstance->SetBodyTransform(WorldTransform, TeleportFlagToEnum(bTeleport));
			InstanceBodyInstance->UpdateBodyScale(WorldTransform.GetScale3D());
		}
	}

	// Request navigation update
	FNavigationSystem::UpdateComponentData(*this);

	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

bool UPaperGroupedSpriteComponent::UpdateInstanceColor(int32 InstanceIndex, FLinearColor NewInstanceColor, bool bMarkRenderStateDirty)
{
	if (PerInstanceSpriteData.IsValidIndex(InstanceIndex))
	{
		FSpriteInstanceData& InstanceData = PerInstanceSpriteData[InstanceIndex];
		InstanceData.VertexColor = NewInstanceColor.ToFColor(/*bSRGB=*/ false);

		if (bMarkRenderStateDirty)
		{
			MarkRenderStateDirty();
		}
		return true;
	}
	else
	{
		return false;
	}
}	

bool UPaperGroupedSpriteComponent::RemoveInstance(int32 InstanceIndex)
{
	if (!PerInstanceSpriteData.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	// Request navigation update
	FNavigationSystem::UpdateComponentData(*this);

	// remove instance
	PerInstanceSpriteData.RemoveAt(InstanceIndex);

	// update the physics state
	if (bPhysicsStateCreated)
	{
		// TODO: it may be possible to instead just update the BodyInstanceIndex for all bodies after the removed instance. 
		ClearAllInstanceBodies();
		CreateAllInstanceBodies();
	}

	// Indicate we need to update render state to reflect changes
	MarkRenderStateDirty();

	return true;
}

void UPaperGroupedSpriteComponent::ClearInstances()
{
	// Clear all the per-instance data
	PerInstanceSpriteData.Empty();

	// Release any physics representations
	ClearAllInstanceBodies();

	// Indicate we need to update render state to reflect changes
	MarkRenderStateDirty();

	FNavigationSystem::UpdateComponentData(*this);
}

int32 UPaperGroupedSpriteComponent::GetInstanceCount() const
{
	return PerInstanceSpriteData.Num();
}

bool UPaperGroupedSpriteComponent::ShouldCreatePhysicsState() const
{
	return IsRegistered() && (bAlwaysCreatePhysicsState || IsCollisionEnabled());
}

void UPaperGroupedSpriteComponent::OnCreatePhysicsState()
{
	check(InstanceBodies.Num() == 0);

	// Create all the bodies.
	CreateAllInstanceBodies();

	USceneComponent::OnCreatePhysicsState();
}

void UPaperGroupedSpriteComponent::OnDestroyPhysicsState()
{
	USceneComponent::OnDestroyPhysicsState();

	// Release all physics representations
	ClearAllInstanceBodies();
}

const UObject* UPaperGroupedSpriteComponent::AdditionalStatObject() const
{
	for (const FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
	{
		if (InstanceData.SourceSprite != nullptr)
		{
			return InstanceData.SourceSprite;
		}
	}

	return nullptr;
}

#if WITH_EDITOR
void UPaperGroupedSpriteComponent::CheckForErrors()
{
	AActor* Owner = GetOwner();

	bool bAnyInstancesWithNoSprites = false;
	for (const FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
	{
		if (InstanceData.SourceSprite == nullptr)
		{
			bAnyInstancesWithNoSprites = true;
			break;
		}
	}

	if (bAnyInstancesWithNoSprites)
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(Owner))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_InstanceSpriteComponent_MissingSprite", "One or more instances have no sprite asset set!")));
	}

	Super::CheckForErrors();
}
#endif

FPrimitiveSceneProxy* UPaperGroupedSpriteComponent::CreateSceneProxy()
{
	if (PerInstanceSpriteData.Num() > 0)
	{
		return ::new FGroupedSpriteSceneProxy(this);
	}
	else
	{
		return nullptr;
	}
}

bool UPaperGroupedSpriteComponent::CanEditSimulatePhysics()
{
	// Simulating for instanced sprite components is never allowed
	return false;
}

FBoxSphereBounds UPaperGroupedSpriteComponent::CalcBounds(const FTransform& BoundTransform) const
{
	FBoxSphereBounds::Builder BoundsBuilder;

	if (PerInstanceSpriteData.Num() > 0)
	{
		const FMatrix BoundTransformMatrix = BoundTransform.ToMatrixWithScale();

		for (const FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
		{
			if (InstanceData.SourceSprite != nullptr)
			{
				const FBoxSphereBounds RenderBounds = InstanceData.SourceSprite->GetRenderBounds();
				const FBoxSphereBounds InstanceBounds = RenderBounds.TransformBy(InstanceData.Transform * BoundTransformMatrix);
				BoundsBuilder += InstanceBounds;
			}
		}
	}

	return BoundsBuilder.IsValid() ? BoundsBuilder : FBoxSphereBounds(BoundTransform.GetLocation(), FVector::ZeroVector, 0.f);
}

#if WITH_EDITOR
void UPaperGroupedSpriteComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Rebuild the material array
	RebuildMaterialList();

	// Rebuild all instances
	//@TODO: Should try and avoid this except when absolutely necessary
	RebuildInstances();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPaperGroupedSpriteComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if ((PropertyChangedEvent.Property != nullptr) && (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPaperGroupedSpriteComponent, PerInstanceSpriteData)))
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			const int32 AddedAtIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
			check(AddedAtIndex != INDEX_NONE);
			SetupNewInstanceData(PerInstanceSpriteData[AddedAtIndex], AddedAtIndex, FTransform::Identity, nullptr, nullptr, FColor::White); //@TODO: Need to pull the sprite from somewhere
		}

		MarkRenderStateDirty();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void UPaperGroupedSpriteComponent::PostEditUndo()
{
	Super::PostEditUndo();

	FNavigationSystem::UpdateComponentData(*this);
}
#endif

void UPaperGroupedSpriteComponent::CreateAllInstanceBodies()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UPaperGroupedSpriteComponent_CreateAllInstanceBodies);

	FPhysScene* PhysScene = GetWorld()->GetPhysicsScene();

	const int32 NumBodies = PerInstanceSpriteData.Num();
	check(InstanceBodies.Num() == 0);
	InstanceBodies.SetNumUninitialized(NumBodies);


	for (int32 InstanceIndex = 0; InstanceIndex < NumBodies; ++InstanceIndex)
	{
		const FSpriteInstanceData& InstanceData = PerInstanceSpriteData[InstanceIndex];
		FBodyInstance* InstanceBody = InitInstanceBody(InstanceIndex, InstanceData, PhysScene);
		InstanceBodies[InstanceIndex] = InstanceBody;
	}
}

void UPaperGroupedSpriteComponent::ClearAllInstanceBodies()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UPaperGroupedSpriteComponent_ClearAllInstanceBodies);

	for (FBodyInstance* OldBodyInstance : InstanceBodies)
	{
		if (OldBodyInstance != nullptr)
		{
			OldBodyInstance->TermBody();
			delete OldBodyInstance;
		}
	}

	InstanceBodies.Empty();
}

void UPaperGroupedSpriteComponent::SetupNewInstanceData(FSpriteInstanceData& InOutNewInstanceData, int32 InInstanceIndex, const FTransform& InInstanceTransform, UPaperSprite* InSprite, UMaterialInterface* MaterialOverride, const FColor& InColor)
{
	InOutNewInstanceData.Transform = InInstanceTransform.ToMatrixWithScale();
	InOutNewInstanceData.SourceSprite = InSprite;
	InOutNewInstanceData.VertexColor = InColor;
	InOutNewInstanceData.MaterialIndex = UpdateMaterialList(InSprite, MaterialOverride);

	if (bPhysicsStateCreated && (InSprite != nullptr) && (InSprite->BodySetup != nullptr))
	{
		FPhysScene* PhysScene = GetWorld()->GetPhysicsScene();
		FBodyInstance* NewBodyInstance = InitInstanceBody(InInstanceIndex, InOutNewInstanceData, PhysScene);

		InstanceBodies.Insert(NewBodyInstance, InInstanceIndex);
	}
}

FBodyInstance* UPaperGroupedSpriteComponent::InitInstanceBody(int32 InstanceIndex, const FSpriteInstanceData& InstanceData, FPhysScene* PhysScene)
{
	FBodyInstance* NewBodyInstance = nullptr;

	if (InstanceData.SourceSprite != nullptr)
	{
		if (UBodySetup* BodySetup = InstanceData.SourceSprite->BodySetup)
		{
			NewBodyInstance = new FBodyInstance();

			NewBodyInstance->CopyBodyInstancePropertiesFrom(&BodyInstance);
			NewBodyInstance->InstanceBodyIndex = InstanceIndex; // Set body index 
			NewBodyInstance->bAutoWeld = false;

			// make sure we never enable bSimulatePhysics for ISMComps
			NewBodyInstance->bSimulatePhysics = false;

			const FTransform InstanceTransform(FTransform(InstanceData.Transform) * GetComponentTransform());
			NewBodyInstance->InitBody(BodySetup, InstanceTransform, this, PhysScene);
		}
	}

	return NewBodyInstance;
}

void UPaperGroupedSpriteComponent::RebuildInstances()
{
	// update the physics state
	if (bPhysicsStateCreated)
	{
		ClearAllInstanceBodies();
		CreateAllInstanceBodies();
	}

	// Indicate we need to update render state to reflect changes
	MarkRenderStateDirty();
}

void UPaperGroupedSpriteComponent::RebuildMaterialList()
{
	TArray<UMaterialInterface*> OldOverrides(OverrideMaterials);
	OverrideMaterials.Empty();

	for (FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
	{
		const int32 OldMaterialIndex = InstanceData.MaterialIndex;
		UMaterialInterface* OldOverride = OldOverrides.IsValidIndex(OldMaterialIndex) ? OldOverrides[OldMaterialIndex] : nullptr;

		const int32 NewMaterialIndex = UpdateMaterialList(InstanceData.SourceSprite, OldOverride);
		InstanceData.MaterialIndex = NewMaterialIndex;
	}
}

int32 UPaperGroupedSpriteComponent::UpdateMaterialList(UPaperSprite* Sprite, UMaterialInterface* MaterialOverride)
{
	int32 Result = INDEX_NONE;

	if (Sprite != nullptr)
	{
		if (UMaterialInterface* SpriteMaterial = Sprite->GetMaterial(0))
		{
			Result = InstanceMaterials.AddUnique(SpriteMaterial);
		}

		if (MaterialOverride != nullptr)
		{
			SetMaterial(Result, MaterialOverride);
		}
	}

	return Result;
}

void UPaperGroupedSpriteComponent::GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel)
{
	// Get the textures referenced by any sprite instances
	for (const FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
	{
		if (InstanceData.SourceSprite != nullptr)
		{
			if (UTexture* BakedTexture = InstanceData.SourceSprite->GetBakedTexture())
			{
				OutTextures.AddUnique(BakedTexture);
			}

			FAdditionalSpriteTextureArray AdditionalTextureList;
			InstanceData.SourceSprite->GetBakedAdditionalSourceTextures(/*out*/ AdditionalTextureList);
			for (UTexture* AdditionalTexture : AdditionalTextureList)
			{
				if (AdditionalTexture != nullptr)
				{
					OutTextures.AddUnique(AdditionalTexture);
				}
			}
		}
	}

	// Get any textures referenced by our materials
	Super::GetUsedTextures(OutTextures, QualityLevel);
}

UMaterialInterface* UPaperGroupedSpriteComponent::GetMaterial(int32 MaterialIndex) const
{
	if (OverrideMaterials.IsValidIndex(MaterialIndex))
	{
		if (UMaterialInterface* OverrideMaterial = OverrideMaterials[MaterialIndex])
		{
			return OverrideMaterial;
		}
	}

	if (InstanceMaterials.IsValidIndex(MaterialIndex))
	{
		return InstanceMaterials[MaterialIndex];
	}

	return nullptr;
}

int32 UPaperGroupedSpriteComponent::GetNumMaterials() const
{
	return FMath::Max<int32>(OverrideMaterials.Num(), FMath::Max<int32>(InstanceMaterials.Num(), 1));
}

bool UPaperGroupedSpriteComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	for (FBodyInstance* InstanceBody : InstanceBodies)
	{
		if (InstanceBody != nullptr)
		{
			if (UBodySetup* BodySetup = InstanceBody->GetBodySetup())
			{
				GeomExport.ExportRigidBodySetup(*BodySetup, FTransform::Identity);
			}
		}

		// Hook per instance transform delegate
		GeomExport.SetNavDataPerInstanceTransformDelegate(FNavDataPerInstanceTransformDelegate::CreateUObject(this, &UPaperGroupedSpriteComponent::GetNavigationPerInstanceTransforms));
	}

	// we don't want "regular" collision export for this component
	return false;
}

void UPaperGroupedSpriteComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	// Hook per instance transform delegate
	Data.NavDataPerInstanceTransformDelegate = FNavDataPerInstanceTransformDelegate::CreateUObject(this, &UPaperGroupedSpriteComponent::GetNavigationPerInstanceTransforms);
}

void UPaperGroupedSpriteComponent::GetNavigationPerInstanceTransforms(const FBox& AreaBox, TArray<FTransform>& OutInstanceTransforms) const
{
	const FTransform& ComponentTransform = GetComponentTransform();
	for (const FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
	{
		//TODO: Is it worth doing per instance bounds check here ?
		const FTransform InstanceToComponent(InstanceData.Transform);
		if (!InstanceToComponent.GetScale3D().IsZero())
		{
			OutInstanceTransforms.Add(InstanceToComponent * ComponentTransform);
		}
	}
}

bool UPaperGroupedSpriteComponent::ContainsSprite(UPaperSprite* SpriteAsset) const
{
	for (const FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
	{
		if (InstanceData.SourceSprite == SpriteAsset)
		{
			return true;
		}
	}

	return false;
}

void UPaperGroupedSpriteComponent::GetReferencedSpriteAssets(TArray<UObject*>& InOutObjects) const
{
	for (const FSpriteInstanceData& InstanceData : PerInstanceSpriteData)
	{
		if (InstanceData.SourceSprite != nullptr)
		{
			InOutObjects.AddUnique(InstanceData.SourceSprite);
		}
	}
}

void UPaperGroupedSpriteComponent::SortInstancesAlongAxis(FVector WorldSpaceSortAxis)
{
	struct FSortStruct
	{
		int32 OldIndex;
		float SortKey;

		FSortStruct(int32 InIndex, float InDepth)
			: OldIndex(InIndex)
			, SortKey(InDepth)
		{
		}
	};

	// Figure out the sort order
	const FTransform& ComponentTransform = GetComponentTransform();
	TArray<FSortStruct> SortArray;
	SortArray.Empty(PerInstanceSpriteData.Num());
	for (int32 Index = 0; Index < PerInstanceSpriteData.Num(); ++Index)
	{
		const FVector InstanceWorldPos = ComponentTransform.TransformPosition(PerInstanceSpriteData[Index].Transform.GetOrigin());
		const float SortKey = FVector::DotProduct(InstanceWorldPos, WorldSpaceSortAxis);
		SortArray.Emplace(Index, SortKey);
	}

	SortArray.Sort([](const FSortStruct& LHS, const FSortStruct& RHS)  { return LHS.SortKey > RHS.SortKey; });

	// Reorganize the array to match
	TArray<FSpriteInstanceData> OldData(PerInstanceSpriteData);
	PerInstanceSpriteData.Reset();

	for (const FSortStruct& SortedItem : SortArray)
	{
		PerInstanceSpriteData.Add(OldData[SortedItem.OldIndex]);
	}

	// Rebuild, as the rendering scene proxy and body setup orderings are both out of date
	RebuildInstances();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

