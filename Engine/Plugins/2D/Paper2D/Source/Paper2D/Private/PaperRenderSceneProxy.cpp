// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperRenderSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "SceneManagement.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "PhysicsEngine/BodySetup.h"
#include "Components/MeshComponent.h"
#include "Engine/Engine.h"
#include "SceneInterface.h"
#include "SceneView.h"

static TAutoConsoleVariable<int32> CVarDrawSpritesAsTwoSided(TEXT("r.Paper2D.DrawTwoSided"), 1, TEXT("Draw sprites as two sided."));
static TAutoConsoleVariable<int32> CVarDrawSpritesUsingPrebuiltVertexBuffers(TEXT("r.Paper2D.UsePrebuiltVertexBuffers"), 1, TEXT("Draw sprites using prebuilt vertex buffers."));

DECLARE_CYCLE_STAT(TEXT("Get New Batch Meshes"), STAT_PaperRender_GetNewBatchMeshes, STATGROUP_Paper2D);
DECLARE_CYCLE_STAT(TEXT("SpriteProxy GDME"), STAT_PaperRenderSceneProxy_GetDynamicMeshElements, STATGROUP_Paper2D);

FPackedNormal FPaperSpriteTangents::PackedNormalX(FVector(1.0f, 0.0f, 0.0f));
FPackedNormal FPaperSpriteTangents::PackedNormalZ(FVector(0.0f, -1.0f, 0.0f));

void FPaperSpriteTangents::SetTangentsFromPaperAxes()
{
	PackedNormalX = PaperAxisX;
	PackedNormalZ = -PaperAxisZ;
	// store determinant of basis in w component of normal vector
	PackedNormalZ.Vector.W = GetBasisDeterminantSignByte(PaperAxisX, PaperAxisY, PaperAxisZ);
}

//////////////////////////////////////////////////////////////////////////
// FSpriteTextureOverrideRenderProxy

/**
 * A material render proxy which overrides various named texture parameters.
 */
class FSpriteTextureOverrideRenderProxy : public FDynamicPrimitiveResource, public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* Parent;
	const UTexture* BaseTexture;
	FAdditionalSpriteTextureArray AdditionalTextures;
	UE_EXPAND_IF_WITH_EDITOR(const FPaperRenderSceneProxyTextureOverrideMap& TextureOverrideList);

	// Used to track if we need to rebuild our texture param setting proxy
	int32 ParentMaterialSerialNumber;

public:
	/** Initialization constructor. */
	FSpriteTextureOverrideRenderProxy(const FMaterialRenderProxy* InParent, const UTexture* InBaseTexture, FAdditionalSpriteTextureArray InAdditionalTextures UE_EXPAND_IF_WITH_EDITOR(, const FPaperRenderSceneProxyTextureOverrideMap& InTextureOverrideList))
		: FMaterialRenderProxy(InParent ? InParent->GetMaterialName() : TEXT("FSpriteTextureOverrideRenderProxy"))
		, Parent(InParent)
		, BaseTexture(InBaseTexture)
		, AdditionalTextures(InAdditionalTextures)
		UE_EXPAND_IF_WITH_EDITOR(, TextureOverrideList(InTextureOverrideList))
		, ParentMaterialSerialNumber(INDEX_NONE)
	{
	}

	bool CheckValidity(FMaterialRenderProxy* InCurrentParent)
	{
		if (InCurrentParent != Parent)
		{
			Parent = InCurrentParent;
			ParentMaterialSerialNumber = INDEX_NONE;
		}

		bool bValid = true;

		if (ParentMaterialSerialNumber != Parent->GetExpressionCacheSerialNumber())
		{
			bValid = false;
			ParentMaterialSerialNumber = Parent->GetExpressionCacheSerialNumber();
		}

		return bValid;
	}

	void Reinitialize(const FMaterialRenderProxy* InParent, const UTexture* InBaseTexture, FAdditionalSpriteTextureArray InAdditionalTextures)
	{
		if ((InParent != Parent) || (InBaseTexture != BaseTexture) || (InAdditionalTextures != AdditionalTextures))
		{
			Parent = InParent;
			BaseTexture = InBaseTexture;
			AdditionalTextures = InAdditionalTextures;
			ParentMaterialSerialNumber = INDEX_NONE;
		}
	}

	// FDynamicPrimitiveResource interface.
	virtual void InitPrimitiveResource(FRHICommandListBase& RHICmdList) override
	{
	}
	virtual void ReleasePrimitiveResource() override
	{
		delete this;
	}

	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetMaterialNoFallback(InFeatureLevel);
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetFallback(InFeatureLevel);
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
	{
		if (Type == EMaterialParameterType::Texture)
		{
			if (ParameterInfo.Name == TextureParameterName)
			{
				OutValue = ApplyEditorOverrides(BaseTexture);
				return true;
			}

			for (int32 AdditionalSlotIndex = 0; AdditionalSlotIndex < AdditionalTextures.Num(); ++AdditionalSlotIndex)
			{
				FName AdditionalSlotName = AdditionalTextureParameterRootName;
				AdditionalSlotName.SetNumber(AdditionalSlotIndex + 1);
				if (ParameterInfo.Name == AdditionalSlotName)
				{
					OutValue = ApplyEditorOverrides(AdditionalTextures[AdditionalSlotIndex]);
					return true;
				}
			}
		}

		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}

#if WITH_EDITOR
	inline const UTexture* ApplyEditorOverrides(const UTexture* InTexture) const
	{
		if (TextureOverrideList.Num() > 0)
		{
			if (const UTexture* const* OverridePtr = TextureOverrideList.Find(InTexture))
			{
				return *OverridePtr;
			}
		}

		return InTexture;
	}
#else
	FORCEINLINE const UTexture* ApplyEditorOverrides(const UTexture* InTexture) const
	{
		return InTexture;
	}
#endif

	static const FName TextureParameterName;
	static const FName AdditionalTextureParameterRootName;
};

const FName FSpriteTextureOverrideRenderProxy::TextureParameterName(TEXT("SpriteTexture"));
const FName FSpriteTextureOverrideRenderProxy::AdditionalTextureParameterRootName(TEXT("SpriteAdditionalTexture"));

//////////////////////////////////////////////////////////////////////////
// FPaperRenderSceneProxy

FPaperRenderSceneProxy::FPaperRenderSceneProxy(const UPrimitiveComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, VertexFactory(InComponent->GetWorld()->GetFeatureLevel())
	, Owner(InComponent->GetOwner())
	, MyBodySetup(const_cast<UPrimitiveComponent*>(InComponent)->GetBodySetup())
	, bCastShadow(InComponent->CastShadow)
	, CollisionResponse(InComponent->GetCollisionResponseToChannels())
{
	SetWireframeColor(FLinearColor::White);

	bDrawTwoSided = CVarDrawSpritesAsTwoSided.GetValueOnAnyThread() != 0;
	bSpritesUseVertexBufferPath = CVarDrawSpritesUsingPrebuiltVertexBuffers.GetValueOnAnyThread() != 0;
}

FPaperRenderSceneProxy::~FPaperRenderSceneProxy()
{
	for (FSpriteTextureOverrideRenderProxy* Proxy : MaterialTextureOverrideProxies)
	{
		if (Proxy != nullptr)
		{
			Proxy->ReleasePrimitiveResource();
		}
	}
	MaterialTextureOverrideProxies.Empty();

	VertexBuffer.ReleaseResource();
	VertexFactory.ReleaseResource();
}

SIZE_T FPaperRenderSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FPaperRenderSceneProxy::DebugDrawBodySetup(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector, UBodySetup* BodySetup, const FMatrix& GeomTransformMatrix, const FLinearColor& CollisionColor, bool bDrawSolid) const
{
	if (FMath::Abs(GeomTransformMatrix.Determinant()) < SMALL_NUMBER)
	{
		// Catch this here or otherwise GeomTransform below will assert
		// This spams so commented out
		//UE_LOGF(LogStaticMesh, Log, "Zero scaling not supported (%ls)", *StaticMesh->GetPathName());
	}
	else
	{
		FTransform GeomTransform(GeomTransformMatrix);

		if (bDrawSolid)
		{
			// Make a material for drawing solid collision stuff
			auto SolidMaterialInstance = new FColoredMaterialRenderProxy(
				GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
				GetWireframeColor()
				);

			Collector.RegisterOneFrameMaterialProxy(SolidMaterialInstance);

			BodySetup->AggGeom.GetAggGeom(GeomTransform, GetWireframeColor().ToFColor(true), SolidMaterialInstance, false, true, AlwaysHasVelocity(), ViewIndex, Collector);
		}
		else
		{
			// wireframe
			BodySetup->AggGeom.GetAggGeom(GeomTransform, GetSelectionColor(CollisionColor, IsSelected(), IsHovered()).ToFColor(true), nullptr, ( Owner == nullptr ), false, AlwaysHasVelocity(), ViewIndex, Collector);
		}
	}
}

void FPaperRenderSceneProxy::RecreateCachedRenderData(FRHICommandListBase& RHICmdList)
{
	int32 BatchIndex = 0;
	for (FSpriteTextureOverrideRenderProxy* Proxy : MaterialTextureOverrideProxies)
	{
		if ((Proxy != nullptr) && BatchedSections.IsValidIndex(BatchIndex))
		{
			const FSpriteRenderSection& Section = BatchedSections[BatchIndex];
			Proxy->Reinitialize(Section.Material->GetRenderProxy(), Section.BaseTexture, Section.AdditionalTextures);
		}
		++BatchIndex;
	}

	if (bSpritesUseVertexBufferPath && (Vertices.Num() > 0))
	{
		VertexBuffer.Vertices = Vertices;

		//We want the proxy to update the buffer, if it has been already initialized
		if (VertexBuffer.IsInitialized())
		{
			const bool bFactoryRequiresReInitialization = VertexBuffer.CommitRequiresBufferRecreation();
			VertexBuffer.CommitVertexData(RHICmdList);

			//When the buffer reallocates, the factory needs to bind the buffers and SRV again, we just init again.
			if (bFactoryRequiresReInitialization)
			{
				VertexFactory.Init(RHICmdList, &VertexBuffer);
			}
		}
		else
		{
			VertexBuffer.InitResource(RHICmdList);
			VertexFactory.Init(RHICmdList, &VertexBuffer);
		}
	}
}

void FPaperRenderSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	if (bSpritesUseVertexBufferPath && (Vertices.Num() > 0))
	{
		VertexBuffer.Vertices = Vertices;

		// Init the resources
		VertexBuffer.InitResource(RHICmdList);
		VertexFactory.Init(RHICmdList, &VertexBuffer);
	}
}

void FPaperRenderSceneProxy::DebugDrawCollision(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector, bool bDrawSolid) const
{
	if (MyBodySetup != nullptr)
	{
		const FColor CollisionColor = FColor(157, 149, 223, 255);
		DebugDrawBodySetup(View, ViewIndex, Collector, MyBodySetup, GetLocalToWorld(), CollisionColor, bDrawSolid);
	}
}

void FPaperRenderSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	SCOPE_CYCLE_COUNTER(STAT_PaperRenderSceneProxy_GetDynamicMeshElements);

	const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;

	bool bDrawSimpleCollision = false;
	bool bDrawComplexCollision = false;
	const bool bInCollisionView = IsCollisionView(EngineShowFlags, bDrawSimpleCollision, bDrawComplexCollision);
	
	// Sprites don't distinguish between simple and complex collision; when viewing visibility we should still render simple collision
	bDrawSimpleCollision |= bDrawComplexCollision;

	// Draw simple collision as wireframe if 'show collision', collision is enabled
	const bool bDrawWireframeCollision = EngineShowFlags.Collision && IsCollisionEnabled();

	const bool bDrawSprite = !bInCollisionView;

	if (bDrawSprite)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];

				GetDynamicMeshElementsForView(View, ViewIndex, Collector);
			}
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			if ((bDrawSimpleCollision || bDrawWireframeCollision) && AllowDebugViewmodes())
			{
				const FSceneView* View = Views[ViewIndex];
				const bool bDrawSolid = !bDrawWireframeCollision;
				DebugDrawCollision(View, ViewIndex, Collector, bDrawSolid);
			}

			// Draw bounds
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (EngineShowFlags.Paper2DSprites)
			{
				RenderBounds(Collector.GetPDI(ViewIndex), EngineShowFlags, GetBounds(), (Owner == nullptr) || IsSelected());
			}
#endif
		}
	}
}

void FPaperRenderSceneProxy::GetDynamicMeshElementsForView(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	if (bSpritesUseVertexBufferPath)
	{
		GetNewBatchMeshesPrebuilt(View, ViewIndex, Collector);
	}
	else
	{
		GetNewBatchMeshes(View, ViewIndex, Collector);
	}
}

void FPaperRenderSceneProxy::GetNewBatchMeshes(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	if (BatchedSections.Num() == 0)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_PaperRender_GetNewBatchMeshes);

	const uint8 DPG = GetDepthPriorityGroup(View);
	const bool bIsWireframeView = View->Family->EngineShowFlags.Wireframe;

	int32 SectionIndex = 0;
	if (Vertices.Num())
	{
		FDynamicMeshBuilder DynamicMeshBuilder(View->GetFeatureLevel());
		DynamicMeshBuilder.AddVertices(Vertices);
		DynamicMeshBuilder.ReserveTriangles(Vertices.Num() / 3);
		for (int32 i = 0; i < Vertices.Num(); i += 3)
		{
			DynamicMeshBuilder.AddTriangle(i, i + 1, i + 2);
		}

		for (const FSpriteRenderSection& Batch : BatchedSections)
		{
			if (Batch.IsValid())
			{
				FMaterialRenderProxy* ParentMaterialProxy = Batch.Material->GetRenderProxy();

				FDynamicMeshBuilderSettings Settings;
				Settings.bCanApplyViewModeOverrides = true;
				Settings.bUseWireframeSelectionColoring = IsSelected();

				// Implementing our own wireframe coloring as the automatic one (controlled by Mesh.bCanApplyViewModeOverrides) only supports per-FPrimitiveSceneProxy WireframeColor
				if (bIsWireframeView)
				{
					const FLinearColor EffectiveWireframeColor = IsOpaqueBlendMode(*Batch.Material) ? GetWireframeColor() : FLinearColor::Green;

					auto WireframeMaterialInstance = new FColoredMaterialRenderProxy(
						GEngine->WireframeMaterial->GetRenderProxy(),
						GetSelectionColor(EffectiveWireframeColor, IsSelected(), IsHovered(), false)
					);

					Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);

					ParentMaterialProxy = WireframeMaterialInstance;

					Settings.bWireframe = true;
					// We are applying our own wireframe override
					Settings.bCanApplyViewModeOverrides = false;
				}

				// Create a texture override material proxy or make sure our existing one is valid
				FSpriteTextureOverrideRenderProxy* SectionMaterialProxy = GetCachedMaterialProxyForSection(Collector, SectionIndex, ParentMaterialProxy);

				Settings.CastShadow = bCastShadow;
				Settings.bDisableBackfaceCulling = bDrawTwoSided;
				Settings.bIsFirstPerson = IsFirstPerson();

				FDynamicMeshDrawOffset DrawOffset;
				DrawOffset.FirstIndex = Batch.VertexOffset;
				DrawOffset.NumPrimitives = Batch.NumVertices / 3;
				DynamicMeshBuilder.GetMesh(GetLocalToWorld(), SectionMaterialProxy, DPG, Settings, &DrawOffset, ViewIndex, Collector);
			}

			++SectionIndex;
		}
	}
}

void FPaperRenderSceneProxy::GetNewBatchMeshesPrebuilt(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector) const
{
	const uint8 DPG = GetDepthPriorityGroup(View);
	const bool bIsWireframeView = View->Family->EngineShowFlags.Wireframe;

	//Go for each section, creating a batch and collecting it
	for (int32 SectionIndex = 0; SectionIndex < BatchedSections.Num(); SectionIndex++)
	{
		FMeshBatch& Batch = Collector.AllocateMesh();

		if (GetMeshElement(Collector, SectionIndex, DPG, IsSelected(), Batch))
		{
			// Implementing our own wireframe coloring as the automatic one (controlled by Mesh.bCanApplyViewModeOverrides) only supports per-FPrimitiveSceneProxy WireframeColor
			if (bIsWireframeView)
			{
				const FSpriteRenderSection& Section = BatchedSections[SectionIndex];
				const FLinearColor EffectiveWireframeColor = IsOpaqueBlendMode(*Section.Material) ? GetWireframeColor() : FLinearColor::Green;

				auto WireframeMaterialInstance = new FColoredMaterialRenderProxy(
					GEngine->WireframeMaterial->GetRenderProxy(),
					GetSelectionColor(EffectiveWireframeColor, IsSelected(), IsHovered(), false)
				);

				Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);

				// We are applying our own wireframe override
				Batch.bCanApplyViewModeOverrides = false;
				Batch.bWireframe = true;

				// Create a texture override material proxy and register it as a dynamic resource so that it won't be deleted until the rendering thread has finished with it
				Batch.MaterialRenderProxy = WireframeMaterialInstance;
			}

			Collector.AddMesh(ViewIndex, Batch);
		}
	}
}

bool FPaperRenderSceneProxy::GetMeshElement(FMeshElementCollector& Collector, int32 SectionIndex, uint8 DepthPriorityGroup, bool bIsSelected, FMeshBatch& OutMeshBatch) const
{
	check(bSpritesUseVertexBufferPath);
	check(SectionIndex < BatchedSections.Num());

	const FSpriteRenderSection& Section = BatchedSections[SectionIndex];
	if (Section.IsValid())
	{
		checkSlow(VertexBuffer.IsInitialized() && VertexFactory.IsInitialized());

		OutMeshBatch.bCanApplyViewModeOverrides = true;
		OutMeshBatch.bUseWireframeSelectionColoring = bIsSelected;

		OutMeshBatch.LODIndex = 0;
		OutMeshBatch.VertexFactory = &VertexFactory;
		OutMeshBatch.LCI = nullptr;
		OutMeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative() ? true : false;
		OutMeshBatch.CastShadow = bCastShadow;
		OutMeshBatch.DepthPriorityGroup = DepthPriorityGroup;
		OutMeshBatch.Type = PT_TriangleList;
		OutMeshBatch.bDisableBackfaceCulling = bDrawTwoSided;
		OutMeshBatch.MaterialRenderProxy = GetCachedMaterialProxyForSection(Collector, SectionIndex, Section.Material->GetRenderProxy());

		// Set up the FMeshBatchElement.
		FMeshBatchElement& BatchElement = OutMeshBatch.Elements[0];
		BatchElement.IndexBuffer = VertexBuffer.GetIndexPtr();
		BatchElement.FirstIndex = Section.VertexOffset;
		BatchElement.MinVertexIndex = Section.VertexOffset;
		BatchElement.MaxVertexIndex = Section.VertexOffset + Section.NumVertices;
		BatchElement.NumPrimitives = Section.NumVertices / 3;
		BatchElement.VertexFactoryUserData = VertexFactory.GetUniformBuffer();

		return true;
	}
	else
	{
		return false;
	}
}

FPrimitiveViewRelevance FPaperRenderSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	const FEngineShowFlags& EngineShowFlags = View->Family->EngineShowFlags;

	checkSlow(IsInParallelRenderingThread());

	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View) && EngineShowFlags.Paper2DSprites;
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;

	MaterialRelevance.SetPrimitiveViewRelevance(Result);

#undef SUPPORT_EXTRA_RENDERING
#define SUPPORT_EXTRA_RENDERING !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR
	

#if SUPPORT_EXTRA_RENDERING
	bool bDrawSimpleCollision = false;
	bool bDrawComplexCollision = false;
	const bool bInCollisionView = IsCollisionView(EngineShowFlags, bDrawSimpleCollision, bDrawComplexCollision);
#endif

	Result.bStaticRelevance = false;
	Result.bDynamicRelevance = true;

	if (!EngineShowFlags.Materials
#if SUPPORT_EXTRA_RENDERING
		|| bInCollisionView
#endif
		)
	{
		Result.bOpaque = true;
	}

	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

	return Result;
}

uint32 FPaperRenderSceneProxy::GetMemoryFootprint() const
{
	return sizeof(*this) + GetAllocatedSize();
}

bool FPaperRenderSceneProxy::CanBeOccluded() const
{
	return !MaterialRelevance.bDisableDepthTest;
}

bool FPaperRenderSceneProxy::IsUsingDistanceCullFade() const
{
	return MaterialRelevance.bUsesDistanceCullFade;
}

void FPaperRenderSceneProxy::SetBodySetup_RenderThread(UBodySetup* NewSetup)
{
	MyBodySetup = NewSetup;
}

bool FPaperRenderSceneProxy::IsCollisionView(const FEngineShowFlags& EngineShowFlags, bool& bDrawSimpleCollision, bool& bDrawComplexCollision) const
{
	bDrawSimpleCollision = false;
	bDrawComplexCollision = false;

	// If in a 'collision view' and collision is enabled
	const bool bInCollisionView = EngineShowFlags.CollisionVisibility || EngineShowFlags.CollisionPawn;
	if (bInCollisionView && IsCollisionEnabled())
	{
		// See if we have a response to the interested channel
		bool bHasResponse = EngineShowFlags.CollisionPawn && (CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore);
		bHasResponse |= EngineShowFlags.CollisionVisibility && (CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore);

		if (bHasResponse)
		{
			bDrawComplexCollision = EngineShowFlags.CollisionVisibility;
			bDrawSimpleCollision = EngineShowFlags.CollisionPawn;
		}
	}

	return bInCollisionView;
}

#if WITH_EDITOR
void FPaperRenderSceneProxy::SetTransientTextureOverride_RenderThread(const UTexture* InTextureToModifyOverrideFor, UTexture* InOverrideTexture)
{
	if (InOverrideTexture != nullptr)
	{
		TextureOverrideList.FindOrAdd(InTextureToModifyOverrideFor) = InOverrideTexture;
	}
	else
	{
		TextureOverrideList.Remove(InTextureToModifyOverrideFor);
	}
}
#endif

FSpriteTextureOverrideRenderProxy* FPaperRenderSceneProxy::GetCachedMaterialProxyForSection(FMeshElementCollector& Collector, int32 SectionIndex, FMaterialRenderProxy* ParentMaterialProxy) const
{
	UE::TScopeLock Lock(MaterialTextureOverrideProxiesMutex);

	if (MaterialTextureOverrideProxies.Num() < BatchedSections.Num())
	{
		MaterialTextureOverrideProxies.AddDefaulted(BatchedSections.Num() - MaterialTextureOverrideProxies.Num());
	}

	FSpriteTextureOverrideRenderProxy*& Result = MaterialTextureOverrideProxies[SectionIndex];
	if (Result == nullptr)
	{
		const FSpriteRenderSection& Section = BatchedSections[SectionIndex];
		Result = new FSpriteTextureOverrideRenderProxy(ParentMaterialProxy, Section.BaseTexture, Section.AdditionalTextures UE_EXPAND_IF_WITH_EDITOR(, TextureOverrideList));
	}

	if (!Result->CheckValidity(ParentMaterialProxy))
	{
		Collector.CacheUniformExpressions(Result, /** bRecreateUniformBuffer */ true);
	}

	return Result;
}

//////////////////////////////////////////////////////////////////////////
// FPaperRenderSceneProxy_SpriteBase

FPaperRenderSceneProxy_SpriteBase::FPaperRenderSceneProxy_SpriteBase(const UMeshComponent* InComponent)
	: FPaperRenderSceneProxy(InComponent)
{
	Material = InComponent->GetMaterial(0);
	if (Material == nullptr)
	{
		Material = UMaterial::GetDefaultMaterial(MD_Surface);
	}

	AlternateMaterial = InComponent->GetMaterial(1);
	if (AlternateMaterial == nullptr)
	{
		AlternateMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}

	MaterialRelevance = InComponent->GetMaterialRelevance(GetScene().GetShaderPlatform());
}

void FPaperRenderSceneProxy_SpriteBase::SetSprite_RenderThread(const FSpriteDrawCallRecord& NewDynamicData, int32 SplitIndex)
{
	SCOPE_CYCLE_COUNTER(STAT_PaperRender_SetSpriteRT);

	BatchedSections.Reset();
	Vertices.Reset();

	if (SplitIndex != INDEX_NONE)
	{
		BatchedSections.AddDefaulted();
		BatchedSections.AddDefaulted();

		FSpriteRenderSection& Section = BatchedSections[0];
		Section.Material = Material;
		Section.AddVerticesFromDrawCallRecord(NewDynamicData, 0, SplitIndex, Vertices);

		FSpriteRenderSection& AlternateSection = BatchedSections[1];
		AlternateSection.Material = AlternateMaterial;
		AlternateSection.AddVerticesFromDrawCallRecord(NewDynamicData, SplitIndex, NewDynamicData.RenderVerts.Num() - SplitIndex, Vertices);
	}
	else
	{
		FSpriteRenderSection& Section = BatchedSections[BatchedSections.AddDefaulted()];
		Section.Material = Material;
		Section.AddVerticesFromDrawCallRecord(NewDynamicData, 0, NewDynamicData.RenderVerts.Num(), Vertices);
	}

	RecreateCachedRenderData(FRHICommandListImmediate::Get());
}

