// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialRelevance.h"
#include "SpriteDrawCall.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveSceneProxy.h"
#include "Paper2DModule.h"
#include "PaperSpriteVertexBuffer.h"
#include "TextureResource.h"

#define UE_API PAPER2D_API

class FTexture;

class FMeshElementCollector;
class UBodySetup;
class UPrimitiveComponent;
class FSpriteTextureOverrideRenderProxy;
class UMeshComponent;
class FMaterialRenderProxy;

#if WITH_EDITOR
typedef TMap<const UTexture*, const UTexture*> FPaperRenderSceneProxyTextureOverrideMap;

#define UE_EXPAND_IF_WITH_EDITOR(...) __VA_ARGS__
#else
#define UE_EXPAND_IF_WITH_EDITOR(...)
#endif

/** A Paper2D sprite vertex. */
struct FPaperSpriteTangents
{
	static UE_API void SetTangentsFromPaperAxes();

	static UE_API FPackedNormal PackedNormalX;
	static UE_API FPackedNormal PackedNormalZ;
};

//////////////////////////////////////////////////////////////////////////
// FSpriteRenderSection

struct FSpriteRenderSection
{
	UMaterialInterface* Material;
	UTexture* BaseTexture;
	FAdditionalSpriteTextureArray AdditionalTextures;

	int32 VertexOffset;
	int32 NumVertices;

	FSpriteRenderSection()
		: Material(nullptr)
		, BaseTexture(nullptr)
		, VertexOffset(INDEX_NONE)
		, NumVertices(0)
	{
	}

	class FTexture* GetBaseTextureResource() const
	{
		return (BaseTexture != nullptr) ? BaseTexture->GetResource() : nullptr;
	}

	bool IsValid() const
	{
		return (Material != nullptr) && (NumVertices > 0) && (GetBaseTextureResource() != nullptr);
	}

	template <typename VerticesArrayAllocator>
	void AddVerticesFromDrawCallRecord(const FSpriteDrawCallRecord& Record, int32 StartIndexWithinRecord, int32 NumVertsToCopy, TArray<FDynamicMeshVertex, VerticesArrayAllocator>& Vertices)
	{
		if (NumVertices == 0)
		{
			VertexOffset = Vertices.Num();
			BaseTexture = Record.BaseTexture;
			AdditionalTextures = Record.AdditionalTextures;
		}
		else
		{
			checkSlow((VertexOffset + NumVertices) == Vertices.Num());
			checkSlow(BaseTexture == Record.BaseTexture);
			// Note: Not checking AdditionalTextures for now, since checking BaseTexture should catch most bugs
		}

		NumVertices += NumVertsToCopy;
		Vertices.Reserve(Vertices.Num() + NumVertsToCopy);

		const FColor VertColor(Record.Color);
		for (int32 VertexIndex = StartIndexWithinRecord; VertexIndex < StartIndexWithinRecord + NumVertsToCopy; ++VertexIndex)
		{
			const FVector4& SourceVert = Record.RenderVerts[VertexIndex];
			
			const FVector Pos((PaperAxisX * SourceVert.X) + (PaperAxisY * SourceVert.Y) + Record.Destination);
			const FVector2f UV(SourceVert.Z, SourceVert.W);	// LWC_TODO: Precision loss

			Vertices.Emplace((FVector3f)Pos, FPaperSpriteTangents::PackedNormalX.ToFVector3f(), FPaperSpriteTangents::PackedNormalZ.ToFVector3f(), UV, VertColor);
		}
	}

	template <typename VerticesArrayAllocator>
	inline void AddVertex(float X, float Y, float U, float V, const FVector& Origin, const FColor& Color, TArray<FDynamicMeshVertex, VerticesArrayAllocator>& Vertices)
	{
		const FVector Pos((PaperAxisX * X) + (PaperAxisY * Y) + Origin);

		Vertices.Emplace(FVector3f(Pos), FPaperSpriteTangents::PackedNormalX.ToFVector3f(), FPaperSpriteTangents::PackedNormalZ.ToFVector3f(), FVector2f(U, V), Color);
		++NumVertices;
	}

	template <typename VerticesArrayAllocator>
	inline void AddVertex(float X, float Y, float U, float V, const FVector& Origin, const FColor& Color, const FPackedNormal& TangentX, const FPackedNormal& TangentZ, TArray<FDynamicMeshVertex, VerticesArrayAllocator>& Vertices)
	{
		const FVector Pos((PaperAxisX * X) + (PaperAxisY * Y) + Origin);

		Vertices.Emplace(FVector3f(Pos), TangentX.ToFVector3f(), TangentZ.ToFVector3f(), FVector2f(U, V), Color);
		++NumVertices;
	}
};

//////////////////////////////////////////////////////////////////////////
// FPaperRenderSceneProxy

class FPaperRenderSceneProxy : public FPrimitiveSceneProxy
{
public:
	UE_API SIZE_T GetTypeHash() const override;

	UE_API FPaperRenderSceneProxy(const UPrimitiveComponent* InComponent);
	UE_API ~FPaperRenderSceneProxy();

	// FPrimitiveSceneProxy interface.
	UE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	UE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	UE_API virtual uint32 GetMemoryFootprint() const override;
	UE_API virtual bool CanBeOccluded() const override;
	UE_API virtual bool IsUsingDistanceCullFade() const override;
	UE_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	// End of FPrimitiveSceneProxy interface.

	UE_API void SetBodySetup_RenderThread(UBodySetup* NewSetup);

#if WITH_EDITOR
	UE_API void SetTransientTextureOverride_RenderThread(const UTexture* InTextureToModifyOverrideFor, UTexture* InOverrideTexture);
#endif

protected:
	UE_API virtual void GetDynamicMeshElementsForView(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector) const;

	UE_API bool GetMeshElement(FMeshElementCollector& Collector, int32 SectionIndex, uint8 DepthPriorityGroup, bool bIsSelected, FMeshBatch& OutMeshBatch) const;

	UE_API void GetNewBatchMeshes(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector) const;
	UE_API void GetNewBatchMeshesPrebuilt(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector) const;

	UE_API bool IsCollisionView(const FEngineShowFlags& EngineShowFlags, bool& bDrawSimpleCollision, bool& bDrawComplexCollision) const;

	UE_API virtual void DebugDrawCollision(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector, bool bDrawSolid) const;
	UE_API virtual void DebugDrawBodySetup(const FSceneView* View, int32 ViewIndex, FMeshElementCollector& Collector, UBodySetup* BodySetup, const FMatrix& GeomTransform, const FLinearColor& CollisionColor, bool bDrawSolid) const;

	// Call this if you modify BatchedSections or Vertices after the proxy has already been created
	UE_API void RecreateCachedRenderData(FRHICommandListBase& RHICmdList);

	UE_API FSpriteTextureOverrideRenderProxy* GetCachedMaterialProxyForSection(FMeshElementCollector& Collector, int32 SectionIndex, FMaterialRenderProxy* ParentMaterialProxy) const;

protected:
	TArray<FSpriteRenderSection> BatchedSections;
	TArray<FDynamicMeshVertex> Vertices;

	mutable UE::FMutex MaterialTextureOverrideProxiesMutex;
	mutable TArray<FSpriteTextureOverrideRenderProxy*> MaterialTextureOverrideProxies;

	FPaperSpriteVertexBuffer VertexBuffer;
	FPaperSpriteVertexFactory VertexFactory;

	//
	AActor* Owner;
	UBodySetup* MyBodySetup;

	uint8 bDrawTwoSided:1;
	uint8 bCastShadow:1;
	uint8 bSpritesUseVertexBufferPath:1;

	// The view relevance for the associated material
	FMaterialRelevance MaterialRelevance;

	// The Collision Response of the component being proxied
	FCollisionResponseContainer CollisionResponse;

	// The texture override list
	UE_EXPAND_IF_WITH_EDITOR(FPaperRenderSceneProxyTextureOverrideMap TextureOverrideList);
};

//////////////////////////////////////////////////////////////////////////
// FPaperRenderSceneProxy_SpriteBase - common base class for sprites and flipbooks (which build from sprites)

class FPaperRenderSceneProxy_SpriteBase : public FPaperRenderSceneProxy
{
public:
	FPaperRenderSceneProxy_SpriteBase(const UMeshComponent* InComponent);

	void SetSprite_RenderThread(const FSpriteDrawCallRecord& NewDynamicData, int32 SplitIndex);

public:
	UMaterialInterface* Material;
	UMaterialInterface* AlternateMaterial;
};

#undef UE_API
