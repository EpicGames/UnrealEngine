// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperRuntimeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperRuntimeSettings)

//////////////////////////////////////////////////////////////////////////
// UPaperRuntimeSettings

UPaperRuntimeSettings::UPaperRuntimeSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableSpriteAtlasGroups(false)
	, bEnableTerrainSplineEditing(false)
	, bResizeSpriteDataToMatchTextures(true)
{
}

