// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSprite.h"
#include "BodySetupEnums.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/MaterialInterface.h"
#include "Engine/CollisionProfile.h"
#include "PhysicsEngine/BodySetup.h"

#include "PaperCustomVersion.h"
#include "GeomTools.h"
#include "PaperSpriteComponent.h"
#include "PaperFlipbookComponent.h"
#include "PaperGroupedSpriteComponent.h"
#include "SpriteDrawCall.h"
#include "PaperFlipbook.h"
#include "Paper2DModule.h"
#include "Paper2DPrivate.h"
#include "UObject/ICookInfo.h"
#include "UObject/LinkerLoad.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSprite)

#if WITH_EDITOR
#endif

#if WITH_EDITOR
static void UpdateGeometryToBeBoxPositionRelative(FSpriteGeometryCollection& Geometry)
{
	// Make sure the per-shape GeometryType fields are up to date (introduced in this version)
	const bool bWasBoundingBox = (Geometry.GeometryType == ESpritePolygonMode::SourceBoundingBox) || (Geometry.GeometryType == ESpritePolygonMode::TightBoundingBox);

	if (bWasBoundingBox)
	{
		for (FSpriteGeometryShape& Shape : Geometry.Shapes)
		{
			Shape.ShapeType = ESpriteShapeType::Box;

			// Recenter the bounding box (BoxPosition is now defined as the center)
			const FVector2D AmountToSubtract = Shape.BoxPosition + Shape.BoxSize * 0.5f;
			Shape.BoxPosition += Shape.BoxSize * 0.5f;
			for (FVector2D& Vertex : Shape.Vertices)
			{
				Vertex -= AmountToSubtract;
			}
		}
	}
	else
	{
		for (FSpriteGeometryShape& Shape : Geometry.Shapes)
		{
			Shape.ShapeType = ESpriteShapeType::Polygon;

			// Make sure BoxPosition is zeroed since polygon points are relative to it now, but it was being ignored
			//@TODO: Consider computing the center and recentering verts to keep the numbers small/relative
			Shape.BoxPosition = FVector2D::ZeroVector;
			Shape.BoxSize = FVector2D::ZeroVector;
		}
	}
}
#endif

#if WITH_EDITOR

#include "PaperSpriteAtlas.h"
#include "BitmapUtils.h"
#include "ComponentReregisterContext.h"
#include "PaperRuntimeSettings.h"

//////////////////////////////////////////////////////////////////////////
// maf

void RemoveCollinearPoints(TArray<FIntPoint>& PointList)
{
	if (PointList.Num() < 3)
	{
		return;
	}
	
	// Wrap around to get the final pair of vertices (N-1, 0, 1)
	for (int32 VertexIndex = 1; VertexIndex <= PointList.Num() && PointList.Num() >= 3; )
	{
		const FVector2D A(PointList[VertexIndex-1]);
		const FVector2D B(PointList[VertexIndex % PointList.Num()]);
		const FVector2D C(PointList[(VertexIndex+1) % PointList.Num()]);

		// Determine if the area of the triangle ABC is zero (if so, they're collinear)
		const float AreaABC = (A.X * (B.Y - C.Y)) + (B.X * (C.Y - A.Y)) + (C.X * (A.Y - B.Y));

		if (FMath::Abs(AreaABC) < KINDA_SMALL_NUMBER)
		{
			// Remove B
			PointList.RemoveAt(VertexIndex % PointList.Num());
		}
		else
		{
			// Continue onwards
			++VertexIndex;
		}
	}
}

float DotPoints(const FIntPoint& V1, const FIntPoint& V2)
{
	return (V1.X * V2.X) + (V1.Y * V2.Y);
}

struct FDouglasPeuckerSimplifier
{
	FDouglasPeuckerSimplifier(const TArray<FIntPoint>& InSourcePoints, float Epsilon)
		: SourcePoints(InSourcePoints)
		, EpsilonSquared(Epsilon * Epsilon)
		, NumRemoved(0)
	{
		OmitPoints.AddZeroed(SourcePoints.Num());
	}

	TArray<FIntPoint> SourcePoints;
	TArray<bool> OmitPoints;
	const float EpsilonSquared;
	int32 NumRemoved;

	// Removes all points between Index1 and Index2, not including them
	void RemovePoints(int32 Index1, int32 Index2)
	{
		for (int32 Index = Index1 + 1; Index < Index2; ++Index)
		{
			OmitPoints[Index] = true;
			++NumRemoved;
		}
	}

	void SimplifyPointsInner(int Index1, int Index2)
	{
		if (Index2 - Index1 < 2)
		{
			return;
		}

		// Find furthest point from the V1..V2 line
		const FVector V1(SourcePoints[Index1].X, 0.0f, SourcePoints[Index1].Y);
		const FVector V2(SourcePoints[Index2].X, 0.0f, SourcePoints[Index2].Y);
		const FVector V1V2 = V2 - V1;
		const float LineScale = 1.0f / V1V2.SizeSquared();

		float FarthestDistanceSquared = -1.0f;
		int32 FarthestIndex = INDEX_NONE;

		for (int32 Index = Index1; Index < Index2; ++Index)
		{
			const FVector VTest(SourcePoints[Index].X, 0.0f, SourcePoints[Index].Y);
			const FVector V1VTest = VTest - V1;

			const float t = FMath::Clamp(FVector::DotProduct(V1VTest, V1V2) * LineScale, 0.0f, 1.0f);
			const FVector ClosestPointOnV1V2 = V1 + t * V1V2;

			const float DistanceToLineSquared = FVector::DistSquared(ClosestPointOnV1V2, VTest);
			if (DistanceToLineSquared > FarthestDistanceSquared)
			{
				FarthestDistanceSquared = DistanceToLineSquared;
				FarthestIndex = Index;
			}
		}

		if (FarthestDistanceSquared > EpsilonSquared)
		{
			// Too far, subdivide further
			SimplifyPointsInner(Index1, FarthestIndex);
			SimplifyPointsInner(FarthestIndex, Index2);
		}
		else
		{
			// The farthest point wasn't too far, so omit all the points in between
			RemovePoints(Index1, Index2);
		}
	}

	void Execute(TArray<FIntPoint>& Result)
	{
		SimplifyPointsInner(0, SourcePoints.Num() - 1);

		Result.Empty(SourcePoints.Num() - NumRemoved);
		for (int32 Index = 0; Index < SourcePoints.Num(); ++Index)
		{
			if (!OmitPoints[Index])
			{
				Result.Add(SourcePoints[Index]);
			}
		}
	}
};

static void BruteForceSimplifier(TArray<FIntPoint>& Points, float Epsilon)
{
	float FlatEdgeDistanceThreshold = (int)(Epsilon * Epsilon);

	// Run through twice to remove remnants from staircase artifacts
	for (int Pass = 0; Pass < 2; ++Pass)
	{
		for (int I = 0; I < Points.Num() && Points.Num() > 3; ++I)
		{
			int StartRemoveIndex = (I + 1) % Points.Num();
			int EndRemoveIndex = StartRemoveIndex;
			FIntPoint& A = Points[I];
			// Keep searching to find if any of the vector rejections fail in subsequent points on the polygon
			// A B C D E F (eg. when testing A B C, test rejection for BA, CA)
			// When testing A E F, test rejection for AB-AF, AC-AF, AD-AF, AE-AF
			// When one of these fails we discard all verts between A and one before the current vertex being tested
			for (int J = I; J < Points.Num(); ++J)
			{
				int IndexC = (J + 2) % Points.Num();
				FIntPoint& C = Points[IndexC];
				bool bSmallOffsetFailed = false;

				for (int K = I; K <= J && !bSmallOffsetFailed; ++K)
				{
					int IndexB = (K + 1) % Points.Num();
					FIntPoint& B = Points[IndexB];

					FVector2D CA = C - A;
					FVector2D BA = B - A;
					FVector2D Rejection_BA_CA = BA - (FVector2D::DotProduct(BA, CA) / FVector2D::DotProduct(CA, CA)) * CA;
					float RejectionLengthSquared = Rejection_BA_CA.SizeSquared();
					// If any of the points is behind the polyline up till now, it gets rejected. Staircase artefacts are handled in a second pass
					if (RejectionLengthSquared > FlatEdgeDistanceThreshold || FVector2D::CrossProduct(CA, BA) < 0) 
					{
						bSmallOffsetFailed = true;
						break;
					}
				}

				if (bSmallOffsetFailed)
				{
					break;
				}
				else
				{
					EndRemoveIndex = (EndRemoveIndex + 1) % Points.Num();
				}
			}

			// Remove the vertices that we deemed "too flat"
			if (EndRemoveIndex > StartRemoveIndex)
			{
				Points.RemoveAt(StartRemoveIndex, EndRemoveIndex - StartRemoveIndex);
			}
			else if (EndRemoveIndex < StartRemoveIndex)
			{
				Points.RemoveAt(StartRemoveIndex, Points.Num() - StartRemoveIndex);
				Points.RemoveAt(0, EndRemoveIndex);
				// The search has wrapped around, no more vertices to test
				break;
			}
		}
	} // Pass
}

void SimplifyPoints(TArray<FIntPoint>& Points, float Epsilon)
{
//	FDouglasPeuckerSimplifier Simplifier(Points, Epsilon);
//	Simplifier.Execute(Points);
	BruteForceSimplifier(Points, Epsilon);
}

	
//////////////////////////////////////////////////////////////////////////
// FBoundaryImage

struct FBoundaryImage
{
	TArray<int8> Pixels;

	// Value to return out of bounds
	int8 OutOfBoundsValue;

	int32 X0;
	int32 Y0;
	int32 Width;
	int32 Height;

	FBoundaryImage(const FIntPoint& Pos, const FIntPoint& Size)
	{
		OutOfBoundsValue = 0;

		X0 = Pos.X - 1;
		Y0 = Pos.Y - 1;
		Width = Size.X + 2;
		Height = Size.Y + 2;

		Pixels.AddZeroed(Width * Height);
	}

	int32 GetIndex(int32 X, int32 Y) const
	{
		const int32 LocalX = X - X0;
		const int32 LocalY = Y - Y0;

		if ((LocalX >= 0) && (LocalX < Width) && (LocalY >= 0) && (LocalY < Height))
		{
			return LocalX + (LocalY * Width);
		}
		else
		{
			return INDEX_NONE;
		}
	}

	int8 GetPixel(int32 X, int32 Y) const
	{
		const int32 Index = GetIndex(X, Y);
		if (Index != INDEX_NONE)
		{
			return Pixels[Index];
		}
		else
		{
			return OutOfBoundsValue;
		}
	}

	void SetPixel(int32 X, int32 Y, int8 Value)
	{
		const int32 Index = GetIndex(X, Y);
		if (Index != INDEX_NONE)
		{
			Pixels[Index] = Value;
		}
	}
};

void UPaperSprite::ExtractSourceRegionFromTexturePoint(const FVector2D& SourcePoint)
{
	FIntPoint SourceIntPoint(FMath::RoundToInt(SourcePoint.X), FMath::RoundToInt(SourcePoint.Y));
	FIntPoint ClosestValidPoint;

	FBitmap Bitmap(GetSourceTexture(), 0, 0);
	if (Bitmap.IsValid() && Bitmap.FoundClosestValidPoint(SourceIntPoint.X, SourceIntPoint.Y, 10, /*out*/ ClosestValidPoint))
	{
		FIntPoint Origin;
		FIntPoint Dimension;
		if (Bitmap.HasConnectedRect(ClosestValidPoint.X, ClosestValidPoint.Y, false, /*out*/ Origin, /*out*/ Dimension))
		{
			if (Dimension.X > 0 && Dimension.Y > 0)
			{
				// Mark the package as dirty
				Modify();

				SourceUV = FVector2D(Origin.X, Origin.Y);
				SourceDimension = FVector2D(Dimension.X, Dimension.Y);
				PostEditChange();
				RefreshBakedData();
			}
		}
	}
}

#endif

//////////////////////////////////////////////////////////////////////////
// FSpriteDrawCallRecord

void FSpriteDrawCallRecord::BuildFromSprite(const UPaperSprite* Sprite)
{
	if (Sprite != nullptr)
	{
		Destination = FVector::ZeroVector;
		BaseTexture = Sprite->GetBakedTexture();
		Sprite->GetBakedAdditionalSourceTextures(/*out*/ AdditionalTextures);

		Color = FColor::White;

		RenderVerts = Sprite->BakedRenderData;
	}
}

//////////////////////////////////////////////////////////////////////////
// UPaperSprite

UPaperSprite::UPaperSprite(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Default to using physics
	SpriteCollisionDomain = ESpriteCollisionMode::Use3DPhysics;
	
	AlternateMaterialSplitIndex = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	PivotMode = ESpritePivotMode::Center_Center;
	bSnapPivotToPixelGrid = true;

	CollisionGeometry.GeometryType = ESpritePolygonMode::TightBoundingBox;
	CollisionThickness = 10.0f;

	bTrimmedInSourceImage = false;
	bRotatedInSourceImage = false;

	SourceTextureDimension.Set(0, 0);
#endif

	PixelsPerUnrealUnit = 2.56f;

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> MaskedMaterialRef(TEXT("/Paper2D/MaskedUnlitSpriteMaterial"));
	DefaultMaterial = MaskedMaterialRef.Object;

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> OpaqueMaterialRef(TEXT("/Paper2D/OpaqueUnlitSpriteMaterial"));
	AlternateMaterial = OpaqueMaterialRef.Object;
}

#if WITH_EDITOR

void UPaperSprite::OnObjectReimported(UTexture2D* Texture)
{
	// Check if its our source texture, and if its dimensions have changed
	// If SourceTetxureDimension == 0, we don't have a previous dimension to work off, so can't
	// rescale sensibly
	if (Texture == GetSourceTexture())
	{
		if (NeedRescaleSpriteData())
		{
			RescaleSpriteData(GetSourceTexture());
			PostEditChange();
		}
		else if (AtlasGroup != nullptr)
		{
			AtlasGroup->PostEditChange();
		}
	}
}

#endif


#if WITH_EDITOR

/** Removes all components that use the specified sprite asset from their scenes for the lifetime of the class. */
class FSpriteReregisterContext
{
public:
	/** Initialization constructor. */
	FSpriteReregisterContext(UPaperSprite* TargetAsset)
	{
		// Look at sprite components
		for (UPaperSpriteComponent* TestComponent : TObjectRange<UPaperSpriteComponent>())
		{
			if (TestComponent->GetSprite() == TargetAsset)
			{
				AddComponentToRefresh(TestComponent);
			}
		}

		// Look at flipbook components
		for (UPaperFlipbookComponent* TestComponent : TObjectRange<UPaperFlipbookComponent>())
		{
			if (UPaperFlipbook* Flipbook = TestComponent->GetFlipbook())
			{
				if (Flipbook->ContainsSprite(TargetAsset))
				{
					AddComponentToRefresh(TestComponent);
				}
			}
		}

		// Look at grouped sprite components
		for (UPaperGroupedSpriteComponent* TestComponent : TObjectRange<UPaperGroupedSpriteComponent>())
		{
			if (TestComponent->ContainsSprite(TargetAsset))
			{
				AddComponentToRefresh(TestComponent);
			}
		}
	}

protected:
	void AddComponentToRefresh(UActorComponent* Component)
	{
		if (ComponentContexts.Num() == 0)
		{
			// wait until resources are released
			FlushRenderingCommands();
		}

		ComponentContexts.Add(new FComponentReregisterContext(Component));
	}

private:
	/** The recreate contexts for the individual components. */
	TIndirectArray<FComponentReregisterContext> ComponentContexts;
};

void UPaperSprite::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	//@TODO: Determine when this is really needed, as it is seriously expensive!
	FSpriteReregisterContext ReregisterExistingComponents(this);

	// Look for changed properties, if null assume all properties might have changed.
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (PixelsPerUnrealUnit <= 0.0f)
	{
		PixelsPerUnrealUnit = 1.0f;
	}

	if (CollisionGeometry.GeometryType == ESpritePolygonMode::Diced)
	{
		// Disallow dicing on collision geometry for now
		CollisionGeometry.GeometryType = ESpritePolygonMode::SourceBoundingBox;
	}
	RenderGeometry.PixelsPerSubdivisionX = FMath::Max(RenderGeometry.PixelsPerSubdivisionX, 4);
	RenderGeometry.PixelsPerSubdivisionY = FMath::Max(RenderGeometry.PixelsPerSubdivisionY, 4);

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceUV) || PropertyName == NAME_None)
	{
		SourceUV.X = FMath::Max(FMath::RoundToFloat(SourceUV.X), 0.0f);
		SourceUV.Y = FMath::Max(FMath::RoundToFloat(SourceUV.Y), 0.0f);
	}

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceDimension) || PropertyName == NAME_None)
	{
		SourceDimension.X = FMath::Max(FMath::RoundToFloat(SourceDimension.X), 0.0f);
		SourceDimension.Y = FMath::Max(FMath::RoundToFloat(SourceDimension.Y), 0.0f);
	}

	// Update the pivot (roundtripping thru the function will round to a pixel position if that option is enabled)
	CustomPivotPoint = GetPivotPosition();

	bool bRebuilAtlas = false;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceTexture) || PropertyName == NAME_None)
	{
		const bool bResizeSprite = 
			(SourceTexture != nullptr) &&
			// If this is a brand new sprite that didn't have a texture set previously, act like we were factoried with the texture
			// If this sprite belongs to an AtlasGroup, then we should resize everything, since we want the entire new texture to be
			// added to the atlas, rather than a subuv.
			(SourceDimension.IsNearlyZero() || (AtlasGroup != nullptr));

		if (bResizeSprite)
		{
			// If this is a brand new sprite that didn't have a texture set previously, act like we were factoried with the texture
			SourceUV = FVector2D::ZeroVector;
			SourceDimension = FVector2D(SourceTexture->GetImportedSize());
			SourceTextureDimension = SourceDimension;
		}

		bRebuilAtlas = true;
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperSprite, Sockets) ||
		(MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPaperSprite, Sockets) && PropertyName == GET_MEMBER_NAME_CHECKED(FPaperSpriteSocket, SocketName)) ||
		PropertyName == NAME_None)
	{
		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			ValidateSocketNames();
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperSprite, AtlasGroup))
	{
		bRebuilAtlas = true;
	}

	// Don't do rebuilds during an interactive event to make things more responsive.
	// They'll always be followed by a ValueSet event at the end to force the change.
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if (bRebuilAtlas)
		{
			TSoftObjectPtr<UPaperSpriteAtlas> AtlasGroupPath = AtlasGroup;
			if (PreviousAtlasGroup != AtlasGroupPath)
			{
				UPaperSpriteAtlas* PreviousAtlasGroupPtr = PreviousAtlasGroup.LoadSynchronous();

				// Update previous
				if (PreviousAtlasGroupPtr != nullptr)
				{
					PreviousAtlasGroupPtr->PostEditChange();
				}

				// Update cached previous atlas group
				PreviousAtlasGroup = AtlasGroup;
			}

			// Rebuild atlas group
			if (AtlasGroup != nullptr)
			{
				AtlasGroup->PostEditChange();
			}
			else
			{
				BakedSourceTexture = nullptr;
				BakedSourceUV = FVector2D(0, 0);
				BakedSourceDimension = FVector2D(0, 0);
			}
		}


		RefreshBakedData();
		RebuildCollisionData();
		RebuildRenderData();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UPaperSprite::RescaleSpriteData(UTexture2D* Texture)
{
	Texture->ConditionalPostLoad();

	FVector2D PreviousTextureDimension = SourceTextureDimension;
	FVector2D NewTextureDimension(Texture->GetImportedSize().X, Texture->GetImportedSize().Y);

	// Don't ever divby0 (no previously stored texture dimensions)
	// or scale to 0, should be covered by NeedRescaleSpriteData
	if (NewTextureDimension.X == 0 || NewTextureDimension.Y == 0 ||
		PreviousTextureDimension.X == 0 || PreviousTextureDimension.Y == 0)
	{
		return;
	}

	const FVector2D& S = NewTextureDimension;
	const FVector2D& D = PreviousTextureDimension;

	struct Local
	{
		static float NoSnap(const float Value, const float Scale, const float Divisor)
		{
			return (Value * Scale) / Divisor;
		}

		static FVector2D RescaleNeverSnap(const FVector2D& Value, const FVector2D& Scale, const FVector2D& Divisor)
		{
			return FVector2D(NoSnap(Value.X, Scale.X, Divisor.X), NoSnap(Value.Y, Scale.Y, Divisor.Y));
		}

		static FVector2D Rescale(const FVector2D& Value, const FVector2D& Scale, const FVector2D& Divisor)
		{
			// Never snap, want to be able to return to original values when rescaled back
			return RescaleNeverSnap(Value, Scale, Divisor);
			//return FVector2D(FMath::FloorToFloat(NoSnap(Value.X, Scale.X, Divisor.X)), FMath::FloorToFloat(NoSnap(Value.Y, Scale.Y, Divisor.Y)));
		}
	};

	// Sockets are in pivot space, convert these to texture space to apply later
	TArray<FVector2D> RescaledTextureSpaceSocketPositions;
	for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); ++SocketIndex)
	{
		FPaperSpriteSocket& Socket = Sockets[SocketIndex];
		FVector Translation = Socket.LocalTransform.GetTranslation();
		FVector2D TextureSpaceSocketPosition = ConvertPivotSpaceToTextureSpace(FVector2D(Translation.X, Translation.Z));
		RescaledTextureSpaceSocketPositions.Add(Local::RescaleNeverSnap(TextureSpaceSocketPosition, S, D));
	}

	SourceUV = Local::Rescale(SourceUV, S, D);
	SourceDimension = Local::Rescale(SourceDimension, S, D);
	SourceImageDimensionBeforeTrimming = Local::Rescale(SourceImageDimensionBeforeTrimming, S, D);
	SourceTextureDimension = NewTextureDimension;

	if (bSnapPivotToPixelGrid)
	{
		CustomPivotPoint = Local::Rescale(CustomPivotPoint, S, D);
	}
	else
	{
		CustomPivotPoint = Local::RescaleNeverSnap(CustomPivotPoint, S, D);
	}

	for (int32 GeometryIndex = 0; GeometryIndex < 2; ++GeometryIndex)
	{
		FSpriteGeometryCollection& Geometry = (GeometryIndex == 0) ? CollisionGeometry : RenderGeometry;
		for (FSpriteGeometryShape& Shape : Geometry.Shapes)
		{
			Shape.BoxPosition = Local::Rescale(Shape.BoxPosition, S, D);
			Shape.BoxSize = Local::Rescale(Shape.BoxSize, S, D);

			for (FVector2D& Vertex : Shape.Vertices)
			{
				const FVector2D TextureSpaceVertex = Shape.ConvertShapeSpaceToTextureSpace(Vertex);
				const FVector2D ScaledTSVertex = Local::Rescale(TextureSpaceVertex, S, D);
				Vertex = Shape.ConvertTextureSpaceToShapeSpace(ScaledTSVertex);
			}
		}
	}

	// Apply texture space pivot positions now that pivot space is correctly defined
	for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); ++SocketIndex)
	{
		const FVector2D PivotSpaceSocketPosition = ConvertTextureSpaceToPivotSpace(RescaledTextureSpaceSocketPositions[SocketIndex]);

		FPaperSpriteSocket& Socket = Sockets[SocketIndex];
		FVector Translation = Socket.LocalTransform.GetTranslation();
		Translation.X = PivotSpaceSocketPosition.X;
		Translation.Z = PivotSpaceSocketPosition.Y;
		Socket.LocalTransform.SetTranslation(Translation);
	}

	RefreshBakedData();
}

bool UPaperSprite::NeedRescaleSpriteData()
{
	const bool bSupportsRescaling = GetDefault<UPaperRuntimeSettings>()->bResizeSpriteDataToMatchTextures;

	if (bSupportsRescaling)
	{
		if (UTexture2D* Texture = GetSourceTexture())
		{
			Texture->ConditionalPostLoad();
			const FIntPoint TextureSize = Texture->GetImportedSize();
			const bool bTextureSizeIsZero = (TextureSize.X == 0) || (TextureSize.Y == 0);
			return !SourceTextureDimension.IsZero() && !bTextureSizeIsZero && ((TextureSize.X != SourceTextureDimension.X) || (TextureSize.Y != SourceTextureDimension.Y));
		}
	}

	return false;
}

class FPaperSpriteToBodySetupBuilder : public FSpriteGeometryCollisionBuilderBase
{
public:
	FPaperSpriteToBodySetupBuilder(UPaperSprite* InSprite, UBodySetup* InBodySetup)
		: FSpriteGeometryCollisionBuilderBase(InBodySetup)
		, MySprite(InSprite)
	{
		UnrealUnitsPerPixel = InSprite->GetUnrealUnitsPerPixel();
		CollisionThickness = InSprite->GetCollisionThickness();
		CollisionDomain = InSprite->GetSpriteCollisionDomain();
	}

protected:
	// FSpriteGeometryCollisionBuilderBase interface
	virtual FVector2D ConvertTextureSpaceToPivotSpace(const FVector2D& Input) const override
	{
		return MySprite->ConvertTextureSpaceToPivotSpace(Input);
	}

	virtual FVector2D ConvertTextureSpaceToPivotSpaceNoTranslation(const FVector2D& Input) const override
	{
		return MySprite->IsRotatedInSourceImage() ? FVector2D(Input.Y, Input.X) : Input;
	}
	// End of FSpriteGeometryCollisionBuilderBase

protected:
	UPaperSprite* MySprite;
};

void UPaperSprite::RebuildCollisionData()
{
	UBodySetup* OldBodySetup = BodySetup;

	// Ensure we have the data structure for the desired collision method
	switch (SpriteCollisionDomain)
	{
	case ESpriteCollisionMode::Use3DPhysics:
		BodySetup = NewObject<UBodySetup>(this);
		break;
	case ESpriteCollisionMode::None:
		BodySetup = nullptr;
		CollisionGeometry.Reset();
		break;
	}

	if (SpriteCollisionDomain != ESpriteCollisionMode::None)
	{
		check(BodySetup);
		BodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
		switch (CollisionGeometry.GeometryType)
		{
		case ESpritePolygonMode::Diced:
		case ESpritePolygonMode::SourceBoundingBox:
			// Ignore diced, treat it like SourceBoundingBox, which just uses the loose bounds
			CreatePolygonFromBoundingBox(CollisionGeometry, /*bUseTightBounds=*/ false);
			break;

		case ESpritePolygonMode::TightBoundingBox:
			// Analyze the texture to tighten the bounds
			CreatePolygonFromBoundingBox(CollisionGeometry, /*bUseTightBounds=*/ true);
			break;

		case ESpritePolygonMode::ShrinkWrapped:
			// Analyze the texture and rebuild the geometry
			BuildGeometryFromContours(CollisionGeometry);
			break;

		case ESpritePolygonMode::FullyCustom:
			// Nothing to rebuild, the data is already ready
			break;
		default:
			check(false); // unknown mode
		};

		// Clean up the geometry (converting polygons back to bounding boxes, etc...)
		CollisionGeometry.ConditionGeometry();

		// Take the geometry and add it to the body setup
		FPaperSpriteToBodySetupBuilder CollisionBuilder(this, BodySetup);
		CollisionBuilder.ProcessGeometry(CollisionGeometry);
		CollisionBuilder.Finalize();

		// Copy across or initialize the only editable property we expose on the body setup
		if (OldBodySetup != nullptr)
		{
			BodySetup->DefaultInstance.CopyBodyInstancePropertiesFrom(&(OldBodySetup->DefaultInstance));
		}
		else
		{
			BodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
		}
	}
}

void UPaperSprite::RebuildRenderData()
{
	FSpriteGeometryCollection AlternateGeometry;

	switch (RenderGeometry.GeometryType)
	{
	case ESpritePolygonMode::Diced:
	case ESpritePolygonMode::SourceBoundingBox:
		CreatePolygonFromBoundingBox(RenderGeometry, /*bUseTightBounds=*/ false);
		break;

	case ESpritePolygonMode::TightBoundingBox:
		CreatePolygonFromBoundingBox(RenderGeometry, /*bUseTightBounds=*/ true);
		break;

	case ESpritePolygonMode::ShrinkWrapped:
		BuildGeometryFromContours(RenderGeometry);
		break;

	case ESpritePolygonMode::FullyCustom:
		// Do nothing special, the data is already in the polygon
		break;
	default:
		check(false); // unknown mode
	};

	// Determine the texture size
	UTexture2D* EffectiveTexture = GetBakedTexture();

	FVector2D TextureSize(1.0f, 1.0f);
	if (EffectiveTexture)
	{
		EffectiveTexture->ConditionalPostLoad();
		TextureSize = FVector2D(EffectiveTexture->GetImportedSize());
	}
	const float InverseWidth = 1.0f / TextureSize.X;
	const float InverseHeight = 1.0f / TextureSize.Y;

	// Adjust for the pivot and store in the baked geometry buffer
	const FVector2D DeltaUV((BakedSourceTexture != nullptr) ? (BakedSourceUV - SourceUV) : FVector2D::ZeroVector);

	const float UnitsPerPixel = GetUnrealUnitsPerPixel();

	if ((RenderGeometry.GeometryType == ESpritePolygonMode::Diced) && (EffectiveTexture != nullptr))
	{
		const int32 AlphaThresholdInt = FMath::Clamp<int32>(RenderGeometry.AlphaThreshold * 255, 0, 255);
		FAlphaBitmap SourceBitmap(EffectiveTexture);
		SourceBitmap.ThresholdImageBothWays(AlphaThresholdInt, 255);

		bool bSeparateOpaqueSections = true;

		// Dice up the source geometry and sort into translucent and opaque sections
		RenderGeometry.Shapes.Empty();

		const int32 X0 = (int32)SourceUV.X;
		const int32 Y0 = (int32)SourceUV.Y;
		const int32 X1 = (int32)(SourceUV.X + SourceDimension.X);
		const int32 Y1 = (int32)(SourceUV.Y + SourceDimension.Y);

		for (int32 Y = Y0; Y < Y1; Y += RenderGeometry.PixelsPerSubdivisionY)
		{
			const int32 TileHeight = FMath::Min(RenderGeometry.PixelsPerSubdivisionY, Y1 - Y);

			for (int32 X = X0; X < X1; X += RenderGeometry.PixelsPerSubdivisionX)
			{
				const int32 TileWidth = FMath::Min(RenderGeometry.PixelsPerSubdivisionX, X1 - X);

				if (!SourceBitmap.IsRegionEmpty(X, Y, X + TileWidth - 1, Y + TileHeight - 1))
				{
					FIntPoint Origin(X, Y);
					FIntPoint Dimension(TileWidth, TileHeight);
					
					SourceBitmap.TightenBounds(Origin, Dimension);

					bool bOpaqueSection = false;
					if (bSeparateOpaqueSections)
					{
						if (SourceBitmap.IsRegionEqual(Origin.X, Origin.Y, Origin.X + Dimension.X - 1, Origin.Y + Dimension.Y - 1, 255))
						{
							bOpaqueSection = true;
						}
					}

					const FVector2D BoxCenter = FVector2D(Origin) + (FVector2D(Dimension) * 0.5f);
					if (bOpaqueSection)
					{
						AlternateGeometry.AddRectangleShape(BoxCenter, Dimension);
					}
					else
					{
						RenderGeometry.AddRectangleShape(BoxCenter, Dimension);
					}
				}
			}
		}
	}

	// Triangulate the render geometry
	TArray<FVector2D> TriangluatedPoints;
	RenderGeometry.Triangulate(/*out*/ TriangluatedPoints, /*bIncludeBoxes=*/ true);

	// Triangulate the alternate render geometry, if present
	if (AlternateGeometry.Shapes.Num() > 0)
	{
		TArray<FVector2D> AlternateTriangluatedPoints;
		AlternateGeometry.Triangulate(/*out*/ AlternateTriangluatedPoints, /*bIncludeBoxes=*/ true);

		AlternateMaterialSplitIndex = TriangluatedPoints.Num();
		TriangluatedPoints.Append(AlternateTriangluatedPoints);
		RenderGeometry.Shapes.Append(AlternateGeometry.Shapes);
	}
	else
	{
		AlternateMaterialSplitIndex = INDEX_NONE;
	}

	// Bake the verts
	BakedRenderData.Empty(TriangluatedPoints.Num());
	for (int32 PointIndex = 0; PointIndex < TriangluatedPoints.Num(); ++PointIndex)
	{
		const FVector2D& SourcePos = TriangluatedPoints[PointIndex];

		const FVector2D PivotSpacePos = ConvertTextureSpaceToPivotSpace(SourcePos);
		const FVector2D UV(SourcePos + DeltaUV);

		new (BakedRenderData) FVector4(PivotSpacePos.X * UnitsPerPixel, PivotSpacePos.Y * UnitsPerPixel, UV.X * InverseWidth, UV.Y * InverseHeight);
	}

	check((BakedRenderData.Num() % 3) == 0);

	// Swap the generated vertices so they end up in counterclockwise order
	for (int32 SVT = 0; SVT < TriangluatedPoints.Num(); SVT += 3)
	{
		Swap(BakedRenderData[SVT + 2], BakedRenderData[SVT + 0]);
	}
}

void UPaperSprite::FindTextureBoundingBox(float AlphaThreshold, /*out*/ FVector2D& OutBoxPosition, /*out*/ FVector2D& OutBoxSize)
{
	// Create an initial guess at the bounds based on the source rectangle
	int32 LeftBound = (int32)SourceUV.X;
	int32 RightBound = (int32)(SourceUV.X + SourceDimension.X - 1);

	int32 TopBound = (int32)SourceUV.Y;
	int32 BottomBound = (int32)(SourceUV.Y + SourceDimension.Y - 1);

	const int32 AlphaThresholdInt = FMath::Clamp<int32>(AlphaThreshold * 255, 0, 255);
	FBitmap SourceBitmap(GetSourceTexture(), AlphaThresholdInt);
	if (SourceBitmap.IsValid())
	{
		// Make sure the initial bounds starts in the texture
		LeftBound = FMath::Clamp(LeftBound, 0, SourceBitmap.Width-1);
		RightBound = FMath::Clamp(RightBound, 0, SourceBitmap.Width-1);
		TopBound = FMath::Clamp(TopBound, 0, SourceBitmap.Height-1);
		BottomBound = FMath::Clamp(BottomBound, 0, SourceBitmap.Height-1);

		// Pull it in from the top
		while ((TopBound < BottomBound) && SourceBitmap.IsRowEmpty(LeftBound, RightBound, TopBound))
		{
			++TopBound;
		}

		// Pull it in from the bottom
		while ((BottomBound > TopBound) && SourceBitmap.IsRowEmpty(LeftBound, RightBound, BottomBound))
		{
			--BottomBound;
		}

		// Pull it in from the left
		while ((LeftBound < RightBound) && SourceBitmap.IsColumnEmpty(LeftBound, TopBound, BottomBound))
		{
			++LeftBound;
		}

		// Pull it in from the right
		while ((RightBound > LeftBound) && SourceBitmap.IsColumnEmpty(RightBound, TopBound, BottomBound))
		{
			--RightBound;
		}
	}

	OutBoxSize.X = RightBound - LeftBound + 1;
	OutBoxSize.Y = BottomBound - TopBound + 1;
	OutBoxPosition.X = LeftBound;
	OutBoxPosition.Y = TopBound;
}

// Get a divisor ("pixel" size) from the "detail" parameter
// Size is fed in for possible changes later
static int32 GetDivisorFromDetail(const FIntPoint& Size, float Detail)
{
	//@TODO: Consider MaxSize somehow when deciding divisor
	//int32 MaxSize = FMath::Max(Size.X, Size.Y);
	return  FMath::Lerp(8, 1, FMath::Clamp(Detail, 0.0f, 1.0f));
}

void UPaperSprite::BuildGeometryFromContours(FSpriteGeometryCollection& GeomOwner)
{
	// First trim the image to the tight fitting bounding box (the other pixels can't matter)
	FVector2D InitialBoxSizeFloat;
	FVector2D InitialBoxPosFloat;
	FindTextureBoundingBox(GeomOwner.AlphaThreshold, /*out*/ InitialBoxPosFloat, /*out*/ InitialBoxSizeFloat);

	const FIntPoint InitialPos((int32)InitialBoxPosFloat.X, (int32)InitialBoxPosFloat.Y);
	const FIntPoint InitialSize((int32)InitialBoxSizeFloat.X, (int32)InitialBoxSizeFloat.Y);

	// Find the contours
	TArray< TArray<FIntPoint> > Contours;

	// DK: FindContours only returns positive contours, i.e. outsides
	// Contour generation is simplified in FindContours by downscaling the detail prior to generating contour data
	FindContours(InitialPos, InitialSize, GeomOwner.AlphaThreshold, GeomOwner.DetailAmount, GetSourceTexture(), /*out*/ Contours);

	// Convert the contours into geometry
	GeomOwner.Shapes.Empty();
	for (int32 ContourIndex = 0; ContourIndex < Contours.Num(); ++ContourIndex)
	{
		TArray<FIntPoint>& Contour = Contours[ContourIndex];

		// Scale the simplification epsilon by the size we know the pixels will be
		int Divisor = GetDivisorFromDetail(InitialSize, GeomOwner.DetailAmount);
		SimplifyPoints(Contour, GeomOwner.SimplifyEpsilon * Divisor);

		if (Contour.Num() > 0)
		{
			FSpriteGeometryShape& NewShape = *new (GeomOwner.Shapes) FSpriteGeometryShape();
			NewShape.ShapeType = ESpriteShapeType::Polygon;
			NewShape.Vertices.Empty(Contour.Num());

			// Add the points
			for (int32 PointIndex = 0; PointIndex < Contour.Num(); ++PointIndex)
			{
				new (NewShape.Vertices) FVector2D(NewShape.ConvertTextureSpaceToShapeSpace(Contour[PointIndex]));
			}

			// Recenter them
			const FVector2D AverageCenterFloat = NewShape.GetPolygonCentroid();
			const FVector2D AverageCenterSnapped(FMath::RoundToInt(AverageCenterFloat.X), FMath::RoundToInt(AverageCenterFloat.Y));
			NewShape.SetNewPivot(AverageCenterSnapped);

			// Get intended winding
			NewShape.bNegativeWinding = !FGeomTools2D::IsPolygonWindingCCW(NewShape.Vertices);
		}
	}
}


static void TraceContour(TArray<FIntPoint>& Result, const TArray<FIntPoint>& Points)
{
	const int PointCount = Points.Num();
	if (PointCount < 2)
	{
		return;
	}

	int CurrentX = (int)Points[0].X;
	int CurrentY = (int)Points[0].Y;
	int CurrentDirection = 0;
	int FirstDx = (int)Points[1].X - CurrentX;
	int FirstDy = (int)Points[1].Y - CurrentY;
	
	if (FirstDx == 1 && FirstDy == 0) CurrentDirection = 0;
	else if (FirstDx == 1 && FirstDy == 1) CurrentDirection = 1;
	else if (FirstDx == 0 && FirstDy == 1) CurrentDirection = 1;
	else if (FirstDx == -1 && FirstDy == 1) CurrentDirection = 2;
	else if (FirstDx == -1 && FirstDy == 0) CurrentDirection = 2;
	else if (FirstDx == -1 && FirstDy == -1) CurrentDirection = 3;
	else if (FirstDx == 0 && FirstDy == -1) CurrentDirection = 3;
	else if (FirstDx == 1 && FirstDy == -1) CurrentDirection = 0;

	int CurrentPointIndex = 0;

	const int StartX = CurrentX;
	const int StartY = CurrentY;
	const int StartDirection = CurrentDirection;

	static const int DirectionDx[] = { 1, 0, -1, 0 };
	static const int DirectionDy[] = { 0, 1, 0, -1 };

	bool bFinished = false;
	while (!bFinished)
	{
		const FIntPoint& NextPoint = Points[(CurrentPointIndex + 1) % PointCount];
		const int NextDx = (int)NextPoint.X - CurrentX;
		const int NextDy = (int)NextPoint.Y - CurrentY;

		int LeftDirection = (CurrentDirection + 3) % 4;
		int CurrentDx = DirectionDx[CurrentDirection];
		int CurrentDy = DirectionDy[CurrentDirection];
		int LeftDx = DirectionDx[LeftDirection];
		int LeftDy = DirectionDy[LeftDirection];
		bool bDidMove = true;
		if (NextDx != 0 || NextDy != 0)
		{
			if (NextDx == LeftDx && NextDy == LeftDy)
			{
				// Space to the left, turn left and move forwards
				CurrentDirection = LeftDirection;
				CurrentX += LeftDx;
				CurrentY += LeftDy;
			}
			else
			{
				// Wall to the left. Add the corner vertex to our output.
				Result.Add(FIntPoint((int)((float)CurrentX + 0.5f + (float)(CurrentDx + LeftDx) * 0.5f), (int)((float)CurrentY + 0.5f + (float)(CurrentDy + LeftDy) * 0.5f)));
				if (NextDx == CurrentDx && NextDy == CurrentDy)
				{
					// Move forward
					CurrentX += CurrentDx;
					CurrentY += CurrentDy;
				}
				else if (NextDx == CurrentDx + LeftDx && NextDy == CurrentDy + LeftDy)
				{
					// Move forward, turn left, move forwards again
					CurrentX += CurrentDx;
					CurrentY += CurrentDy;
					CurrentDirection = LeftDirection;
					CurrentX += LeftDx;
					CurrentY += LeftDy;
				}
				else
				{
					// Turn right
					CurrentDirection = (CurrentDirection + 1) % 4;
					bDidMove = false;
				}
			}
		}
		if (bDidMove)
		{
			++CurrentPointIndex;
		}

		if (CurrentX == StartX && CurrentY == StartY && CurrentDirection == StartDirection)
		{
			bFinished = true;
		}
	}
}

void UPaperSprite::FindContours(const FIntPoint& ScanPos, const FIntPoint& ScanSize, float AlphaThreshold, float Detail, UTexture2D* Texture, TArray< TArray<FIntPoint> >& OutPoints)
{
	OutPoints.Empty();

	if ((ScanSize.X <= 0) || (ScanSize.Y <= 0))
	{
		return;
	}

	// Neighborhood array (clockwise starting at -X,-Y; assuming prev is at -X)
	const int32 NeighborX[] = {-1, 0,+1,+1,+1, 0,-1,-1};
	const int32 NeighborY[] = {-1,-1,-1, 0,+1,+1,+1, 0};
	//                       0  1  2  3  4  5  6  7
	// 0 1 2
	// 7   3
	// 6 5 4
	const int32 StateMutation[] = {
		5, //from0
		6, //from1
		7, //from2
		0, //from3
		1, //from4
		2, //from5
		3, //from6
		4, //from7
	};

	int32 AlphaThresholdInt = FMath::Clamp<int32>(AlphaThreshold * 255, 0, 255);

	FBitmap FullSizeBitmap(Texture, AlphaThresholdInt);
	if (FullSizeBitmap.IsValid())
	{
		const int32 DownsampleAmount = GetDivisorFromDetail(ScanSize, Detail);

		FBitmap SourceBitmap((ScanSize.X + DownsampleAmount - 1) / DownsampleAmount, (ScanSize.Y + DownsampleAmount - 1) / DownsampleAmount, 0);
		for (int32 Y = 0; Y < ScanSize.Y; ++Y)
		{
			for (int32 X = 0; X < ScanSize.X; ++X)
			{
				SourceBitmap.SetPixel(X / DownsampleAmount, Y / DownsampleAmount, SourceBitmap.GetPixel(X / DownsampleAmount, Y / DownsampleAmount) | FullSizeBitmap.GetPixel(ScanPos.X + X, ScanPos.Y + Y));
			}
		}

		const int32 LeftBound = 0;
		const int32 RightBound = SourceBitmap.Width - 1;
		const int32 TopBound = 0;
		const int32 BottomBound = SourceBitmap.Height - 1;

		//checkSlow((LeftBound >= 0) && (TopBound >= 0) && (RightBound < SourceBitmap.Width) && (BottomBound < SourceBitmap.Height));

		// Create the 'output' boundary image
		FBoundaryImage BoundaryImage(FIntPoint(0, 0), FIntPoint(SourceBitmap.Width, SourceBitmap.Height));

		bool bInsideBoundary = false;

		for (int32 Y = TopBound - 1; Y < BottomBound + 2; ++Y)
		{
			for (int32 X = LeftBound - 1; X < RightBound + 2; ++X)
			{
				const bool bAlreadyTaggedAsBoundary = BoundaryImage.GetPixel(X, Y) > 0;
				const bool bPixelInsideBounds = (X >= LeftBound && X <= RightBound && Y >= TopBound && Y <= BottomBound);
				const bool bIsFilledPixel = bPixelInsideBounds && SourceBitmap.GetPixel(X, Y) != 0;

				if (bInsideBoundary)
				{
					if (!bIsFilledPixel)
					{
						// We're leaving the known boundary
						bInsideBoundary = false;
					}
				}
				else
				{
					if (bAlreadyTaggedAsBoundary)
					{
						// We're re-entering a known boundary
						bInsideBoundary = true;
					}
					else if (bIsFilledPixel)
					{
						// Create the output chain we'll build from the boundary image
						//TArray<FIntPoint>& Contour = *new (OutPoints) TArray<FIntPoint>();
						TArray<FIntPoint> Contour;

						// Moving into an undiscovered boundary
						BoundaryImage.SetPixel(X, Y, 1);
						new (Contour) FIntPoint(X, Y);

						// Current pixel
						int32 NeighborPhase = 0;
						int32 PX = X;
						int32 PY = Y;

						int32 EnteredStartingSquareCount = 0;
						int32 SinglePixelCounter = 0;

						for (;;)
						{
							// Test pixel (clockwise from the current pixel)
							const int32 CX = PX + NeighborX[NeighborPhase];
							const int32 CY = PY + NeighborY[NeighborPhase];
							const bool bTestPixelInsideBounds = (CX >= LeftBound && CX <= RightBound && CY >= TopBound && CY <= BottomBound);
							const bool bTestPixelPasses = bTestPixelInsideBounds && SourceBitmap.GetPixel(CX, CY) != 0;

							//UE_LOGF(LogPaper2D, Log, "Outer P(%d,%d), C(%d,%d) Ph%d %ls",
							//	PX, PY, CX, CY, NeighborPhase, bTestPixelPasses ? TEXT("[BORDER]") : TEXT("[]"));

							if (bTestPixelPasses)
							{
								// Move to the next pixel

								// Check to see if we closed the loop
								if ((CX == X) && (CY == Y))
								{
// 									// If we went thru the boundary pixel more than two times, or
// 									// entered it from the same way we started, then we're done
// 									++EnteredStartingSquareCount;
// 									if ((EnteredStartingSquareCount > 2) || (NeighborPhase == 0))
// 									{
									//@TODO: Not good enough, will early out too soon some of the time!
										bInsideBoundary = true;
										break;
									}
// 								}

								BoundaryImage.SetPixel(CX, CY, NeighborPhase+1);
								new (Contour) FIntPoint(CX, CY);

								PX = CX;
								PY = CY;
								NeighborPhase = StateMutation[NeighborPhase];

								SinglePixelCounter = 0;
								//NeighborPhase = (NeighborPhase + 1) % 8;
							}
							else
							{
								NeighborPhase = (NeighborPhase + 1) % 8;

								++SinglePixelCounter;
								if (SinglePixelCounter > 8)
								{
									// Went all the way around the neighborhood; it's an island of a single pixel
									break;
								}
							}
						}
						
						// Trace the contour shape creating polygon edges
						TArray<FIntPoint>& ContourPoly = *new (OutPoints)TArray<FIntPoint>();
						TraceContour(/*out*/ContourPoly, Contour);

						// Remove collinear points from the result
						RemoveCollinearPoints(/*inout*/ ContourPoly);

						if (!FGeomTools2D::IsPolygonWindingCCW(ContourPoly))
						{
							// Remove newly added polygon, we don't support holes just yet
							OutPoints.RemoveAt(OutPoints.Num() - 1);
						}
						else
						{
							for (int ContourPolyIndex = 0; ContourPolyIndex < ContourPoly.Num(); ++ContourPolyIndex)
							{
								// Rescale and recenter contour poly
								FIntPoint RescaledPoint = ScanPos + ContourPoly[ContourPolyIndex] * DownsampleAmount;

								// Make sure rescaled point doesn't exceed the original max bounds
								RescaledPoint.X = FMath::Min(RescaledPoint.X, ScanPos.X + ScanSize.X);
								RescaledPoint.Y = FMath::Min(RescaledPoint.Y, ScanPos.Y + ScanSize.Y);

								ContourPoly[ContourPolyIndex] = RescaledPoint;
							}
						}
					}
				}
			}
		}
	}
}

void UPaperSprite::CreatePolygonFromBoundingBox(FSpriteGeometryCollection& GeomOwner, bool bUseTightBounds)
{
	FVector2D BoxSize;
	FVector2D BoxPosition;

	if (bUseTightBounds)
	{
		FindTextureBoundingBox(GeomOwner.AlphaThreshold, /*out*/ BoxPosition, /*out*/ BoxSize);
	}
	else
	{
		BoxSize = SourceDimension;
		BoxPosition = SourceUV;
	}

	// Recenter the box
	BoxPosition += BoxSize * 0.5f;

	// Put the bounding box into the geometry array
	GeomOwner.Shapes.Empty();
	GeomOwner.AddRectangleShape(BoxPosition, BoxSize);
}

void UPaperSprite::ExtractRectsFromTexture(UTexture2D* Texture, TArray<FIntRect>& OutRects)
{
	FBitmap SpriteTextureBitmap(Texture, 0, 0);
	SpriteTextureBitmap.ExtractRects(/*out*/ OutRects);
}

void UPaperSprite::RebuildData()
{
	RebuildCollisionData();
	RebuildRenderData();
}

void UPaperSprite::InitializeSprite(const FSpriteAssetInitParameters& InitParams, bool bRebuildData /*= true*/)
{
	if (InitParams.bOverridePixelsPerUnrealUnit)
	{
		PixelsPerUnrealUnit = InitParams.PixelsPerUnrealUnit;
	}

	if (InitParams.DefaultMaterialOverride != nullptr)
	{
		DefaultMaterial = InitParams.DefaultMaterialOverride;
	}

	if (InitParams.AlternateMaterialOverride != nullptr)
	{
		AlternateMaterial = InitParams.AlternateMaterialOverride;
	}

	SourceTexture = InitParams.Texture;
	if (SourceTexture != nullptr)
	{
		SourceTextureDimension.Set(SourceTexture->GetImportedSize().X, SourceTexture->GetImportedSize().Y);
	}
	else
	{
		SourceTextureDimension.Set(0, 0);
	}
	AdditionalSourceTextures = InitParams.AdditionalTextures;

	SourceUV = InitParams.Offset;
	SourceDimension = InitParams.Dimension;

	if (bRebuildData)
	{
		RebuildData();
	}

	RefreshBakedData();
	CustomPivotPoint = GetPivotPosition();
}

void UPaperSprite::SetTrim(bool bTrimmed, const FVector2D& OriginInSourceImage, const FVector2D& SourceImageDimension, bool bRebuildData /*= true*/)
{
	this->bTrimmedInSourceImage = bTrimmed;
	this->OriginInSourceImageBeforeTrimming = OriginInSourceImage;
	this->SourceImageDimensionBeforeTrimming = SourceImageDimension;
	if (bRebuildData)
	{
		RebuildData();
	}
}

void UPaperSprite::SetRotated(bool bRotated, bool bRebuildData /*= true*/)
{
	this->bRotatedInSourceImage = bRotated;
	if (bRebuildData)
	{
		RebuildData();
	}
}

void UPaperSprite::SetPivotMode(ESpritePivotMode::Type InPivotMode, FVector2D InCustomTextureSpacePivot, bool bRebuildData /*= true*/)
{
	PivotMode = InPivotMode;
	CustomPivotPoint = InCustomTextureSpacePivot;
	if (bRebuildData)
	{
		RebuildData();
	}
}

#if WITH_EDITOR

UTexture2D* UPaperSprite::GetSourceTexture() const
{
	// Verify the cache is still valid.
	UTexture2D* SourceTexturePtr = SourceTexture.Get();
	if (SourceTextureCacheNeverSerialized && SourceTexturePtr == SourceTextureCacheNeverSerialized)
	{
		return SourceTextureCacheNeverSerialized;
	}

	if (SourceTexturePtr == nullptr)
	{
		// The SourceTexture is editoronly and does not need to be cooked, but we sometimes load it during PostLoad due
		// to testing NeedRescaleSpriteData
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
		SourceTexturePtr = SourceTexture.LoadSynchronous();
	}

	// We need to completely load the texture if we're in post load and this texture is needed immediately,
	// not safe to do in a game with EDL, but safe at editor time.
	if (SourceTexturePtr)
	{
		SourceTexturePtr->ConditionalPreload();
		SourceTexturePtr->ConditionalPostLoad();
	}

	SourceTextureCacheNeverSerialized = SourceTexturePtr;

	return SourceTexturePtr;
}

#endif

FVector2D UPaperSprite::ConvertTextureSpaceToPivotSpace(FVector2D Input) const
{
	const FVector2D Pivot = GetPivotPosition();

	const float X = Input.X - Pivot.X;
	const float Y = -Input.Y + Pivot.Y;

	return bRotatedInSourceImage ? FVector2D(-Y, X) : FVector2D(X, Y);
}

FVector2D UPaperSprite::ConvertPivotSpaceToTextureSpace(FVector2D Input) const
{
	const FVector2D Pivot = GetPivotPosition();

	if (bRotatedInSourceImage)
	{
		Swap(Input.X, Input.Y);
		Input.Y = -Input.Y;
	}

	const float X = Input.X + Pivot.X;
	const float Y = -Input.Y + Pivot.Y;

	return FVector2D(X, Y);
}

FVector UPaperSprite::ConvertTextureSpaceToPivotSpace(FVector Input) const
{
	const FVector2D Pivot = GetPivotPosition();

	const float X = Input.X - Pivot.X;
	const float Z = -Input.Z + Pivot.Y;

	return FVector(X, Input.Y, Z);
}

FVector UPaperSprite::ConvertPivotSpaceToTextureSpace(FVector Input) const
{
	const FVector2D Pivot = GetPivotPosition();

	const float X = Input.X + Pivot.X;
	const float Z = -Input.Z + Pivot.Y;

	return FVector(X, Input.Y, Z);
}

FVector UPaperSprite::ConvertTextureSpaceToWorldSpace(const FVector2D& SourcePoint) const
{
	const float UnitsPerPixel = GetUnrealUnitsPerPixel();

	const FVector2D SourcePointInUU = ConvertTextureSpaceToPivotSpace(SourcePoint) * UnitsPerPixel;
	return (PaperAxisX * SourcePointInUU.X) + (PaperAxisY * SourcePointInUU.Y);
}

FVector2D UPaperSprite::ConvertWorldSpaceToTextureSpace(const FVector& WorldPoint) const
{
	const FVector ProjectionX = WorldPoint.ProjectOnTo(PaperAxisX);
	const FVector ProjectionY = WorldPoint.ProjectOnTo(PaperAxisY);

	const float XValue = FMath::Sign(ProjectionX | PaperAxisX) * ProjectionX.Size() * PixelsPerUnrealUnit;
	const float YValue = FMath::Sign(ProjectionY | PaperAxisY) * ProjectionY.Size() * PixelsPerUnrealUnit;

	return ConvertPivotSpaceToTextureSpace(FVector2D(XValue, YValue));
}

FVector2D UPaperSprite::ConvertWorldSpaceDeltaToTextureSpace(const FVector& WorldSpaceDelta, bool bIgnoreRotation) const
{
	const FVector ProjectionX = WorldSpaceDelta.ProjectOnTo(PaperAxisX);
	const FVector ProjectionY = WorldSpaceDelta.ProjectOnTo(PaperAxisY);

	float XValue = FMath::Sign(ProjectionX | PaperAxisX) * ProjectionX.Size() * PixelsPerUnrealUnit;
	float YValue = FMath::Sign(ProjectionY | PaperAxisY) * ProjectionY.Size() * PixelsPerUnrealUnit;

	// Undo pivot space rotation, ignoring pivot position
	if (bRotatedInSourceImage && !bIgnoreRotation)
	{
		Swap(XValue, YValue);
		XValue = -XValue;
	}

	return FVector2D(XValue, YValue);
}

FTransform UPaperSprite::GetPivotToWorld() const
{
	const FVector Translation(0, 0, 0);
	const FVector Scale3D(GetUnrealUnitsPerPixel());
	return FTransform(FRotator::ZeroRotator, Translation, Scale3D);
}

FVector2D UPaperSprite::GetRawPivotPosition() const
{
	FVector2D TopLeftUV = SourceUV;
	FVector2D Dimension = SourceDimension;
	
	if (bTrimmedInSourceImage)
	{
		TopLeftUV = SourceUV - OriginInSourceImageBeforeTrimming;
		Dimension = SourceImageDimensionBeforeTrimming;
	}

	if (bRotatedInSourceImage)
	{
		switch (PivotMode)
		{
		case ESpritePivotMode::Top_Left:
			return FVector2D(TopLeftUV.X + Dimension.X, TopLeftUV.Y);
		case ESpritePivotMode::Top_Center:
			return FVector2D(TopLeftUV.X + Dimension.X, TopLeftUV.Y + Dimension.Y * 0.5f);
		case ESpritePivotMode::Top_Right:
			return FVector2D(TopLeftUV.X + Dimension.X, TopLeftUV.Y + Dimension.Y);
		case ESpritePivotMode::Center_Left:
			return FVector2D(TopLeftUV.X + Dimension.X * 0.5f, TopLeftUV.Y);
		case ESpritePivotMode::Center_Center:
			return FVector2D(TopLeftUV.X + Dimension.X * 0.5f, TopLeftUV.Y + Dimension.Y * 0.5f);
		case ESpritePivotMode::Center_Right:
			return FVector2D(TopLeftUV.X + Dimension.X * 0.5f, TopLeftUV.Y + Dimension.Y);
		case ESpritePivotMode::Bottom_Left:
			return TopLeftUV;
		case ESpritePivotMode::Bottom_Center:
			return FVector2D(TopLeftUV.X, TopLeftUV.Y + Dimension.Y * 0.5f);
		case ESpritePivotMode::Bottom_Right:
			return FVector2D(TopLeftUV.X, TopLeftUV.Y + Dimension.Y);
		default:
		case ESpritePivotMode::Custom:
			return CustomPivotPoint;
		};
	}
	else
	{
		switch (PivotMode)
		{
		case ESpritePivotMode::Top_Left:
			return TopLeftUV;
		case ESpritePivotMode::Top_Center:
			return FVector2D(TopLeftUV.X + Dimension.X * 0.5f, TopLeftUV.Y);
		case ESpritePivotMode::Top_Right:
			return FVector2D(TopLeftUV.X + Dimension.X, TopLeftUV.Y);
		case ESpritePivotMode::Center_Left:
			return FVector2D(TopLeftUV.X, TopLeftUV.Y + Dimension.Y * 0.5f);
		case ESpritePivotMode::Center_Center:
			return FVector2D(TopLeftUV.X + Dimension.X * 0.5f, TopLeftUV.Y + Dimension.Y * 0.5f);
		case ESpritePivotMode::Center_Right:
			return FVector2D(TopLeftUV.X + Dimension.X, TopLeftUV.Y + Dimension.Y * 0.5f);
		case ESpritePivotMode::Bottom_Left:
			return FVector2D(TopLeftUV.X, TopLeftUV.Y + Dimension.Y);
		case ESpritePivotMode::Bottom_Center:
			return FVector2D(TopLeftUV.X + Dimension.X * 0.5f, TopLeftUV.Y + Dimension.Y);
		case ESpritePivotMode::Bottom_Right:
			return TopLeftUV + Dimension;

		default:
		case ESpritePivotMode::Custom:
			return CustomPivotPoint;
		};
	}
}

FVector2D UPaperSprite::GetPivotPosition() const
{
	FVector2D RawPivot = GetRawPivotPosition();

	if (bSnapPivotToPixelGrid)
	{
		RawPivot.X = FMath::RoundToFloat(RawPivot.X);
		RawPivot.Y = FMath::RoundToFloat(RawPivot.Y);
	}

	return RawPivot;
}

void UPaperSprite::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (AtlasGroup != nullptr)
	{
		Context.AddTag(FAssetRegistryTag(TEXT("AtlasGroupGUID"), AtlasGroup->AtlasGUID.ToString(EGuidFormats::Digits), FAssetRegistryTag::TT_Hidden));
	}
}

#endif

FSlateAtlasData UPaperSprite::GetSlateAtlasData() const
{
	if (BakedSourceTexture)
	{
		const FVector2D ImportedSize = FVector2D(BakedSourceTexture->GetImportedSize());

		const FVector2D StartUV = BakedSourceUV / ImportedSize;
		const FVector2D SizeUV = BakedSourceDimension / ImportedSize;

		return FSlateAtlasData(BakedSourceTexture, StartUV, SizeUV);
	}
#if WITH_EDITOR
	else if (UTexture2D* SourceTexturePtr = GetSourceTexture())
	{
		const FVector2D ImportedSize = FVector2D(SourceTexturePtr->GetImportedSize());

		const FVector2D StartUV = SourceUV / ImportedSize;
		const FVector2D SizeUV = SourceDimension / ImportedSize;

		return FSlateAtlasData(SourceTexturePtr, StartUV, SizeUV);
	}
#endif
	else
	{
		return FSlateAtlasData(nullptr, FVector2D::ZeroVector, FVector2D::ZeroVector);
	}
}

bool UPaperSprite::GetPhysicsTriMeshData(FTriMeshCollisionData* OutCollisionData, bool InUseAllTriData)
{
	//@TODO: Probably want to support this
	return false;
}

bool UPaperSprite::ContainsPhysicsTriMeshData(bool InUseAllTriData) const
{
	//@TODO: Probably want to support this
	return false;
}

#if WITH_EDITOR
FName UPaperSprite::GetSourceTextureMemberName()
{
	return GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceTexture);
}
#endif // WITH_EDITOR

FBoxSphereBounds UPaperSprite::GetRenderBounds() const
{
	FBox BoundingBox(ForceInit);
	
	for (int32 VertexIndex = 0; VertexIndex < BakedRenderData.Num(); ++VertexIndex)
	{
		const FVector4& VertXYUV = BakedRenderData[VertexIndex];
		const FVector Vert((PaperAxisX * VertXYUV.X) + (PaperAxisY * VertXYUV.Y));
		BoundingBox += Vert;
	}
	
	// Make the whole thing a single unit 'deep'
	const FVector HalfThicknessVector = 0.5f * PaperAxisZ;
	BoundingBox += -HalfThicknessVector;
	BoundingBox += HalfThicknessVector;

	return FBoxSphereBounds(BoundingBox);
}

FPaperSpriteSocket* UPaperSprite::FindSocket(FName SocketName)
{
	for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); ++SocketIndex)
	{
		FPaperSpriteSocket& Socket = Sockets[SocketIndex];
		if (Socket.SocketName == SocketName)
		{
			return &Socket; 
		}
	}

	return nullptr;
}

void UPaperSprite::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); ++SocketIndex)
	{
		const FPaperSpriteSocket& Socket = Sockets[SocketIndex];
		new (OutSockets) FComponentSocketDescription(Socket.SocketName, EComponentSocketType::Socket);
	}
}

#if WITH_EDITOR
void UPaperSprite::ValidateSocketNames()
{
	TSet<FName> SocketNames;
	struct Local
	{
		static FName GetUniqueName(const TSet<FName>& InSocketNames, FName Name)
		{
			int Counter = Name.GetNumber();
			FName TestName;
			do 
			{
				TestName = Name;
				TestName.SetNumber(++Counter);
			} while (InSocketNames.Contains(TestName));

			return TestName;
		}
	};

	bool bHasChanged = false;
	for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); ++SocketIndex)
	{
		FPaperSpriteSocket& Socket = Sockets[SocketIndex];
		if (Socket.SocketName.IsNone())
		{
			Socket.SocketName = Local::GetUniqueName(SocketNames, FName(TEXT("Socket")));
			bHasChanged = true;
		}
		else if (SocketNames.Contains(Socket.SocketName))
		{
			Socket.SocketName = Local::GetUniqueName(SocketNames, Socket.SocketName);
			bHasChanged = true;
		}

		// Add the corrected name
		SocketNames.Add(Socket.SocketName);
	}

	if (bHasChanged)
	{
		PostEditChange();
	}
}
#endif

#if WITH_EDITOR
void UPaperSprite::RemoveSocket(FName SocketNameToDelete)
{
	Sockets.RemoveAll([=](const FPaperSpriteSocket& Socket){ return Socket.SocketName == SocketNameToDelete; });
}
#endif

void UPaperSprite::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);

	if (SpriteCollisionDomain == ESpriteCollisionMode::Use2DPhysics)
	{
		UE_LOGF(LogPaper2D, Warning, "PaperSprite '%ls' was using 2D physics which has been removed, it has been switched to 3D physics.", *GetPathName());
		SpriteCollisionDomain = ESpriteCollisionMode::Use3DPhysics;
	}
}

void UPaperSprite::PostLoad()
{
	Super::PostLoad();
	
	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);

#if !WITH_EDITORONLY_DATA
	if (PaperVer < FPaperCustomVersion::LatestVersion)
	{
		UE_LOGF(LogPaper2D, Warning, "Stale UPaperSprite asset '%ls' with version %d detected in a cooked build (latest version is %d).  Please perform a full recook.", *GetPathName(), PaperVer, (int32)FPaperCustomVersion::LatestVersion);
	}
#else
	if (UTexture2D* EffectiveTexture = GetBakedTexture())
	{
		EffectiveTexture->ConditionalPostLoad();
	}

	bool bRebuildCollision = false;
	bool bRebuildRenderData = false;

	if (PaperVer < FPaperCustomVersion::AddTransactionalToClasses)
	{
		SetFlags(RF_Transactional);
	}

	if (PaperVer < FPaperCustomVersion::RefactorPolygonStorageToSupportShapes)
	{
		UpdateGeometryToBeBoxPositionRelative(CollisionGeometry);
		UpdateGeometryToBeBoxPositionRelative(RenderGeometry);
	}

	if (PaperVer < FPaperCustomVersion::AddPivotSnapToPixelGrid)
	{
		bSnapPivotToPixelGrid = false;
	}

	if (PaperVer < FPaperCustomVersion::FixTangentGenerationForFrontFace)
	{
		bRebuildRenderData = true;
	}

	if (PaperVer < FPaperCustomVersion::AddPixelsPerUnrealUnit)
	{
		PixelsPerUnrealUnit = 1.0f;
		bRebuildCollision = true;
		bRebuildRenderData = true;
	}
	else if (PaperVer < FPaperCustomVersion::FixIncorrectCollisionOnSourceRotatedSprites)
	{
		bRebuildCollision = true;
	}

	if ((PaperVer < FPaperCustomVersion::AddDefaultCollisionProfileInSpriteAsset) && (BodySetup != nullptr))
	{
		BodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
	}

	if ((PaperVer >= FPaperCustomVersion::AddSourceTextureSize) && NeedRescaleSpriteData())
	{
		RescaleSpriteData(GetSourceTexture());
		bRebuildCollision = true;
		bRebuildRenderData = true;
	}

	if (bRebuildCollision)
	{
		RebuildCollisionData();
	}

	if (bRebuildRenderData)
	{
		RebuildRenderData();
	}

	RefreshBakedData();
#endif
}

#if WITH_EDITOR

void UPaperSprite::RefreshBakedData()
{
	/** If we don't have an atlas group make sure we assign the source texture to our baked texture. */
	if (AtlasGroup == nullptr)
	{
		BakedSourceTexture = GetSourceTexture();
		BakedSourceUV = SourceUV;
		BakedSourceDimension = SourceDimension;
	}
}
#endif

UTexture2D* UPaperSprite::GetBakedTexture() const
{
#if WITH_EDITOR
	return (BakedSourceTexture != nullptr) ? ToRawPtr(BakedSourceTexture) : GetSourceTexture();
#else
	return BakedSourceTexture;
#endif
}

void UPaperSprite::GetBakedAdditionalSourceTextures(FAdditionalSpriteTextureArray& OutTextureList) const
{
	OutTextureList = ToRawPtrTArrayUnsafe(AdditionalSourceTextures);
}

UMaterialInterface* UPaperSprite::GetMaterial(int32 MaterialIndex) const
{
	if (MaterialIndex == 0)
	{
		return GetDefaultMaterial();
	}
	else if (MaterialIndex == 1)
	{
		return GetAlternateMaterial();
	}
	else
	{
		return nullptr;
	}
}

int32 UPaperSprite::GetNumMaterials() const
{
	return (AlternateMaterialSplitIndex != INDEX_NONE) ? 2 : 1;
}

//////////////////////////////////////////////////////////////////////////
// FSpriteGeometryCollection

void FSpriteGeometryCollection::AddRectangleShape(FVector2D Position, FVector2D Size)
{
	const FVector2D HalfSize = Size * 0.5f;
	
	FSpriteGeometryShape& NewShape = *new (Shapes) FSpriteGeometryShape();
	new (NewShape.Vertices) FVector2D(-HalfSize.X, -HalfSize.Y);
	new (NewShape.Vertices) FVector2D(+HalfSize.X, -HalfSize.Y);
	new (NewShape.Vertices) FVector2D(+HalfSize.X, +HalfSize.Y);
	new (NewShape.Vertices) FVector2D(-HalfSize.X, +HalfSize.Y);
	NewShape.ShapeType = ESpriteShapeType::Box;
	NewShape.BoxSize = Size;
	NewShape.BoxPosition = Position;
}

void FSpriteGeometryCollection::AddCircleShape(FVector2D Position, FVector2D Size)
{
	FSpriteGeometryShape& NewShape = *new (Shapes) FSpriteGeometryShape();
	NewShape.ShapeType = ESpriteShapeType::Circle;
	NewShape.BoxSize = Size;
	NewShape.BoxPosition = Position;
}

void FSpriteGeometryCollection::Reset()
{
	Shapes.Empty();
	GeometryType = ESpritePolygonMode::TightBoundingBox;
}

void FSpriteGeometryCollection::Triangulate(TArray<FVector2D>& Target, bool bIncludeBoxes) const
{
	Target.Empty();

	TArray<FVector2D> AllGeneratedTriangles;

	// AOS -> Validate -> SOA
	TArray<bool> PolygonsNegativeWinding; // do these polygons have negative winding?
	TArray<TArray<FVector2D> > ValidPolygonTriangles;
	PolygonsNegativeWinding.Empty(Shapes.Num());
	ValidPolygonTriangles.Empty(Shapes.Num());
	bool bSourcePolygonHasHoles = false;

	// Correct polygon winding for additive and subtractive polygons
	// Invalid polygons (< 3 verts) removed from this list
	for (int32 PolygonIndex = 0; PolygonIndex < Shapes.Num(); ++PolygonIndex)
	{
		const FSpriteGeometryShape& SourcePolygon = Shapes[PolygonIndex];

		if ((SourcePolygon.ShapeType == ESpriteShapeType::Polygon) || (bIncludeBoxes && (SourcePolygon.ShapeType == ESpriteShapeType::Box)))
		{
			if (SourcePolygon.Vertices.Num() >= 3)
			{
				TArray<FVector2D> TextureSpaceVertices;
				SourcePolygon.GetTextureSpaceVertices(/*out*/ TextureSpaceVertices);

				TArray<FVector2D>& FixedVertices = *new (ValidPolygonTriangles) TArray<FVector2D>();
				FGeomTools2D::CorrectPolygonWinding(/*out*/ FixedVertices, TextureSpaceVertices, SourcePolygon.bNegativeWinding);
				PolygonsNegativeWinding.Add(SourcePolygon.bNegativeWinding);
			}

			if (SourcePolygon.bNegativeWinding)
			{
				bSourcePolygonHasHoles = true;
			}
		}
	}

	// Check if polygons overlap, or have inconsistent winding, or edges overlap
	if (!FGeomTools2D::ArePolygonsValid(ValidPolygonTriangles))
	{
		return;
	}

	// Merge each additive and associated subtractive polygons to form a list of polygons in CCW winding
	ValidPolygonTriangles = FGeomTools2D::ReducePolygons(ValidPolygonTriangles, PolygonsNegativeWinding);

	// Triangulate the polygons
	for (int32 PolygonIndex = 0; PolygonIndex < ValidPolygonTriangles.Num(); ++PolygonIndex)
	{
		TArray<FVector2D> Generated2DTriangles;
		if (FGeomTools2D::TriangulatePoly(Generated2DTriangles, ValidPolygonTriangles[PolygonIndex], bAvoidVertexMerging))
		{
			AllGeneratedTriangles.Append(Generated2DTriangles);
		}
	}

	// This doesn't work when polys have holes as edges will likely form a loop around the poly
	if (!bSourcePolygonHasHoles && !bAvoidVertexMerging && ((ValidPolygonTriangles.Num() > 1) && (AllGeneratedTriangles.Num() > 1)))
	{
		TArray<FVector2D> TrianglesCopy = AllGeneratedTriangles;
		AllGeneratedTriangles.Empty();
		FGeomTools2D::RemoveRedundantTriangles(/*out*/ AllGeneratedTriangles, TrianglesCopy);
	}

	Target.Append(AllGeneratedTriangles);
}

bool AreVectorsParallel(const FVector2D& Vector1, const FVector2D& Vector2, float Threshold = KINDA_SMALL_NUMBER)
{
	const float DotProduct = FVector2D::DotProduct(Vector1, Vector2);
	const float LengthProduct = Vector1.Size() * Vector2.Size();

	return FMath::IsNearlyEqual(FMath::Abs(DotProduct / LengthProduct), 1.0f, Threshold);
}

bool AreVectorsPerpendicular(const FVector2D& Vector1, const FVector2D& Vector2, float Threshold = KINDA_SMALL_NUMBER)
{
	const float DotProduct = FVector2D::DotProduct(Vector1, Vector2);
	return FMath::IsNearlyEqual(DotProduct, 0.0f, Threshold);
}

bool FSpriteGeometryCollection::ConditionGeometry()
{
	bool bModifiedGeometry = false;

	for (FSpriteGeometryShape& Shape : Shapes)
	{
		if ((Shape.ShapeType == ESpriteShapeType::Polygon) && (Shape.Vertices.Num() == 4))
		{
			const FVector2D& A = Shape.Vertices[0];
			const FVector2D& B = Shape.Vertices[1];
			const FVector2D& C = Shape.Vertices[2];
			const FVector2D& D = Shape.Vertices[3];

			const FVector2D AB = B - A;
			const FVector2D BC = C - B;
			const FVector2D CD = D - C;
			const FVector2D DA = A - D;

			if (AreVectorsPerpendicular(AB, BC) &&
				AreVectorsPerpendicular(CD, DA) &&
				AreVectorsParallel(AB, CD) &&
				AreVectorsParallel(BC, DA))
			{
				// Checking in local space, so we still want the rotation to be 0 here
				const bool bMeetsRotationConstraint = FMath::IsNearlyEqual(AB.Y, 0.0f);
				if (bMeetsRotationConstraint)
				{
					const FVector2D NewPivotTextureSpace = Shape.GetPolygonCentroid();
					Shape.SetNewPivot(NewPivotTextureSpace);
					Shape.BoxSize = FVector2D(AB.Size(), DA.Size());
					Shape.ShapeType = ESpriteShapeType::Box;
					bModifiedGeometry = true;
				}
			}
		}
	}

	return bModifiedGeometry;
}

//////////////////////////////////////////////////////////////////////////
// FSpriteGeometryCollisionBuilderBase

FSpriteGeometryCollisionBuilderBase::FSpriteGeometryCollisionBuilderBase(UBodySetup* InBodySetup)
	: MyBodySetup(InBodySetup)
	, UnrealUnitsPerPixel(1.0f)
	, CollisionThickness(64.0f)
	, ZOffsetAmount(0.0f)
	, CollisionDomain(ESpriteCollisionMode::Use3DPhysics)
{
	check(MyBodySetup);
}

void FSpriteGeometryCollisionBuilderBase::ProcessGeometry(const FSpriteGeometryCollection& InGeometry)
{
	// Add geometry to the body setup
	AddBoxCollisionShapesToBodySetup(InGeometry);
	AddPolygonCollisionShapesToBodySetup(InGeometry);
	AddCircleCollisionShapesToBodySetup(InGeometry);
}

void FSpriteGeometryCollisionBuilderBase::Finalize()
{
	// Rebuild the body setup
	MyBodySetup->InvalidatePhysicsData();
	MyBodySetup->CreatePhysicsMeshes();
}

void FSpriteGeometryCollisionBuilderBase::AddBoxCollisionShapesToBodySetup(const FSpriteGeometryCollection& InGeometry)
{
	// Bake all of the boxes to the body setup
	for (const FSpriteGeometryShape& Shape : InGeometry.Shapes)
	{
		if (Shape.ShapeType == ESpriteShapeType::Box)
		{
			// Determine the box size and center in pivot space
			const FVector2D& BoxSizeInTextureSpace = Shape.BoxSize;
			const FVector2D CenterInTextureSpace = Shape.BoxPosition;
			const FVector2D CenterInPivotSpace = ConvertTextureSpaceToPivotSpace(CenterInTextureSpace);

			// Convert from pixels to uu
			const FVector2D BoxSizeInPivotSpace = ConvertTextureSpaceToPivotSpaceNoTranslation(BoxSizeInTextureSpace);
			const FVector2D BoxSize2D = BoxSizeInPivotSpace * UnrealUnitsPerPixel;
			const FVector2D CenterInScaledSpace = CenterInPivotSpace * UnrealUnitsPerPixel;

			// Create a new box primitive
			switch (CollisionDomain)
			{
				case ESpriteCollisionMode::Use3DPhysics:
					{
						const FVector BoxPos3D = (PaperAxisX * CenterInScaledSpace.X) + (PaperAxisY * CenterInScaledSpace.Y) + (PaperAxisZ * ZOffsetAmount);
						const FVector BoxSize3D = (PaperAxisX * BoxSize2D.X) + (PaperAxisY * BoxSize2D.Y) + (PaperAxisZ * CollisionThickness);

						// Create a new box primitive
						FKBoxElem& Box = *new (MyBodySetup->AggGeom.BoxElems) FKBoxElem(FMath::Abs(BoxSize3D.X), FMath::Abs(BoxSize3D.Y), FMath::Abs(BoxSize3D.Z));
						Box.Center = BoxPos3D;
						Box.Rotation = FRotator(Shape.Rotation, 0.0f, 0.0f);
					}
					break;
				default:
					check(false);
					break;
			}
		}
	}
}

void FSpriteGeometryCollisionBuilderBase::AddPolygonCollisionShapesToBodySetup(const FSpriteGeometryCollection& InGeometry)
{
	// Rebuild the runtime geometry for polygons
	TArray<FVector2D> CollisionData;
	InGeometry.Triangulate(/*out*/ CollisionData, /*bIncludeBoxes=*/ false);

	// Adjust the collision data to be relative to the pivot and scaled from pixels to uu
	for (FVector2D& Point : CollisionData)
	{
		Point = ConvertTextureSpaceToPivotSpace(Point) * UnrealUnitsPerPixel;
	}

	//@TODO: Use this instead: DecomposeMeshToHulls
	//@TODO: Merge triangles that are convex together!

	// Bake it to the runtime structure
	switch (CollisionDomain)
	{
	case ESpriteCollisionMode::Use3DPhysics:
		{
			UBodySetup* BodySetup3D = MyBodySetup;

			const FVector HalfThicknessVector = PaperAxisZ * 0.5f * CollisionThickness;

			int32 RunningIndex = 0;
			for (int32 TriIndex = 0; TriIndex < CollisionData.Num() / 3; ++TriIndex)
			{
				FKConvexElem& ConvexTri = *new (BodySetup3D->AggGeom.ConvexElems) FKConvexElem();
				ConvexTri.VertexData.Empty(6);
				for (int32 Index = 0; Index < 3; ++Index)
				{
					const FVector2D& Pos2D = CollisionData[RunningIndex++];
					
					const FVector Pos3D = (PaperAxisX * Pos2D.X) + (PaperAxisY * Pos2D.Y) + (PaperAxisZ * ZOffsetAmount);

					new (ConvexTri.VertexData) FVector(Pos3D - HalfThicknessVector);
					new (ConvexTri.VertexData) FVector(Pos3D + HalfThicknessVector);
				}
				ConvexTri.UpdateElemBox();
			}
		}
		break;
	default:
		check(false);
		break;
	}
}

void FSpriteGeometryCollisionBuilderBase::AddCircleCollisionShapesToBodySetup(const FSpriteGeometryCollection& InGeometry)
{
	// Bake all of the boxes to the body setup
	for (const FSpriteGeometryShape& Shape : InGeometry.Shapes)
	{
		if (Shape.ShapeType == ESpriteShapeType::Circle)
		{
			// Determine the box size and center in pivot space
			const FVector2D& CircleSizeInTextureSpace = Shape.BoxSize;
			const FVector2D& CenterInTextureSpace = Shape.BoxPosition;
			const FVector2D CenterInPivotSpace = ConvertTextureSpaceToPivotSpace(CenterInTextureSpace);

			// Convert from pixels to uu
			const FVector2D CircleSizeInPivotSpace = ConvertTextureSpaceToPivotSpaceNoTranslation(CircleSizeInTextureSpace);
			const FVector2D CircleSize2D = CircleSizeInPivotSpace * UnrealUnitsPerPixel;
			const FVector2D CenterInScaledSpace = CenterInPivotSpace * UnrealUnitsPerPixel;

			//@TODO: Neither Box2D nor PhysX support ellipses, currently forcing to be circular, but should we instead convert to an n-gon?
			const float AverageDiameter = (FMath::Abs(CircleSize2D.X) + FMath::Abs(CircleSize2D.Y)) * 0.5f;
			const float AverageRadius = AverageDiameter * 0.5f;

			// Create a new circle/sphere primitive
			switch (CollisionDomain)
			{
				case ESpriteCollisionMode::Use3DPhysics:
					{
						// Create a new box primitive
						FKSphereElem& Sphere = *new (MyBodySetup->AggGeom.SphereElems) FKSphereElem(AverageRadius);
						Sphere.Center = (PaperAxisX * CenterInScaledSpace.X) + (PaperAxisY * CenterInScaledSpace.Y) + (PaperAxisZ * ZOffsetAmount);
					}
					break;
				default:
					check(false);
					break;
			}
		}
	}
}

FVector2D FSpriteGeometryCollisionBuilderBase::ConvertTextureSpaceToPivotSpace(const FVector2D& Input) const
{
	return Input;
}

FVector2D FSpriteGeometryCollisionBuilderBase::ConvertTextureSpaceToPivotSpaceNoTranslation(const FVector2D& Input) const
{
	return Input;
}

