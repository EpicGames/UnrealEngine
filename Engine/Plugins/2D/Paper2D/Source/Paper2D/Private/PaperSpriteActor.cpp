// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteActor.h"
#include "Engine/CollisionProfile.h"
#include "PaperSpriteComponent.h"
#include "PaperSprite.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteActor)

//////////////////////////////////////////////////////////////////////////
// APaperSpriteActor

APaperSpriteActor::APaperSpriteActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RenderComponent = CreateDefaultSubobject<UPaperSpriteComponent>(TEXT("RenderComponent"));
	RenderComponent->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	RenderComponent->Mobility = EComponentMobility::Static;

	RootComponent = RenderComponent;
}

#if WITH_EDITOR
bool APaperSpriteActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (UPaperSprite* SourceSprite = RenderComponent->GetSprite())
	{
		Objects.Add(SourceSprite);
	}
	return true;
}
#endif

