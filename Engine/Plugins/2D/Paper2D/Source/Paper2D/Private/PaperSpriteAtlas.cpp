// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteAtlas.h"
#include "Engine/Texture.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteAtlas)

//////////////////////////////////////////////////////////////////////////
// UPaperSpriteAtlas

UPaperSpriteAtlas::UPaperSpriteAtlas(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, MaxWidth(2048)
	, MaxHeight(2048)
	, MipCount(1)
	, PaddingType(EPaperSpriteAtlasPadding::DilateBorder)
	, Padding(1)
	, CompressionSettings(TextureCompressionSettings::TC_Default)
	, Filter(TextureFilter::TF_Bilinear)
	, bRebuildAtlas(false)
#endif
{
}

void UPaperSpriteAtlas::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

#if WITH_EDITORONLY_DATA
	Context.AddTag(FAssetRegistryTag(GET_MEMBER_NAME_CHECKED(UPaperSpriteAtlas, AtlasDescription), AtlasDescription, FAssetRegistryTag::TT_Hidden));
#endif
}

#if WITH_EDITORONLY_DATA

void UPaperSpriteAtlas::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	FPlatformMisc::CreateGuid(AtlasGUID);
}

void UPaperSpriteAtlas::PostInitProperties()
{
	Super::PostInitProperties();
	FPlatformMisc::CreateGuid(AtlasGUID);
}

#endif

