// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteBlueprintLibrary.h"
#include "PaperSprite.h"
#include "Brushes/SlateNoResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteBlueprintLibrary)

//////////////////////////////////////////////////////////////////////////

FSlateBrush UPaperSpriteBlueprintLibrary::MakeBrushFromSprite(UPaperSprite* Sprite, int32 Width, int32 Height)
{
	if ( Sprite )
	{
		const FSlateAtlasData SpriteAtlasData = Sprite->GetSlateAtlasData();
		const FVector2D SpriteSize = SpriteAtlasData.GetSourceDimensions();

		FSlateBrush Brush;
		Brush.SetResourceObject(Sprite);
		Width = ( Width > 0 ) ? Width : SpriteSize.X;
		Height = ( Height > 0 ) ? Height : SpriteSize.Y;
		Brush.ImageSize = FVector2D(Width, Height);
		return Brush;
	}

	return FSlateNoResource();
}
