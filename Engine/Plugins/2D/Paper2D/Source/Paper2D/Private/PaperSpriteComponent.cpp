// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteComponent.h"
#include "RenderingThread.h"
#include "Engine/CollisionProfile.h"
#include "PaperSpriteSceneProxy.h"
#include "PaperCustomVersion.h"
#include "PhysicsEngine/BodySetup.h"
#include "ContentStreaming.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteComponent)

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// UPaperSpriteComponent

UPaperSpriteComponent::UPaperSpriteComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);

	MaterialOverride_DEPRECATED = nullptr;

	SpriteColor = FLinearColor::White;

	CastShadow = false;
	bUseAsOccluder = false;
}

#if WITH_EDITOR
void UPaperSpriteComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FBodyInstanceEditorHelpers::EnsureConsistentMobilitySimulationSettingsOnPostEditChange(this, PropertyChangedEvent);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UPaperSpriteComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);
}

void UPaperSpriteComponent::PostLoad()
{
	Super::PostLoad();

	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);

	if (PaperVer < FPaperCustomVersion::ConvertPaperSpriteComponentToBeMeshComponent)
	{
		if (MaterialOverride_DEPRECATED != nullptr)
		{
			SetMaterial(0, MaterialOverride_DEPRECATED);
		}
	}

	if (PaperVer < FPaperCustomVersion::FixVertexColorSpace)
	{
		const FColor SRGBColor = SpriteColor.ToFColor(/*bSRGB=*/ true);
		SpriteColor = SRGBColor.ReinterpretAsLinear();
	}
}

FPrimitiveSceneProxy* UPaperSpriteComponent::CreateSceneProxy()
{
	FPaperSpriteSceneProxy* NewProxy = new FPaperSpriteSceneProxy(this);
	if (SourceSprite != nullptr)
	{
		FSpriteDrawCallRecord DrawCall;
		DrawCall.BuildFromSprite(SourceSprite);
		DrawCall.Color = SpriteColor.ToFColor(/*bSRGB=*/ false);

		FPaperSpriteSceneProxy* InSceneProxy = (FPaperSpriteSceneProxy*)NewProxy;
		ENQUEUE_RENDER_COMMAND(FCreatePaperSpriteProxy_SetSprite)(
			[InSceneProxy, DrawCall, SplitIndex = SourceSprite->AlternateMaterialSplitIndex](FRHICommandListImmediate& RHICmdList)
		{
			InSceneProxy->SetSprite_RenderThread(DrawCall, SplitIndex);
		});
	}
	return NewProxy;
}

FBoxSphereBounds UPaperSpriteComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (SourceSprite != nullptr)
	{
		// Graphics bounds.
		FBoxSphereBounds NewBounds = SourceSprite->GetRenderBounds().TransformBy(LocalToWorld);

		// Add bounds of collision geometry (if present).
		if (UBodySetup* BodySetup = SourceSprite->BodySetup)
		{
			const FBox AggGeomBox = BodySetup->AggGeom.CalcAABB(LocalToWorld);
			if (AggGeomBox.IsValid)
			{
				NewBounds = Union(NewBounds,FBoxSphereBounds(AggGeomBox));
			}
		}

		// Apply bounds scale
		NewBounds.BoxExtent *= BoundsScale;
		NewBounds.SphereRadius *= BoundsScale;

		return NewBounds;
	}
	else
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}
}

void UPaperSpriteComponent::SendRenderDynamicData_Concurrent()
{
	if (SceneProxy != nullptr)
	{
		FSpriteDrawCallRecord DrawCall;
		DrawCall.BuildFromSprite(SourceSprite);
		DrawCall.Color = SpriteColor.ToFColor(/*bSRGB=*/ false);
		const int32 SplitIndex = (SourceSprite != nullptr) ? SourceSprite->AlternateMaterialSplitIndex : INDEX_NONE;

		FPaperSpriteSceneProxy* InSceneProxy = (FPaperSpriteSceneProxy*)SceneProxy;
		ENQUEUE_RENDER_COMMAND(FSendPaperSpriteComponentDynamicData)(
			[InSceneProxy, DrawCall, SplitIndex](FRHICommandListImmediate& RHICmdList)
			{
				InSceneProxy->SetSprite_RenderThread(DrawCall, SplitIndex);
			});
	}
}

bool UPaperSpriteComponent::HasAnySockets() const
{
	if (SourceSprite != nullptr)
	{
		return SourceSprite->HasAnySockets();
	}

	return false;
}

bool UPaperSpriteComponent::DoesSocketExist(FName InSocketName) const
{
	if (SourceSprite != nullptr)
	{
		if (FPaperSpriteSocket* Socket = SourceSprite->FindSocket(InSocketName))
		{
			return true;
		}
	}

	return false;
}

FTransform UPaperSpriteComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	if (SourceSprite != nullptr)
	{
		if (FPaperSpriteSocket* Socket = SourceSprite->FindSocket(InSocketName))
		{
			FTransform SocketLocalTransform = Socket->LocalTransform;
			SocketLocalTransform.ScaleTranslation(SourceSprite->GetUnrealUnitsPerPixel());

			switch (TransformSpace)
			{
				case RTS_World:
					return SocketLocalTransform * GetComponentTransform();

				case RTS_Actor:
					if (const AActor* Actor = GetOwner())
					{
						const FTransform SocketTransform = SocketLocalTransform * GetComponentTransform();
						return SocketTransform.GetRelativeTransform(Actor->GetTransform());
					}
					break;

				case RTS_Component:
				case RTS_ParentBoneSpace:
					return SocketLocalTransform;

				default:
					check(false);
			}
		}
	}

	return Super::GetSocketTransform(InSocketName, TransformSpace);
}

void UPaperSpriteComponent::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	if (SourceSprite != nullptr)
	{
		return SourceSprite->QuerySupportedSockets(OutSockets);
	}
}

UBodySetup* UPaperSpriteComponent::GetBodySetup()
{
	return (SourceSprite != nullptr) ? SourceSprite->BodySetup : nullptr;
}

bool UPaperSpriteComponent::SetSprite(class UPaperSprite* NewSprite)
{
	if (NewSprite != SourceSprite)
	{
		// Don't allow changing the sprite if we are "static".
		AActor* ComponentOwner = GetOwner();
		if ((ComponentOwner == nullptr) || AreDynamicDataChangesAllowed())
		{
			SourceSprite = NewSprite;

			// Need to send this to render thread at some point
			MarkRenderStateDirty();

			// Update physics representation right away
			RecreatePhysicsState();

			// Update this component streaming data.
			IStreamingManager::Get().NotifyPrimitiveUpdated(this);

			// Since we have new mesh, we need to update bounds
			UpdateBounds();

			return true;
		}
	}

	return false;
}

void UPaperSpriteComponent::GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel)
{
	// Get the texture referenced by the sprite
	if (SourceSprite != nullptr)
	{
		if (UTexture* BakedTexture = SourceSprite->GetBakedTexture())
		{
			OutTextures.AddUnique(BakedTexture);
		}

		FAdditionalSpriteTextureArray AdditionalTextureList;
		SourceSprite->GetBakedAdditionalSourceTextures(/*out*/ AdditionalTextureList);
		for (UTexture* AdditionalTexture : AdditionalTextureList)
		{
			if (AdditionalTexture != nullptr)
			{
				OutTextures.AddUnique(AdditionalTexture);
			}
		}
	}

	// Get any textures referenced by our materials
	Super::GetUsedTextures(OutTextures, QualityLevel);
}

UMaterialInterface* UPaperSpriteComponent::GetMaterial(int32 MaterialIndex) const
{
	if (OverrideMaterials.IsValidIndex(MaterialIndex) && (OverrideMaterials[MaterialIndex] != nullptr))
	{
		return OverrideMaterials[MaterialIndex];
	}
	else if (SourceSprite != nullptr)
	{
		return SourceSprite->GetMaterial(MaterialIndex);
	}

	return nullptr;
}

void UPaperSpriteComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	return Super::GetUsedMaterials(OutMaterials, bGetDebugMaterials);
}

void UPaperSpriteComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	//@TODO: PAPER2D: Need to support this for proper texture streaming
	return Super::GetStreamingRenderAssetInfo(LevelContext, OutStreamingRenderAssets);
}

int32 UPaperSpriteComponent::GetNumMaterials() const
{
	if (SourceSprite != nullptr)
	{
		return FMath::Max<int32>(OverrideMaterials.Num(), SourceSprite->GetNumMaterials());
	}
	else
	{
		return FMath::Max<int32>(OverrideMaterials.Num(), 1);
	}
}

UPaperSprite* UPaperSpriteComponent::GetSprite()
{
	return SourceSprite;
}

void UPaperSpriteComponent::SetSpriteColor(FLinearColor NewColor)
{
	// Can't set color on a static component
	if (AreDynamicDataChangesAllowed() && (SpriteColor != NewColor))
	{
		SpriteColor = NewColor;

		//@TODO: Should we send immediately?
		MarkRenderDynamicDataDirty();
	}
}

FLinearColor UPaperSpriteComponent::GetWireframeColor() const
{
	if (Mobility == EComponentMobility::Static)
	{
		return FColor(0, 255, 255, 255);
	}
	else
	{
		if (BodyInstance.bSimulatePhysics)
		{
			return FColor(0, 255, 128, 255);
		}
		else
		{
			return FColor(255, 0, 255, 255);
		}
	}
}

const UObject* UPaperSpriteComponent::AdditionalStatObject() const
{
	return SourceSprite;
}

#if WITH_EDITOR
void UPaperSpriteComponent::CheckForErrors()
{
	Super::CheckForErrors();

	AActor* Owner = GetOwner();

	for (int32 MaterialIndex = 0; MaterialIndex < GetNumMaterials(); ++MaterialIndex)
	{
		if (UMaterialInterface* Material = GetMaterial(MaterialIndex))
		{
			if (!Material->IsTwoSided())
			{
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(Owner))
					->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_PaperSpriteMaterialNotTwoSided", "The material applied to the sprite component is not marked as two-sided, which may cause lighting artifacts.")))
					->AddToken(FUObjectToken::Create(Material))
					->AddToken(FMapErrorToken::Create(FName(TEXT("PaperSpriteMaterialNotTwoSided"))));
			}
		}
	}

	// Make sure any non uniform scaled sprites have appropriate collision
	if (IsCollisionEnabled() && (SourceSprite != nullptr) && (SourceSprite->BodySetup != nullptr) && (Owner != nullptr))
	{
		UBodySetup* BodySetup = SourceSprite->BodySetup;

		// Overall scale factor for this mesh.
		const FVector& TotalScale3D = GetComponentTransform().GetScale3D();
		if (!TotalScale3D.IsUniform() && ((BodySetup->AggGeom.BoxElems.Num() > 0) || (BodySetup->AggGeom.SphylElems.Num() > 0) || (BodySetup->AggGeom.SphereElems.Num() > 0)))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("SpriteName"), FText::FromString(SourceSprite->GetName()));
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(Owner))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_SimpleCollisionButNonUniformScaleSprite", "'{SpriteName}' has simple collision but is being scaled non-uniformly - collision creation will fail"), Arguments)))
				->AddToken(FMapErrorToken::Create(FMapErrors::SimpleCollisionButNonUniformScale));
		}
	}
}
#endif

#if WITH_EDITOR
void UPaperSpriteComponent::SetTransientTextureOverride(const UTexture* TextureToModifyOverrideFor, UTexture* OverrideTexture)
{
	FPaperSpriteSceneProxy* InSceneProxy = (FPaperSpriteSceneProxy*)SceneProxy;
	ENQUEUE_RENDER_COMMAND(FSendPaperSpriteComponentDynamicData)(
		[InSceneProxy, TextureToModifyOverrideFor, OverrideTexture](FRHICommandListImmediate& RHICmdList)
		{
			InSceneProxy->SetTransientTextureOverride_RenderThread(TextureToModifyOverrideFor, OverrideTexture);
		});
}
#endif

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

