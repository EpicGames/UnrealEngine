// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteSceneProxy.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneManagement.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/Engine.h"
#include "PaperSpriteComponent.h"
#include "SceneView.h"

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteSceneProxy

FPaperSpriteSceneProxy::FPaperSpriteSceneProxy(UPaperSpriteComponent* InComponent)
	: FPaperRenderSceneProxy_SpriteBase(InComponent)
	, BodySetup(InComponent->GetBodySetup())
{
	SetWireframeColor(InComponent->GetWireframeColor());
}

SIZE_T FPaperSpriteSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FPaperSpriteSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	if (BodySetup != nullptr)
	{
		// Show 3D physics
		if ((ViewFamily.EngineShowFlags.Collision /*@TODO: && bIsCollisionEnabled*/) && AllowDebugViewmodes())
		{
			if (FMath::Abs(GetLocalToWorld().Determinant()) < SMALL_NUMBER)
			{
				// Catch this here or otherwise GeomTransform below will assert
				// This spams so commented out
				//UE_LOGF(LogStaticMesh, Log, "Zero scaling not supported (%ls)", *StaticMesh->GetPathName());
			}
			else
			{
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if (VisibilityMap & (1 << ViewIndex))
					{
						// Make a material for drawing solid collision stuff
						const UMaterial* LevelColorationMaterial = ViewFamily.EngineShowFlags.Lighting 
							? GEngine->ShadedLevelColorationLitMaterial : GEngine->ShadedLevelColorationUnlitMaterial;

						auto CollisionMaterialInstance = new FColoredMaterialRenderProxy(
							LevelColorationMaterial->GetRenderProxy(),
							GetWireframeColor()
							);

						Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);

						// Draw the sprite body setup.

						// Get transform without scaling.
						FTransform GeomTransform(GetLocalToWorld());

						// In old wireframe collision mode, always draw the wireframe highlighted (selected or not).
						bool bDrawWireSelected = IsSelected();
						if (ViewFamily.EngineShowFlags.Collision)
						{
							bDrawWireSelected = true;
						}

						// Differentiate the color based on bBlockNonZeroExtent.  Helps greatly with skimming a level for optimization opportunities.
						const FColor CollisionColor = FColor(220,149,223,255);

						const bool bUseSeparateColorPerHull = (Owner == nullptr);
						const bool bDrawSolid = false;
						BodySetup->AggGeom.GetAggGeom(GeomTransform, GetSelectionColor(CollisionColor, bDrawWireSelected, IsHovered()).ToFColor(true), CollisionMaterialInstance, bUseSeparateColorPerHull, bDrawSolid, AlwaysHasVelocity(), ViewIndex, Collector);
					}
				}
			}
		}
	}

	FPaperRenderSceneProxy::GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector);
}
