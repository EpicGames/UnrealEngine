// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperRenderSceneProxy.h"

class FMeshElementCollector;
class UBodySetup;
class UPaperSpriteComponent;

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteSceneProxy

class FPaperSpriteSceneProxy final : public FPaperRenderSceneProxy_SpriteBase
{
public:
	SIZE_T GetTypeHash() const override;

	FPaperSpriteSceneProxy(UPaperSpriteComponent* InComponent);

	// FPrimitiveSceneProxy interface
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	// End of FPrimitiveSceneProxy interface

protected:
	const UBodySetup* BodySetup;
};
