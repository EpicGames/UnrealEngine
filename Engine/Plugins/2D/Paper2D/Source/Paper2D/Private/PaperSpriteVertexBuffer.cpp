// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteVertexBuffer.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderingThread.h"

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteVertexBuffer

void FPaperSpriteVertexBuffer::SetDynamicUsage(bool bInDynamicUsage)
{
	bDynamicUsage = bInDynamicUsage;
}

void FPaperSpriteVertexBuffer::CreateBuffers(FRHICommandListBase& RHICmdList, int32 InNumVertices)
{
	//Make sure we don't have dangling buffers
	if (NumAllocatedVertices > 0)
	{
		ReleaseBuffers();
	}

	//The buffer will always be a shader resource, but they can be static/dynamic depending of the usage
	const EBufferUsageFlags Usage = BUF_ShaderResource | (bDynamicUsage ? BUF_Dynamic : BUF_Static);
	NumAllocatedVertices = InNumVertices;

	uint32 PositionSize = NumAllocatedVertices * sizeof(FVector3f);
	// create vertex buffer
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("PaperSpritePositionBuffer"), PositionSize)
			.AddUsage(Usage)
			.DetermineInitialState();

		PositionBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			PositionBufferSRV = RHICmdList.CreateShaderResourceView(
				PositionBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_FLOAT));
		}

	}

	uint32 TangentSize = NumAllocatedVertices * 2 * sizeof(FPackedNormal);
	// create vertex buffer
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("PaperSpriteTangentBuffer"), TangentSize)
			.AddUsage(Usage)
			.DetermineInitialState();

		TangentBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			TangentBufferSRV = RHICmdList.CreateShaderResourceView(
				TangentBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8G8B8A8_SNORM));
		}
	}

	uint32 TexCoordSize = NumAllocatedVertices * sizeof(FVector2f);
	// create vertex buffer
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("PaperSpriteTexCoordBuffer"), TexCoordSize)
			.AddUsage(Usage)
			.DetermineInitialState();

		TexCoordBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			TexCoordBufferSRV = RHICmdList.CreateShaderResourceView(
				TexCoordBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_G32R32F));
		}
	}

	uint32 ColorSize = NumAllocatedVertices * sizeof(FColor);
	// create vertex buffer
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("PaperSpriteColorBuffer"), ColorSize)
			.AddUsage(Usage)
			.DetermineInitialState();

		ColorBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			ColorBufferSRV = RHICmdList.CreateShaderResourceView(
				ColorBuffer.VertexBufferRHI,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8G8B8A8));
		}
	}

	//Create Index Buffer
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex<uint32>(TEXT("PaperSpriteIndexBuffer"), Vertices.Num())
			.AddUsage(Usage)
			.DetermineInitialState();

		IndexBuffer.IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}
}

void FPaperSpriteVertexBuffer::ReleaseBuffers()
{
	PositionBuffer.ReleaseRHI();
	TangentBuffer.ReleaseRHI();
	TexCoordBuffer.ReleaseRHI();
	ColorBuffer.ReleaseRHI();
	IndexBuffer.ReleaseRHI();

	TangentBufferSRV.SafeRelease();
	TexCoordBufferSRV.SafeRelease();
	ColorBufferSRV.SafeRelease();
	PositionBufferSRV.SafeRelease();

	NumAllocatedVertices = 0;
}

void FPaperSpriteVertexBuffer::CommitVertexData(FRHICommandListBase& RHICmdList)
{
	if (Vertices.Num())
	{
		//Check if we have to accommodate the buffer size
		if (NumAllocatedVertices != Vertices.Num())
		{
			CreateBuffers(RHICmdList, Vertices.Num());
		}

		//Lock vertices
		FVector3f* PositionBufferData = nullptr;
		uint32 PositionSize = Vertices.Num() * sizeof(FVector3f);
		{
			void* Data = RHICmdList.LockBuffer(PositionBuffer.VertexBufferRHI, 0, PositionSize, RLM_WriteOnly);
			PositionBufferData = static_cast<FVector3f*>(Data);
		}

		FPackedNormal* TangentBufferData = nullptr;
		uint32 TangentSize = Vertices.Num() * 2 * sizeof(FPackedNormal);
		{
			void* Data = RHICmdList.LockBuffer(TangentBuffer.VertexBufferRHI, 0, TangentSize, RLM_WriteOnly);
			TangentBufferData = static_cast<FPackedNormal*>(Data);
		}

		FVector2f* TexCoordBufferData = nullptr;
		uint32 TexCoordSize = Vertices.Num() * sizeof(FVector2f);
		{
			void* Data = RHICmdList.LockBuffer(TexCoordBuffer.VertexBufferRHI, 0, TexCoordSize, RLM_WriteOnly);
			TexCoordBufferData = static_cast<FVector2f*>(Data);
		}

		FColor* ColorBufferData = nullptr;
		uint32 ColorSize = Vertices.Num() * sizeof(FColor);
		{
			void* Data = RHICmdList.LockBuffer(ColorBuffer.VertexBufferRHI, 0, ColorSize, RLM_WriteOnly);
			ColorBufferData = static_cast<FColor*>(Data);
		}

		uint32* IndexBufferData = nullptr;
		uint32 IndexSize = Vertices.Num() * sizeof(uint32);
		{
			void* Data = RHICmdList.LockBuffer(IndexBuffer.IndexBufferRHI, 0, IndexSize, RLM_WriteOnly);
			IndexBufferData = static_cast<uint32*>(Data);
		}

		//Fill verts
		for (int32 i = 0; i < Vertices.Num(); i++)
		{
			PositionBufferData[i] = (FVector3f)Vertices[i].Position;
			TangentBufferData[2 * i + 0] = Vertices[i].TangentX;
			TangentBufferData[2 * i + 1] = Vertices[i].TangentZ;
			ColorBufferData[i] = Vertices[i].Color;
			TexCoordBufferData[i] = Vertices[i].TextureCoordinate[0];
			IndexBufferData[i] = i;
		}

		// Unlock the buffer.
		RHICmdList.UnlockBuffer(PositionBuffer.VertexBufferRHI);
		RHICmdList.UnlockBuffer(TangentBuffer.VertexBufferRHI);
		RHICmdList.UnlockBuffer(TexCoordBuffer.VertexBufferRHI);
		RHICmdList.UnlockBuffer(ColorBuffer.VertexBufferRHI);
		RHICmdList.UnlockBuffer(IndexBuffer.IndexBufferRHI);

		//We clear the vertex data, as it isn't needed anymore
		Vertices.Empty();
	}
}

void FPaperSpriteVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	//Automatically try to create the data and use it
	CommitVertexData(RHICmdList);
}

void FPaperSpriteVertexBuffer::ReleaseRHI()
{
	PositionBuffer.ReleaseRHI();
	TangentBuffer.ReleaseRHI();
	TexCoordBuffer.ReleaseRHI();
	ColorBuffer.ReleaseRHI();
	IndexBuffer.ReleaseRHI();

	TangentBufferSRV.SafeRelease();
	TexCoordBufferSRV.SafeRelease();
	ColorBufferSRV.SafeRelease();
	PositionBufferSRV.SafeRelease();
}

void FPaperSpriteVertexBuffer::InitResource(FRHICommandListBase& RHICmdList)
{
	FRenderResource::InitResource(RHICmdList);
	PositionBuffer.InitResource(RHICmdList);
	TangentBuffer.InitResource(RHICmdList);
	TexCoordBuffer.InitResource(RHICmdList);
	ColorBuffer.InitResource(RHICmdList);
	IndexBuffer.InitResource(RHICmdList);
}

void FPaperSpriteVertexBuffer::ReleaseResource()
{
	FRenderResource::ReleaseResource();
	PositionBuffer.ReleaseResource();
	TangentBuffer.ReleaseResource();
	TexCoordBuffer.ReleaseResource();
	ColorBuffer.ReleaseResource();
	IndexBuffer.ReleaseResource();
}

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteVertexFactory

FPaperSpriteVertexFactory::FPaperSpriteVertexFactory(ERHIFeatureLevel::Type FeatureLevel)
	: FLocalVertexFactory(FeatureLevel, "FPaperSpriteVertexFactory")
{
}

void FPaperSpriteVertexFactory::Init(FRHICommandListBase& RHICmdList, const FPaperSpriteVertexBuffer* InVertexBuffer)
{
	FLocalVertexFactory::FDataType VertexData;
	VertexData.NumTexCoords = 1;

	//SRV setup
	VertexData.LightMapCoordinateIndex = 0;
	VertexData.TangentsSRV = InVertexBuffer->TangentBufferSRV;
	VertexData.TextureCoordinatesSRV = InVertexBuffer->TexCoordBufferSRV;
	VertexData.ColorComponentsSRV = InVertexBuffer->ColorBufferSRV;
	VertexData.PositionComponentSRV = InVertexBuffer->PositionBufferSRV;

	// Vertex Streams
	VertexData.PositionComponent = FVertexStreamComponent(&InVertexBuffer->PositionBuffer, 0, sizeof(FVector3f), VET_Float3, EVertexStreamUsage::Default);
	VertexData.TangentBasisComponents[0] = FVertexStreamComponent(&InVertexBuffer->TangentBuffer, 0, 2 * sizeof(FPackedNormal), VET_PackedNormal, EVertexStreamUsage::ManualFetch);
	VertexData.TangentBasisComponents[1] = FVertexStreamComponent(&InVertexBuffer->TangentBuffer, sizeof(FPackedNormal), 2 * sizeof(FPackedNormal), VET_PackedNormal, EVertexStreamUsage::ManualFetch);
	VertexData.ColorComponent = FVertexStreamComponent(&InVertexBuffer->ColorBuffer, 0, sizeof(FColor), VET_Color, EVertexStreamUsage::ManualFetch);
	VertexData.TextureCoordinates.Add(FVertexStreamComponent(&InVertexBuffer->TexCoordBuffer, 0, sizeof(FVector2f), VET_Float2, EVertexStreamUsage::ManualFetch));

	SetData(RHICmdList, VertexData);
	VertexBuffer = InVertexBuffer;

	InitResource(RHICmdList);
}
