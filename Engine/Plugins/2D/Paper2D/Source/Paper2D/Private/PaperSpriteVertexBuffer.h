// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LocalVertexFactory.h"
#include "DynamicMeshBuilder.h"

//////////////////////////////////////////////////////////////////////////
//

/** A Paper2D sprite vertex. */
using FPaperSpriteVertex = FDynamicMeshVertex;

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteVertexBuffer

/* By default this buffer is static, but can be configured to be dynamic prior to initialization. */
class FPaperSpriteVertexBuffer : public FVertexBuffer
{

public:
	//Buffers
	FVertexBuffer PositionBuffer;
	FVertexBuffer TangentBuffer;
	FVertexBuffer TexCoordBuffer;
	FVertexBuffer ColorBuffer;
	FIndexBuffer IndexBuffer;

	//SRVs for Manual Fetch on platforms that support it
	FShaderResourceViewRHIRef TangentBufferSRV;
	FShaderResourceViewRHIRef TexCoordBufferSRV;
	FShaderResourceViewRHIRef ColorBufferSRV;
	FShaderResourceViewRHIRef PositionBufferSRV;

	//Vertex data
	TArray<FPaperSpriteVertex> Vertices;

	//Ctor
	FPaperSpriteVertexBuffer()
		: bDynamicUsage(true)
		, NumAllocatedVertices(0)
	{}

	/* Marks this buffer as dynamic, so it gets initialized as so. */
	void SetDynamicUsage(bool bInDynamicUsage);

	/* Initializes the buffers with the given number of vertices to accommodate. */
	void CreateBuffers(FRHICommandListBase& RHICmdList, int32 NumVertices);

	/* Clear all the buffers currently being used. */
	void ReleaseBuffers();

	/* Moves all the PaperVertex data onto the RHI buffers. */
	void CommitVertexData(FRHICommandListBase& RHICmdList);

	// FRenderResource interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	virtual void InitResource(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseResource() override;
	// End of FRenderResource interface

	/* True if generating a commit would require a reallocation of the buffers. */
	FORCEINLINE bool CommitRequiresBufferRecreation() const { return NumAllocatedVertices != Vertices.Num(); }

	/* Checks if the buffer has been initialized. */
	FORCEINLINE bool IsInitialized() const { return NumAllocatedVertices > 0; }

	/* Obtain the index buffer initialized for this buffer. */
	FORCEINLINE const FIndexBuffer* GetIndexPtr() const { return &IndexBuffer; }

private:
	/* Indicates if this buffer will be configured for dynamic usage. */
	bool bDynamicUsage;

	/* Amount of vertices allocated on the vertex buffer. */
	int32 NumAllocatedVertices;
};


//////////////////////////////////////////////////////////////////////////
// FPaperSpriteVertexFactory

class FPaperSpriteVertexFactory : public FLocalVertexFactory
{
public:
	FPaperSpriteVertexFactory(ERHIFeatureLevel::Type FeatureLevel);

	/* Initializes this factory with a given vertex buffer. */
	void Init(FRHICommandListBase& RHICmdList, const FPaperSpriteVertexBuffer* VertexBuffer);

private:
	/* Vertex buffer used to initialize this factory. */
	const FPaperSpriteVertexBuffer* VertexBuffer;
};
