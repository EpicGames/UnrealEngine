// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileLayer.h"
#include "Paper2DModule.h"
#include "PaperTileMap.h"
#include "PaperTileSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileLayer)

//////////////////////////////////////////////////////////////////////////
// FPaperTileLayerToBodySetupBuilder

class FPaperTileLayerToBodySetupBuilder : public FSpriteGeometryCollisionBuilderBase
{
public:
	FPaperTileLayerToBodySetupBuilder(UPaperTileMap* InTileMap, UBodySetup* InBodySetup, float InZOffset, float InThickness)
		: FSpriteGeometryCollisionBuilderBase(InBodySetup)
	{
		UnrealUnitsPerPixel = InTileMap->GetUnrealUnitsPerPixel();
		CollisionThickness = InThickness;
		CollisionDomain = InTileMap->GetSpriteCollisionDomain();
		CurrentCellOffset = FVector2D::ZeroVector;
		ZOffsetAmount = InZOffset;
	}

	void SetCellOffset(const FVector2D& NewOffset, const FTransform& NewTransform)
	{
		CurrentCellOffset = NewOffset;
		MyTransform = NewTransform;
	}

protected:
	// FSpriteGeometryCollisionBuilderBase interface
	virtual FVector2D ConvertTextureSpaceToPivotSpace(const FVector2D& Input) const override
	{
		const FVector LocalPos3D = (Input.X * PaperAxisX) - (Input.Y * PaperAxisY);
		const FVector RotatedLocalPos3D = MyTransform.TransformPosition(LocalPos3D);

		const float OutputX = CurrentCellOffset.X + FVector::DotProduct(RotatedLocalPos3D, PaperAxisX);
		const float OutputY = CurrentCellOffset.Y + FVector::DotProduct(RotatedLocalPos3D, PaperAxisY);

		return FVector2D(OutputX, OutputY);
	}

	virtual FVector2D ConvertTextureSpaceToPivotSpaceNoTranslation(const FVector2D& Input) const override
	{
		const FVector LocalPos3D = (Input.X * PaperAxisX) + (Input.Y * PaperAxisY);
		const FVector RotatedLocalPos3D = MyTransform.TransformVector(LocalPos3D);

		const float OutputX = FVector::DotProduct(RotatedLocalPos3D, PaperAxisX);
		const float OutputY = FVector::DotProduct(RotatedLocalPos3D, PaperAxisY);

		return FVector2D(OutputX, OutputY);
	}
	// End of FSpriteGeometryCollisionBuilderBase

protected:
	FTransform MyTransform;
	UPaperTileLayer* MySprite;
	FVector2D CurrentCellOffset;
};

//////////////////////////////////////////////////////////////////////////
// UPaperTileLayer

UPaperTileLayer::UPaperTileLayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LayerWidth(4)
	, LayerHeight(4)
#if WITH_EDITORONLY_DATA
	, bHiddenInEditor(false)
#endif
	, bHiddenInGame(false)
	, bLayerCollides(true)
	, bOverrideCollisionThickness(false)
	, bOverrideCollisionOffset(false)
	, CollisionThicknessOverride(50.0f)
	, CollisionOffsetOverride(0.0f)
	, LayerColor(FLinearColor::White)
{

	DestructiveAllocateMap(LayerWidth, LayerHeight);
}

void UPaperTileLayer::DestructiveAllocateMap(int32 NewWidth, int32 NewHeight)
{
	check((NewWidth > 0) && (NewHeight > 0));
	LayerWidth = NewWidth;
	LayerHeight = NewHeight;

	const int32 NumCells = NewWidth * NewHeight;
	AllocatedCells.Empty(NumCells);
	AllocatedCells.AddDefaulted(NumCells);

	AllocatedWidth = NewWidth;
	AllocatedHeight = NewHeight;
}

void UPaperTileLayer::ResizeMap(int32 NewWidth, int32 NewHeight)
{
	if ((LayerWidth != NewWidth) || (LayerHeight != NewHeight))
	{
		LayerWidth = NewWidth;
		LayerHeight = NewHeight;
		ReallocateAndCopyMap();
	}
}

void UPaperTileLayer::ReallocateAndCopyMap()
{
	const int32 SavedWidth = AllocatedWidth;
	const int32 SavedHeight = AllocatedHeight;
	TArray<FPaperTileInfo> SavedDesignedMap(AllocatedCells);

	DestructiveAllocateMap(LayerWidth, LayerHeight);

	const int32 CopyWidth = FMath::Min<int32>(LayerWidth, SavedWidth);
	const int32 CopyHeight = FMath::Min<int32>(LayerHeight, SavedHeight);

	for (int32 Y = 0; Y < CopyHeight; ++Y)
	{
		for (int32 X = 0; X < CopyWidth; ++X)
		{
			const int32 SrcIndex = Y*SavedWidth + X;
			const int32 DstIndex = Y*LayerWidth + X;

			AllocatedCells[DstIndex] = SavedDesignedMap[SrcIndex];
		}
	}
}

#if WITH_EDITOR

void UPaperTileLayer::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UPaperTileLayer, LayerWidth)) || (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperTileLayer, LayerHeight)))
	{
		// Minimum size
		LayerWidth = FMath::Max<int32>(1, LayerWidth);
		LayerHeight = FMath::Max<int32>(1, LayerHeight);

		// Resize the map, trying to preserve existing data
		ReallocateAndCopyMap();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Force our owning tile map to recreate any component instances
	if (UPaperTileMap* TileMap = Cast<UPaperTileMap>(GetOuter()))
	{
		TileMap->PostEditChange();
	}
}
#endif

UPaperTileMap* UPaperTileLayer::GetTileMap() const
{
	return CastChecked<UPaperTileMap>(GetOuter());
}

int32 UPaperTileLayer::GetLayerIndex() const
{
	return GetTileMap()->TileLayers.Find(const_cast<UPaperTileLayer*>(this));
}

bool UPaperTileLayer::InBounds(int32 X, int32 Y) const
{
	return (X >= 0) && (X < LayerWidth) && (Y >= 0) && (Y < LayerHeight);
}

FPaperTileInfo UPaperTileLayer::GetCell(int32 X, int32 Y) const
{
	return InBounds(X, Y) ? AllocatedCells[X + (Y*LayerWidth)] : FPaperTileInfo();
}

void UPaperTileLayer::SetCell(int32 X, int32 Y, const FPaperTileInfo& NewValue)
{
	if (InBounds(X, Y))
	{
		AllocatedCells[X + (Y*LayerWidth)] = NewValue;
	}
}

void UPaperTileLayer::AugmentBodySetup(UBodySetup* ShapeBodySetup, float RenderSeparation)
{
	if (!bLayerCollides)
	{
		return;
	}

	UPaperTileMap* TileMap = GetTileMap();
	const float TileWidth = TileMap->TileWidth;
	const float TileHeight = TileMap->TileHeight;

	const float EffectiveCollisionOffset = bOverrideCollisionOffset ? CollisionOffsetOverride : RenderSeparation;
	const float EffectiveCollisionThickness = bOverrideCollisionThickness ? CollisionThicknessOverride : TileMap->GetCollisionThickness();

	// Generate collision for all cells that contain a tile with collision metadata
	FPaperTileLayerToBodySetupBuilder CollisionBuilder(TileMap, ShapeBodySetup, EffectiveCollisionOffset, EffectiveCollisionThickness);

	for (int32 CellY = 0; CellY < LayerHeight; ++CellY)
	{
		for (int32 CellX = 0; CellX < LayerWidth; ++CellX)
		{
			const FPaperTileInfo CellInfo = GetCell(CellX, CellY);

			if (CellInfo.IsValid())
			{
				if (const FPaperTileMetadata* CellMetadata = CellInfo.TileSet->GetTileMetadata(CellInfo.GetTileIndex()))
				{
					const int32 Flags = CellInfo.GetFlagsAsIndex();

					const FTransform& LocalTransform = GetTileTransform(Flags);
					
					// When building the collison we must consider setting the correct offsets for the projection mode.
					// This must be inline with the way you are making any changes too the 2D grid.
					FVector ProjectionOffset;
					switch (TileMap->ProjectionMode)
					{
					case ETileMapProjectionMode::Orthogonal: // General offset used by default.
					default:
						ProjectionOffset.X = TileWidth * CellX;
						ProjectionOffset.Y = TileHeight * -CellY;
					break;

					case ETileMapProjectionMode::IsometricDiamond: // Offset to compensate for the Isometric Scatter of the grid.
						ProjectionOffset.X = (CellX * TileWidth * 0.5f) + -(CellY * TileWidth * 0.5f);
						ProjectionOffset.Y = -(CellX * TileHeight * 0.5f) - (CellY * TileHeight * 0.5f);
					break;

					case ETileMapProjectionMode::IsometricStaggered: // Isometric & Hexagonal share the same offsets.
					case ETileMapProjectionMode::HexagonalStaggered:
						ProjectionOffset.X = (CellX * TileWidth) + CellY % 2 * (TileWidth * 0.5f);
						ProjectionOffset.Y = -CellY * (TileHeight * 0.5f);
					break;
					}

					// Calculate offset for tile sets that don't match the tile map's size
					const FIntPoint TileSetSize = CellInfo.TileSet->GetTileSize();
					ProjectionOffset.X += (TileSetSize.X - TileWidth) * 0.5f;
					ProjectionOffset.Y += (TileSetSize.Y - TileHeight) * 0.5f;

					const FVector2D CellOffset(ProjectionOffset.X, ProjectionOffset.Y);
					
					CollisionBuilder.SetCellOffset(CellOffset, LocalTransform);

					CollisionBuilder.ProcessGeometry(CellMetadata->CollisionData);
				}
			}
		}
	}
}

FLinearColor UPaperTileLayer::GetLayerColor() const
{
	return LayerColor;
}

void UPaperTileLayer::SetLayerColor(FLinearColor NewColor)
{
	LayerColor = NewColor;
}

void UPaperTileLayer::ConvertToTileSetPerCell()
{
	AllocatedCells.Empty(AllocatedGrid_DEPRECATED.Num());

	const int32 NumCells = AllocatedWidth * AllocatedHeight;
	for (int32 Index = 0; Index < NumCells; ++Index)
	{
		FPaperTileInfo* Info = new (AllocatedCells) FPaperTileInfo();
		Info->TileSet = TileSet_DEPRECATED;
		Info->PackedTileIndex = AllocatedGrid_DEPRECATED[Index];
	}
}

bool UPaperTileLayer::UsesTileSet(UPaperTileSet* TileSet) const
{
	for (const FPaperTileInfo& TileInfo : AllocatedCells)
	{
		if (TileInfo.TileSet == TileSet)
		{
			if (TileInfo.IsValid())
			{
				return true;
			}
		}
	}

	return false;
}

FTransform UPaperTileLayer::GetTileTransform(int32 FlagIndex)
{
	// Handles the rotation and flipping of collision geometry from a tile
	// 0,5,6,3 are clockwise rotations of a regular tile
	// 4,7,2,1 are clockwise rotations of a horizontally flipped tile
	const static FTransform TilePermutationTransforms[8] =
	{
		// 000 - normal
		FTransform::Identity,

		// 001 - diagonal
		FTransform(FRotator(90.0f, 0.0f, 0.0f), FVector::ZeroVector, -PaperAxisX.GetAbs() + PaperAxisY.GetAbs() + PaperAxisZ.GetAbs()),

		// 010 - flip Y
		FTransform(FRotator(-180.0f, 0.0f, 0.0f), FVector::ZeroVector, -PaperAxisX.GetAbs() + PaperAxisY.GetAbs() + PaperAxisZ.GetAbs()),

		// 011 - diagonal then flip Y (rotate 270 clockwise)
		FTransform(FRotator(90.0f, 0.0f, 0.0f)),

		// 100 - flip X
		FTransform(FRotator::ZeroRotator, FVector::ZeroVector, -PaperAxisX.GetAbs() + PaperAxisY.GetAbs() + PaperAxisZ.GetAbs()),

		// 101 - diagonal then flip X (clockwise 90)
		FTransform(FRotator(-90.0f, 0.0f, 0.0f)),

		// 110 - flip X and flip Y (rotate 180 either way)
		FTransform(FRotator(-180.0f, 0.0f, 0.0f)),

		// 111 - diagonal then flip X and Y
		FTransform(FRotator(-90.0f, 0.0f, 0.0f), FVector::ZeroVector, -PaperAxisX.GetAbs() + PaperAxisY.GetAbs() + PaperAxisZ.GetAbs()),
	};


	checkSlow((FlagIndex >= 0) && (FlagIndex < 8));
	return TilePermutationTransforms[FlagIndex];
}

int32 UPaperTileLayer::GetNumOccupiedCells() const
{
	int32 NumOccupiedCells = 0;

	for (const FPaperTileInfo& TileInfo : AllocatedCells)
	{
		if (TileInfo.IsValid())
		{
			++NumOccupiedCells;
		}
	}

	return NumOccupiedCells;
}

