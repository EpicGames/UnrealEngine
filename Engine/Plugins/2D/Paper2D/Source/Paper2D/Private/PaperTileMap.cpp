// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileMap.h"
#include "BodySetupEnums.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/MaterialInterface.h"
#include "EditorFramework/AssetImportData.h"
#include "Paper2DModule.h"
#include "PhysicsEngine/BodySetup.h"
#include "PaperCustomVersion.h"
#include "PaperTileSet.h"
#include "PaperTileLayer.h"
#include "Paper2DPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileMap)

#if WITH_EDITOR
#include "PaperTileMapComponent.h"
#include "ComponentReregisterContext.h"
#endif

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// UPaperTileMap

UPaperTileMap::UPaperTileMap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MapWidth = 4;
	MapHeight = 4;
	TileWidth = 32;
	TileHeight = 32;
	PixelsPerUnrealUnit = 1.0f;
	SeparationPerTileX = 0.0f;
	SeparationPerTileY = 0.0f;
	SeparationPerLayer = 4.0f;
	CollisionThickness = 50.0f;
	SpriteCollisionDomain = ESpriteCollisionMode::Use3DPhysics;

#if WITH_EDITORONLY_DATA
	SelectedLayerIndex = INDEX_NONE;
	BackgroundColor = FColor(55, 55, 55);
	TileGridColor = FLinearColor::Gray;
	MultiTileGridColor = FLinearColor::Red;
	MultiTileGridWidth = 0;
	MultiTileGridHeight = 0;
	MultiTileGridOffsetX = 0;
	MultiTileGridOffsetY = 0;
	LayerGridColor = FLinearColor::White;
#endif

	LayerNameIndex = 0;

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultMaterial(TEXT("/Paper2D/MaskedUnlitSpriteMaterial"));
	Material = DefaultMaterial.Object;
}

void UPaperTileMap::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject) && !(GetOuter() && GetOuter()->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject)))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
	Super::PostInitProperties();
}

#if WITH_EDITOR
void UPaperTileMap::PreEditChange(FProperty* PropertyAboutToChange)
{
	if ((PropertyAboutToChange != nullptr) && (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UPaperTileMap, HexSideLength)))
	{
		// Subtract out the hex side length; we'll add it back (along with any changes) in PostEditChangeProperty
		TileHeight -= HexSideLength;
	}

	Super::PreEditChange(PropertyAboutToChange);
}
#endif

void UPaperTileMap::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && (Ar.UEVer() < VER_UE4_ASSET_IMPORT_DATA_AS_JSON) && (AssetImportData == nullptr))
	{
		// AssetImportData should always be valid
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif

	if (SpriteCollisionDomain == ESpriteCollisionMode::Use2DPhysics)
	{
		UE_LOGF(LogPaper2D, Warning, "PaperTileMap '%ls' was using 2D physics which has been removed, it has been switched to 3D physics.", *GetPathName());
		SpriteCollisionDomain = ESpriteCollisionMode::Use3DPhysics;
	}
}

void UPaperTileMap::PostLoad()
{
	Super::PostLoad();

	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);

	// Make sure that the layers are all of the right size (there was a bug at one point when undoing resizes that could cause the layers to get stuck at a bad size)
	for (UPaperTileLayer* TileLayer : TileLayers)
	{
		TileLayer->ConditionalPostLoad();

		TileLayer->ResizeMap(MapWidth, MapHeight);

		if (PaperVer < FPaperCustomVersion::FixVertexColorSpace)
		{
			const FColor SRGBColor = TileLayer->GetLayerColor().ToFColor(/*bSRGB=*/ true);
			TileLayer->SetLayerColor(SRGBColor.ReinterpretAsLinear());
		}
	}

#if WITH_EDITOR
	ValidateSelectedLayerIndex();
#endif
}

#if WITH_EDITOR

/** Removes all components that use the specified sprite asset from their scenes for the lifetime of the class. */
class FTileMapReregisterContext
{
public:
	/** Initialization constructor. */
	FTileMapReregisterContext(UPaperTileMap* TargetAsset)
	{
		// Look at tile map components
		for (TObjectIterator<UPaperTileMapComponent> TileMapIt; TileMapIt; ++TileMapIt)
		{
			if (UPaperTileMapComponent* TestComponent = *TileMapIt)
			{
				if (TestComponent->TileMap == TargetAsset)
				{
					AddComponentToRefresh(TestComponent);
				}
			}
		}
	}

protected:
	void AddComponentToRefresh(UActorComponent* Component)
	{
		if (ComponentContexts.Num() == 0)
		{
			// wait until resources are released
			FlushRenderingCommands();
		}

		ComponentContexts.Add(new FComponentReregisterContext(Component));
	}

private:
	/** The recreate contexts for the individual components. */
	TIndirectArray<FComponentReregisterContext> ComponentContexts;
};

void UPaperTileMap::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	//@TODO: Determine when these are really needed, as they're seriously expensive!
	FTileMapReregisterContext ReregisterTileMapComponents(this);

	ValidateSelectedLayerIndex();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperTileMap, HexSideLength))
	{
		HexSideLength = FMath::Max<int32>(HexSideLength, 0);

		// The side length needs to be included in the overall tile height
		TileHeight += HexSideLength;
	}

	TileWidth = FMath::Max(TileWidth, 1);
	TileHeight = FMath::Max(TileHeight, 1);
	MapWidth = FMath::Max(MapWidth, 1);
	MapHeight = FMath::Max(MapHeight, 1);

	if (PixelsPerUnrealUnit <= 0.0f)
	{
		PixelsPerUnrealUnit = 1.0f;
	}

	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UPaperTileMap, MapWidth)) || (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperTileMap, MapHeight)))
	{
		ResizeMap(MapWidth, MapHeight, /*bForceResize=*/ true);
	}
	else
	{
		// Make sure that the layers are all of the right size
		for (UPaperTileLayer* TileLayer : TileLayers)
		{
			if (TileLayer == nullptr)
			{
				continue;
			}

			if ((TileLayer->GetLayerWidth() != MapWidth) || (TileLayer->GetLayerHeight() != MapHeight))
			{
				TileLayer->Modify();
				TileLayer->ResizeMap(MapWidth, MapHeight);
			}
		}
	}

	if (!IsTemplate())
	{
		UpdateBodySetup();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UPaperTileMap::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperTileMap, HexSideLength))
		{
			bIsEditable = ProjectionMode == ETileMapProjectionMode::HexagonalStaggered;
		}
	}

	return bIsEditable;
}

void UPaperTileMap::ValidateSelectedLayerIndex()
{
	if (!TileLayers.IsValidIndex(SelectedLayerIndex))
	{
		// Select the top-most visible layer
		SelectedLayerIndex = INDEX_NONE;
		for (int32 LayerIndex = 0; (LayerIndex < TileLayers.Num()) && (SelectedLayerIndex == INDEX_NONE); ++LayerIndex)
		{
			if (TileLayers[LayerIndex] && TileLayers[LayerIndex]->ShouldRenderInEditor())
			{
				SelectedLayerIndex = LayerIndex;
			}
		}

		if ((SelectedLayerIndex == INDEX_NONE) && (TileLayers.Num() > 0))
		{
			SelectedLayerIndex = 0;
		}
	}
}

#endif

#if WITH_EDITORONLY_DATA
void UPaperTileMap::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	Super::GetAssetRegistryTags(Context);
}
#endif

void UPaperTileMap::UpdateBodySetup()
{
	// Ensure we have the data structure for the desired collision method
	switch (SpriteCollisionDomain)
	{
	case ESpriteCollisionMode::Use3DPhysics:
		BodySetup = NewObject<UBodySetup>(this);
		break;
	case ESpriteCollisionMode::None:
		BodySetup = nullptr;
		return;
	}

	BodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;

	for (int32 LayerIndex = 0; LayerIndex < TileLayers.Num(); ++LayerIndex)
	{
		const float ZSeparation = LayerIndex * SeparationPerLayer;
		TileLayers[LayerIndex]->AugmentBodySetup(BodySetup, ZSeparation);
	}

	// Finalize the BodySetup
	BodySetup->InvalidatePhysicsData();
	BodySetup->CreatePhysicsMeshes();
}

void UPaperTileMap::GetTileToLocalParameters(FVector& OutCornerPosition, FVector& OutStepX, FVector& OutStepY, FVector& OutOffsetYFactor) const
{
	const float UnrealUnitsPerPixel = GetUnrealUnitsPerPixel();
	const float TileWidthInUU = TileWidth * UnrealUnitsPerPixel;
	const float TileHeightInUU = TileHeight * UnrealUnitsPerPixel;

	switch (ProjectionMode)
	{
	case ETileMapProjectionMode::Orthogonal:
	default:
		OutCornerPosition = -(TileWidthInUU * PaperAxisX * 0.5f) + (TileHeightInUU * PaperAxisY * 0.5f);
		OutOffsetYFactor = FVector::ZeroVector;
		OutStepX = PaperAxisX * TileWidthInUU;
		OutStepY = -PaperAxisY * TileHeightInUU;
		break;
	case ETileMapProjectionMode::IsometricDiamond:
		OutCornerPosition = (TileHeightInUU * PaperAxisY * 0.5f);
		OutOffsetYFactor = FVector::ZeroVector;
		OutStepX = (TileWidthInUU * PaperAxisX * 0.5f) - (TileHeightInUU * PaperAxisY * 0.5f);
		OutStepY = (TileWidthInUU * PaperAxisX * -0.5f) - (TileHeightInUU * PaperAxisY * 0.5f);
		break;
	case ETileMapProjectionMode::HexagonalStaggered:
	case ETileMapProjectionMode::IsometricStaggered:
		OutCornerPosition = -(TileWidthInUU * PaperAxisX * 0.5f) + (TileHeightInUU * PaperAxisY * 0.5f);
		OutOffsetYFactor = 0.5f * TileWidthInUU * PaperAxisX;
		OutStepX = PaperAxisX * TileWidthInUU;
		OutStepY = 0.5f * -PaperAxisY * TileHeightInUU;
		break;
	}
}

void UPaperTileMap::GetLocalToTileParameters(FVector& OutCornerPosition, FVector& OutStepX, FVector& OutStepY, FVector& OutOffsetYFactor) const
{
	const float UnrealUnitsPerPixel = GetUnrealUnitsPerPixel();
	const float TileWidthInUU = TileWidth * UnrealUnitsPerPixel;
	const float TileHeightInUU = TileHeight * UnrealUnitsPerPixel;

	switch (ProjectionMode)
	{
	case ETileMapProjectionMode::Orthogonal:
	default:
		OutCornerPosition = -(TileWidthInUU * PaperAxisX * 0.5f) + (TileHeightInUU * PaperAxisY * 0.5f);
		OutOffsetYFactor = FVector::ZeroVector;
		OutStepX = PaperAxisX / TileWidthInUU;
		OutStepY = -PaperAxisY / TileHeightInUU;
		break;
	case ETileMapProjectionMode::IsometricDiamond:
		OutCornerPosition = (TileHeightInUU * PaperAxisY * 0.5f);
		OutOffsetYFactor = FVector::ZeroVector;
		OutStepX = (PaperAxisX / TileWidthInUU) - (PaperAxisY / TileHeightInUU);
		OutStepY = (-PaperAxisX / TileWidthInUU) - (PaperAxisY / TileHeightInUU);
		break;
	case ETileMapProjectionMode::HexagonalStaggered:
	case ETileMapProjectionMode::IsometricStaggered:
		OutCornerPosition = -(TileWidthInUU * PaperAxisX * 0.5f) + (TileHeightInUU * PaperAxisY * 0.5f);
		OutOffsetYFactor = 0.5f * TileWidthInUU * PaperAxisX;
		OutStepX = PaperAxisX / TileWidthInUU;
		OutStepY = -PaperAxisY / TileHeightInUU;
		break;
	}
}

void UPaperTileMap::GetTileCoordinatesFromLocalSpacePosition(const FVector& Position, int32& OutTileX, int32& OutTileY) const
{
	FVector CornerPosition;
	FVector OffsetYFactor;
	FVector ParameterAxisX;
	FVector ParameterAxisY;

	// position is in pixels
	// axis is tiles to pixels

	GetLocalToTileParameters(/*out*/ CornerPosition, /*out*/ ParameterAxisX, /*out*/ ParameterAxisY, /*out*/ OffsetYFactor);

	const FVector RelativePosition = Position - CornerPosition;
	const float ProjectionSpaceXInTiles = FVector::DotProduct(RelativePosition, ParameterAxisX);
	const float ProjectionSpaceYInTiles = FVector::DotProduct(RelativePosition, ParameterAxisY);

	float X2 = ProjectionSpaceXInTiles;
	float Y2 = ProjectionSpaceYInTiles;

	if ((ProjectionMode == ETileMapProjectionMode::IsometricStaggered) || (ProjectionMode == ETileMapProjectionMode::HexagonalStaggered))
	{
		const float px = FMath::Frac(ProjectionSpaceXInTiles);
		const float py = FMath::Frac(ProjectionSpaceYInTiles);

		// Determine if the point is inside of the diamond or outside
		const float h = 0.5f;
		const float Det1 = -((px - h)*h - py*h);
		const float Det2 = -(px - 1.0f)*h - (py - h)*h;
		const float Det3 = -(-(px - h)*h + (py - 1.0f)*h);
		const float Det4 = px*h + (py - h)*h;

		const bool bOutsideTile = (Det1 < 0.0f) || (Det2 < 0.0f) || (Det3 < 0.0f) || (Det4 < 0.0f);

		if (bOutsideTile)
		{
			X2 = ProjectionSpaceXInTiles - ((px < 0.5f) ? 1.0f : 0.0f);
			Y2 = FMath::FloorToFloat(ProjectionSpaceYInTiles)*2.0f + py + ((py < 0.5f) ? -1.0f : 1.0f);
		}
 		else
 		{
 			X2 = ProjectionSpaceXInTiles;
			Y2 = FMath::FloorToFloat(ProjectionSpaceYInTiles)*2.0f + py;
 		}
	}

	OutTileX = FMath::FloorToInt(X2);
	OutTileY = FMath::FloorToInt(Y2);
}

FVector UPaperTileMap::GetTilePositionInLocalSpace(float TileX, float TileY, int32 LayerIndex) const
{
	FVector CornerPosition;
	FVector OffsetYFactor;
	FVector StepX;
	FVector StepY;

	GetTileToLocalParameters(/*out*/ CornerPosition, /*out*/ StepX, /*out*/ StepY, /*out*/ OffsetYFactor);

	FVector TotalOffset;
	switch (ProjectionMode)
	{
	case ETileMapProjectionMode::Orthogonal:
	default:
		TotalOffset = CornerPosition;
		break;
	case ETileMapProjectionMode::IsometricDiamond:
		TotalOffset = CornerPosition;
		break;
	case ETileMapProjectionMode::HexagonalStaggered:
	case ETileMapProjectionMode::IsometricStaggered:
		TotalOffset = CornerPosition + ((int32)TileY & 1) * OffsetYFactor;
		break;
	}

	const FVector PartialX = TileX * StepX;
	const FVector PartialY = TileY * StepY;

	const float TotalSeparation = (SeparationPerLayer * LayerIndex) + (SeparationPerTileX * TileX) + (SeparationPerTileY * TileY);
	const FVector PartialZ = TotalSeparation * PaperAxisZ;

	const FVector LocalPos(PartialX + PartialY + PartialZ + TotalOffset);

	return LocalPos;
}

void UPaperTileMap::GetTilePolygon(int32 TileX, int32 TileY, int32 LayerIndex, TArray<FVector>& LocalSpacePoints) const
{
	switch (ProjectionMode)
	{
	case ETileMapProjectionMode::Orthogonal:
	case ETileMapProjectionMode::IsometricDiamond:
	default:
		LocalSpacePoints.Add(GetTilePositionInLocalSpace(TileX, TileY, LayerIndex));
		LocalSpacePoints.Add(GetTilePositionInLocalSpace(TileX + 1, TileY, LayerIndex));
		LocalSpacePoints.Add(GetTilePositionInLocalSpace(TileX + 1, TileY + 1, LayerIndex));
		LocalSpacePoints.Add(GetTilePositionInLocalSpace(TileX, TileY + 1, LayerIndex));
		break;

	case ETileMapProjectionMode::IsometricStaggered:
		{
			const float UnrealUnitsPerPixel = GetUnrealUnitsPerPixel();
			const float TileWidthInUU = TileWidth * UnrealUnitsPerPixel;
			const float TileHeightInUU = TileHeight * UnrealUnitsPerPixel;

			const FVector RecenterOffset = PaperAxisX*TileWidthInUU*0.5f;
			const FVector LSTM = GetTilePositionInLocalSpace(TileX, TileY, LayerIndex) + RecenterOffset;

			LocalSpacePoints.Add(LSTM);
			LocalSpacePoints.Add(LSTM + PaperAxisX*TileWidthInUU*0.5f - PaperAxisY*TileHeightInUU*0.5f);
			LocalSpacePoints.Add(LSTM - PaperAxisY*TileHeightInUU*1.0f);
			LocalSpacePoints.Add(LSTM - PaperAxisX*TileWidthInUU*0.5f - PaperAxisY*TileHeightInUU*0.5f);
		}
		break;

	case ETileMapProjectionMode::HexagonalStaggered:
		{
			const float UnrealUnitsPerPixel = GetUnrealUnitsPerPixel();
			const float TileWidthInUU = TileWidth * UnrealUnitsPerPixel;
			const float TileHeightInUU = TileHeight * UnrealUnitsPerPixel;

			const FVector HalfWidth = PaperAxisX*TileWidthInUU*0.5f;
			const FVector LSTM = GetTilePositionInLocalSpace(TileX, TileY, LayerIndex) + HalfWidth;

			const float HexSideLengthInUU = HexSideLength * UnrealUnitsPerPixel;
			const float HalfHexLength = HexSideLengthInUU*0.5f;
			const FVector Top(LSTM - PaperAxisY*HalfHexLength);

			const FVector StepTopSides = PaperAxisY*(TileHeightInUU*0.5f - HalfHexLength);
			const FVector RightTop(LSTM + HalfWidth - StepTopSides);
			const FVector LeftTop(LSTM - HalfWidth - StepTopSides);

			const FVector StepBottomSides = PaperAxisY*(TileHeightInUU*0.5f + HalfHexLength);
			const FVector RightBottom(LSTM + HalfWidth - StepBottomSides);
			const FVector LeftBottom(LSTM - HalfWidth - StepBottomSides);

			const FVector Bottom(LSTM - PaperAxisY*(TileHeightInUU - HalfHexLength));

			LocalSpacePoints.Add(Top);
			LocalSpacePoints.Add(RightTop);
			LocalSpacePoints.Add(RightBottom);
			LocalSpacePoints.Add(Bottom);
			LocalSpacePoints.Add(LeftBottom);
			LocalSpacePoints.Add(LeftTop);
		}
		break;
	}
}

FVector UPaperTileMap::GetTileCenterInLocalSpace(float TileX, float TileY, int32 LayerIndex) const
{
	switch (ProjectionMode)
	{
	case ETileMapProjectionMode::Orthogonal:
	default:
		return GetTilePositionInLocalSpace(TileX + 0.5f, TileY + 0.5f, LayerIndex);

	case ETileMapProjectionMode::IsometricDiamond:
		return GetTilePositionInLocalSpace(TileX + 0.5f, TileY + 0.5f, LayerIndex);

	case ETileMapProjectionMode::IsometricStaggered:
		{
			const float UnrealUnitsPerPixel = GetUnrealUnitsPerPixel();
			const float TileWidthInUU = TileWidth * UnrealUnitsPerPixel;
			const float TileHeightInUU = TileHeight * UnrealUnitsPerPixel;

			const FVector RecenterOffset = PaperAxisX*TileWidthInUU*0.5f - PaperAxisY*TileHeightInUU*0.5f;
			return GetTilePositionInLocalSpace(TileX, TileY, LayerIndex) + RecenterOffset;
		}

	case ETileMapProjectionMode::HexagonalStaggered:
		{
			const float UnrealUnitsPerPixel = GetUnrealUnitsPerPixel();
			const float TileWidthInUU = TileWidth * UnrealUnitsPerPixel;
			const float TileHeightInUU = TileHeight * UnrealUnitsPerPixel;
			const float HexSideLengthInUU = HexSideLength * UnrealUnitsPerPixel;

			const FVector RecenterOffset = PaperAxisX*TileWidthInUU*0.5f - PaperAxisY*(TileHeightInUU)*0.5f;
			return GetTilePositionInLocalSpace(TileX, TileY, LayerIndex) + RecenterOffset;
		}
	}
}

void UPaperTileMap::SetCollisionThickness(float Thickness)
{
	CollisionThickness = Thickness;
}

void UPaperTileMap::SetCollisionDomain(ESpriteCollisionMode::Type Domain)
{
	SpriteCollisionDomain = Domain;
	UpdateBodySetup();
}

void UPaperTileMap::RebuildCollision()
{
	UpdateBodySetup();
}

FBoxSphereBounds UPaperTileMap::GetRenderBounds() const
{
	const float Depth = SeparationPerLayer * (TileLayers.Num() - 1);
	const float HalfThickness = 2.0f;

	const float UnrealUnitsPerPixel = GetUnrealUnitsPerPixel();
	const float TileWidthInUU = TileWidth * UnrealUnitsPerPixel;
	const float TileHeightInUU = TileHeight * UnrealUnitsPerPixel;

	switch (ProjectionMode)
	{
		case ETileMapProjectionMode::Orthogonal:
		default:
		{
			const FVector BottomLeft((-0.5f) * TileWidthInUU, -HalfThickness - Depth, -(MapHeight - 0.5f) * TileHeightInUU);
			const FVector Dimensions(MapWidth * TileWidthInUU, Depth + 2 * HalfThickness, MapHeight * TileHeightInUU);

			const FBox Box(BottomLeft, BottomLeft + Dimensions);
			return FBoxSphereBounds(Box);
		}
		case ETileMapProjectionMode::IsometricDiamond:
		{
			 const FVector BottomLeft((-0.5f) * TileWidthInUU * MapWidth, -HalfThickness - Depth, -MapHeight * TileHeightInUU);
			 const FVector Dimensions(MapWidth * TileWidthInUU, Depth + 2 * HalfThickness, (MapHeight + 1) * TileHeightInUU);

			 const FBox Box(BottomLeft, BottomLeft + Dimensions);
			 return FBoxSphereBounds(Box);
		}
		case ETileMapProjectionMode::HexagonalStaggered:
		case ETileMapProjectionMode::IsometricStaggered:
		{
			const int32 RoundedHalfHeight = (MapHeight + 1) / 2;
			const FVector BottomLeft((-0.5f) * TileWidthInUU, -HalfThickness - Depth, -(RoundedHalfHeight) * TileHeightInUU);
			const FVector Dimensions((MapWidth + 0.5f) * TileWidthInUU, Depth + 2 * HalfThickness, (RoundedHalfHeight + 1.0f) * TileHeightInUU);

			const FBox Box(BottomLeft, BottomLeft + Dimensions);
			return FBoxSphereBounds(Box);
		}
	}
}

UPaperTileLayer* UPaperTileMap::AddNewLayer(int32 InsertionIndex)
{
	// Create the new layer
	UPaperTileLayer* NewLayer = NewObject<UPaperTileLayer>(this);
	NewLayer->SetFlags(RF_Transactional);

	NewLayer->DestructiveAllocateMap(MapWidth, MapHeight);
	NewLayer->LayerName = GenerateNewLayerName(this);

	// Insert the new layer
	if (TileLayers.IsValidIndex(InsertionIndex))
	{
		TileLayers.Insert(NewLayer, InsertionIndex);
	}
	else
	{
		TileLayers.Add(NewLayer);
	}

	return NewLayer;
}

void UPaperTileMap::AddExistingLayer(UPaperTileLayer* NewLayer, int32 InsertionIndex)
{
	NewLayer->SetFlags(RF_Transactional);
	NewLayer->Modify();

	// Make sure the layer has the correct outer
	if (NewLayer->GetOuter() != this)
	{
		NewLayer->Rename(nullptr, this);
	}

	// And correct size
	NewLayer->ResizeMap(MapWidth, MapHeight);

	// And a unique name
	if (IsLayerNameInUse(NewLayer->LayerName))
	{
		NewLayer->LayerName = GenerateNewLayerName(this);
	}

	// Insert the new layer
	if (TileLayers.IsValidIndex(InsertionIndex))
	{
		TileLayers.Insert(NewLayer, InsertionIndex);
	}
	else
	{
		TileLayers.Add(NewLayer);
	}
}

FText UPaperTileMap::GenerateNewLayerName(UPaperTileMap* TileMap)
{
	// Create a set of existing names
	TSet<FString> ExistingNames;
	for (UPaperTileLayer* ExistingLayer : TileMap->TileLayers)
	{
		ExistingNames.Add(ExistingLayer->LayerName.ToString());
	}

	// Find a good name
	FText TestLayerName;
	do
	{
		TileMap->LayerNameIndex++;
		TestLayerName = FText::Format(LOCTEXT("NewLayerNameFormatString", "Layer {0}"), FText::AsNumber(TileMap->LayerNameIndex, &FNumberFormattingOptions::DefaultNoGrouping()));
	} while (ExistingNames.Contains(TestLayerName.ToString()));

	return TestLayerName;
}

bool UPaperTileMap::IsLayerNameInUse(const FText& LayerName) const
{
	for (UPaperTileLayer* ExistingLayer : TileLayers)
	{
		if (ExistingLayer->LayerName.EqualToCaseIgnored(LayerName))
		{
			return true;
		}
	}

	return false;
}

void UPaperTileMap::ResizeMap(int32 NewWidth, int32 NewHeight, bool bForceResize)
{
	if (bForceResize || (NewWidth != MapWidth) || (NewHeight != MapHeight))
	{
		MapWidth = FMath::Max(NewWidth, 1);
		MapHeight = FMath::Max(NewHeight, 1);

		// Resize all of the existing layers
		for (int32 LayerIndex = 0; LayerIndex < TileLayers.Num(); ++LayerIndex)
		{
			UPaperTileLayer* TileLayer = TileLayers[LayerIndex];
			TileLayer->Modify();
			TileLayer->ResizeMap(MapWidth, MapHeight);
		}
	}
}

void UPaperTileMap::InitializeNewEmptyTileMap(UPaperTileSet* InitialTileSet)
{
	if (InitialTileSet != nullptr)
	{
		const FIntPoint TileSetTileSize = InitialTileSet->GetTileSize();
		TileWidth = TileSetTileSize.X;
		TileHeight = TileSetTileSize.Y;
		SelectedTileSet = InitialTileSet;
	}

	AddNewLayer();
}

UPaperTileMap* UPaperTileMap::CloneTileMap(UObject* OuterForClone)
{
	return CastChecked<UPaperTileMap>(StaticDuplicateObject(this, OuterForClone));
}

bool UPaperTileMap::UsesTileSet(UPaperTileSet* TileSet) const
{
	for (UPaperTileLayer* Layer : TileLayers)
	{
		if (Layer->UsesTileSet(TileSet))
		{
			return true;
		}
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

