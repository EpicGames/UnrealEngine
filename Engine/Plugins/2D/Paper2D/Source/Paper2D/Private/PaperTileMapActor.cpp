// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileMapActor.h"
#include "PaperTileMapComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileMapActor)

//////////////////////////////////////////////////////////////////////////
// APaperTileMapActor

APaperTileMapActor::APaperTileMapActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RenderComponent = CreateDefaultSubobject<UPaperTileMapComponent>(TEXT("RenderComponent"));

	RootComponent = RenderComponent;
}

#if WITH_EDITOR
bool APaperTileMapActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (const UObject* Asset = RenderComponent->AdditionalStatObject())
	{
		Objects.Add(const_cast<UObject*>(Asset));
	}
	return true;
}
#endif

