// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileMapComponent.h"
#include "PaperTileLayer.h"
#include "UObject/ConstructorHelpers.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Paper2DPrivate.h"
#include "PaperTileMapRenderSceneProxy.h"
#include "PaperCustomVersion.h"
#include "PhysicsEngine/BodySetup.h"
#include "PaperTileMap.h"
#include "PaperTileSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileMapComponent)

#define LOCTEXT_NAMESPACE "Paper2D"

DECLARE_CYCLE_STAT(TEXT("Rebuild Tile Map"), STAT_PaperRender_TileMapRebuild, STATGROUP_Paper2D);

//////////////////////////////////////////////////////////////////////////
// UPaperTileMapComponent

UPaperTileMapComponent::UPaperTileMapComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TileMapColor(FLinearColor::White)
	, UseSingleLayerIndex(0)
	, bUseSingleLayer(false)
{
	BodyInstance.SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);

	MapWidth_DEPRECATED = 4;
	MapHeight_DEPRECATED = 4;
	TileWidth_DEPRECATED = 32;
	TileHeight_DEPRECATED = 32;

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultMaterial(TEXT("/Paper2D/MaskedUnlitSpriteMaterial"));
	Material_DEPRECATED = DefaultMaterial.Object;

	CastShadow = false;
	bUseAsOccluder = false;
	bCanEverAffectNavigation = true;

#if WITH_EDITORONLY_DATA
	bShowPerTileGridWhenSelected = true;
	bShowPerTileGridWhenUnselected = false;
	bShowPerLayerGridWhenSelected = true;
	bShowPerLayerGridWhenUnselected = false;
	bShowOutlineWhenUnselected = true;
#endif

#if WITH_EDITOR
	NumBatches = 0;
	NumTriangles = 0;
#endif
}

FPrimitiveSceneProxy* UPaperTileMapComponent::CreateSceneProxy()
{
	SCOPE_CYCLE_COUNTER(STAT_PaperRender_TileMapRebuild);

	TArray<FSpriteRenderSection>* Sections;
	TArray<FDynamicMeshVertex>* Vertices;
	FPaperTileMapRenderSceneProxy* Proxy = FPaperTileMapRenderSceneProxy::CreateTileMapProxy(this, /*out*/ Sections, /*out*/ Vertices);

	RebuildRenderData(*Sections, *Vertices);

	Proxy->FinishConstruction_GameThread();

	return Proxy;
}

void UPaperTileMapComponent::PostInitProperties()
{
	EObjectFlags Flags = HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ? 
		GetMaskedFlags(RF_PropagateToSubObjects) : RF_NoFlags;
	// In a post GFastPathUniqueNameGeneration world we have to provide a stable name
	// for all archetypes, here I'm using PaperTileMap_0 to match old content:
	TileMap = NewObject<UPaperTileMap>(this, TEXT("PaperTileMap_0"), Flags);

	TileMap->SetFlags(RF_Transactional);

	Super::PostInitProperties();
}

FBoxSphereBounds UPaperTileMapComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (TileMap != nullptr)
	{
		// Graphics bounds.
		FBoxSphereBounds NewBounds = TileMap->GetRenderBounds().TransformBy(LocalToWorld);

		// Add bounds of collision geometry (if present).
		if (UBodySetup* BodySetup = TileMap->BodySetup)
		{
			const FBox AggGeomBox = BodySetup->AggGeom.CalcAABB(LocalToWorld);
			if (AggGeomBox.IsValid)
			{
				NewBounds = Union(NewBounds, FBoxSphereBounds(AggGeomBox));
			}
		}

		// Apply bounds scale
		NewBounds.BoxExtent *= BoundsScale;
		NewBounds.SphereRadius *= BoundsScale;

		return NewBounds;
	}
	else
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}
}

void UPaperTileMapComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);
}

void UPaperTileMapComponent::PostLoad()
{
	Super::PostLoad();

	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);
	
	if (PaperVer < FPaperCustomVersion::MovedTileMapDataToSeparateClass)
	{
		// Create a tile map object and move our old properties over to it
		TileMap = NewObject<UPaperTileMap>(this);
		TileMap->SetFlags(RF_Transactional);
		TileMap->MapWidth = MapWidth_DEPRECATED;
		TileMap->MapHeight = MapHeight_DEPRECATED;
		TileMap->TileWidth = TileWidth_DEPRECATED;
		TileMap->TileHeight = TileHeight_DEPRECATED;
		TileMap->PixelsPerUnrealUnit = 1.0f;
		TileMap->SelectedTileSet = DefaultLayerTileSet_DEPRECATED;
		TileMap->Material = Material_DEPRECATED;
		TileMap->TileLayers = TileLayers_DEPRECATED;

		// Convert the layers
		for (UPaperTileLayer* Layer : TileMap->TileLayers)
		{
			Layer->Rename(nullptr, TileMap, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			Layer->ConvertToTileSetPerCell();
		}

		// Remove references in the deprecated variables to prevent issues with deleting referenced assets, etc...
		DefaultLayerTileSet_DEPRECATED = nullptr;
		Material_DEPRECATED = nullptr;
		TileLayers_DEPRECATED.Empty();
	}

	if (PaperVer < FPaperCustomVersion::FixVertexColorSpace)
	{
		const FColor SRGBColor = TileMapColor.ToFColor(/*bSRGB=*/ true);
		TileMapColor = SRGBColor.ReinterpretAsLinear();
	}
}

#if WITH_EDITOR
void UPaperTileMapComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (TileMap != nullptr)
	{
		if (TileMap->TileLayers.Num() > 0)
		{
			UseSingleLayerIndex = FMath::Clamp(UseSingleLayerIndex, 0, TileMap->TileLayers.Num() - 1);
		}
		else
		{
			UseSingleLayerIndex = 0;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

UBodySetup* UPaperTileMapComponent::GetBodySetup()
{
	return (TileMap != nullptr) ? TileMap->BodySetup : nullptr;
}

void UPaperTileMapComponent::GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel)
{
	// Get the texture referenced by the tile maps
	if (TileMap != nullptr)
	{
		for (UPaperTileLayer* Layer : TileMap->TileLayers)
		{
			for (int32 Y = 0; Y < TileMap->MapHeight; ++Y)
			{
				for (int32 X = 0; X < TileMap->MapWidth; ++X)
				{
					FPaperTileInfo TileInfo = Layer->GetCell(X, Y);
					if (TileInfo.IsValid() && (TileInfo.TileSet != nullptr))
					{
						if (UTexture2D* TileSheet = TileInfo.TileSet->GetTileSheetTexture())
						{
							OutTextures.AddUnique(TileSheet);
						}
					}
				}
			}
		}
	}
		
	// Get any textures referenced by our materials
	Super::GetUsedTextures(OutTextures, QualityLevel);
}

UMaterialInterface* UPaperTileMapComponent::GetMaterial(int32 MaterialIndex) const
{
	if (OverrideMaterials.IsValidIndex(MaterialIndex) && (OverrideMaterials[MaterialIndex] != nullptr))
	{
		return OverrideMaterials[MaterialIndex];
	}
	else if (TileMap != nullptr)
	{
		return TileMap->Material;
	}

	return nullptr;
}

int32 UPaperTileMapComponent::GetNumMaterials() const
{
	return FMath::Max<int32>(OverrideMaterials.Num(), 1);
}

const UObject* UPaperTileMapComponent::AdditionalStatObject() const
{
	if (TileMap != nullptr)
	{
		if (TileMap->GetOuter() != this)
		{
			return TileMap;
		}
	}

	return nullptr;
}

void UPaperTileMapComponent::RebuildRenderData(TArray<FSpriteRenderSection>& Sections, TArray<FDynamicMeshVertex>& Vertices)
{
	if (TileMap == nullptr)
	{
		return;
	}

	// Handles the rotation and flipping of UV coordinates in a tile
	// 0123 = BL BR TR TL
	const static uint8 PermutationTable[8][4] = 
	{
		{0, 1, 2, 3}, // 000 - normal
		{2, 1, 0, 3}, // 001 - diagonal
		{3, 2, 1, 0}, // 010 - flip Y
		{3, 0, 1, 2}, // 011 - diagonal then flip Y
		{1, 0, 3, 2}, // 100 - flip X
		{1, 2, 3, 0}, // 101 - diagonal then flip X
		{2, 3, 0, 1}, // 110 - flip X and flip Y
		{0, 3, 2, 1}  // 111 - diagonal then flip X and Y
	};

	FVector CornerOffset;
	FVector OffsetYFactor;
	FVector StepPerTileX;
	FVector StepPerTileY;
	TileMap->GetTileToLocalParameters(/*out*/ CornerOffset, /*out*/ StepPerTileX, /*out*/ StepPerTileY, /*out*/ OffsetYFactor);
	
	UTexture2D* LastSourceTexture = nullptr;
	FVector TileSetOffset = FVector::ZeroVector;
	FVector2f InverseTextureSize(1.0f, 1.0f);
	FVector2f SourceDimensionsUV(1.0f, 1.0f);
	FVector2f TileSizeXY(0.0f, 0.0f);

	const float UnrealUnitsPerPixel = TileMap->GetUnrealUnitsPerPixel();


	// Run thru the layers and estimate how big of an allocation we will need
	int32 EstimatedNumVerts = 0;

	for (int32 Z = TileMap->TileLayers.Num() - 1; Z >= 0; --Z)
	{
		UPaperTileLayer* Layer = TileMap->TileLayers[Z];

		if ((Layer != nullptr) && (!bUseSingleLayer || (Z == UseSingleLayerIndex)))
		{
			const int32 NumOccupiedCells = Layer->GetNumOccupiedCells();
			EstimatedNumVerts += 6 * NumOccupiedCells;
		}
	}

	Vertices.Empty(EstimatedNumVerts);

	UMaterialInterface* TileMapMaterial = GetMaterial(0);
	if (TileMapMaterial == nullptr)
	{
		TileMapMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}

	// Actual pass
	for (int32 Z = TileMap->TileLayers.Num() - 1; Z >= 0; --Z)
	{
		UPaperTileLayer* Layer = TileMap->TileLayers[Z];

		if (Layer == nullptr)
		{
			continue;
		}

		if (bUseSingleLayer)
		{
			if (Z != UseSingleLayerIndex)
			{
				continue;
			}
		}

		const FLinearColor DrawColorLinear = TileMapColor * Layer->GetLayerColor();
		const FColor DrawColor(DrawColorLinear.ToFColor(/*bSRGB=*/ false));

#if WITH_EDITORONLY_DATA
		if (!Layer->ShouldRenderInEditor())
		{
			continue;
		}
#endif

		FSpriteRenderSection* CurrentBatch = nullptr;
		FVector CurrentDestinationOrigin;
		const FPaperTileInfo* CurrentCellPtr = Layer->PRIVATE_GetAllocatedCells();
		check(Layer->GetLayerWidth() == TileMap->MapWidth);
		check(Layer->GetLayerHeight() == TileMap->MapHeight);


		for (int32 Y = 0; Y < TileMap->MapHeight; ++Y)
		{
			// In pixels
			FVector EffectiveTopLeftCorner;

			switch (TileMap->ProjectionMode)
			{
			case ETileMapProjectionMode::Orthogonal:
			default:
				EffectiveTopLeftCorner = CornerOffset;
				break;
			case ETileMapProjectionMode::IsometricDiamond:
				EffectiveTopLeftCorner = CornerOffset - 0.5f * StepPerTileX + 0.5f * StepPerTileY;
				break;
			case ETileMapProjectionMode::IsometricStaggered:
			case ETileMapProjectionMode::HexagonalStaggered:
				EffectiveTopLeftCorner = CornerOffset + (Y & 1) * OffsetYFactor;
				break;
			}

			for (int32 X = 0; X < TileMap->MapWidth; ++X)
			{
				const FPaperTileInfo& TileInfo = *CurrentCellPtr++;

				// do stuff
				const float TotalSeparation = (TileMap->SeparationPerLayer * Z) + (TileMap->SeparationPerTileX * X) + (TileMap->SeparationPerTileY * Y);
				FVector TopLeftCornerOfTile = (StepPerTileX * X) + (StepPerTileY * Y) + EffectiveTopLeftCorner;
				TopLeftCornerOfTile += TotalSeparation * PaperAxisZ;

				const int32 TileWidth = TileMap->TileWidth;
				const int32 TileHeight = TileMap->TileHeight;


				{
					UTexture2D* SourceTexture = nullptr;

					FVector2D SourceUVDbl = FVector2D::ZeroVector;	// LWC_TODO: Rework Paper2D API to support both FVector2f and FVector2d?

					if (TileInfo.TileSet == nullptr)
					{
						continue;
					}

					if (!TileInfo.TileSet->GetTileUV(TileInfo.GetTileIndex(), /*out*/ SourceUVDbl))
					{
						continue;
					}

					SourceTexture = TileInfo.TileSet->GetTileSheetTexture();
					if (SourceTexture == nullptr)
					{
						continue;
					}

					FVector2f SourceUV = static_cast<FVector2f>(SourceUVDbl);

					if ((SourceTexture != LastSourceTexture) || (CurrentBatch == nullptr))
					{
						CurrentBatch = new (Sections) FSpriteRenderSection();
						CurrentBatch->BaseTexture = SourceTexture;
						CurrentBatch->AdditionalTextures = TileInfo.TileSet->GetAdditionalTextures(); 
						//@TODO: Not checking the AdditionalTextures array to see if it's changed to break a batch (almost always going to be fine to skip it as the same base texture is fairly unlikely to be shared with different additional textures)
						CurrentBatch->Material = TileMapMaterial;
						CurrentBatch->VertexOffset = Vertices.Num();
						CurrentDestinationOrigin = TopLeftCornerOfTile.ProjectOnTo(PaperAxisZ);
					}

					if (SourceTexture != LastSourceTexture)
					{
						const FVector2f TextureSize(SourceTexture->GetImportedSize());
						InverseTextureSize = FVector2f(1.0f / TextureSize.X, 1.0f / TextureSize.Y);

						if (TileInfo.TileSet != nullptr)
						{
							const FIntPoint TileSetTileSize(TileInfo.TileSet->GetTileSize());

							SourceDimensionsUV = FVector2f(TileSetTileSize.X * InverseTextureSize.X, TileSetTileSize.Y * InverseTextureSize.Y);
							TileSizeXY = FVector2f(UnrealUnitsPerPixel * TileSetTileSize.X, UnrealUnitsPerPixel * TileSetTileSize.Y);

							const FIntPoint TileSetDrawingOffset = TileInfo.TileSet->GetDrawingOffset();
							const float HorizontalCellOffset = TileSetDrawingOffset.X * UnrealUnitsPerPixel;
							const float VerticalCellOffset = (-TileSetDrawingOffset.Y - TileHeight + TileSetTileSize.Y) * UnrealUnitsPerPixel;
							TileSetOffset = (HorizontalCellOffset * PaperAxisX) + (VerticalCellOffset * PaperAxisY);
						}
						else
						{
							SourceDimensionsUV = FVector2f(TileWidth * InverseTextureSize.X, TileHeight * InverseTextureSize.Y);
							TileSizeXY = FVector2f(UnrealUnitsPerPixel * TileWidth, UnrealUnitsPerPixel * TileHeight);
							TileSetOffset = FVector::ZeroVector;
						}
						LastSourceTexture = SourceTexture;
					}
					TopLeftCornerOfTile += TileSetOffset;

					SourceUV.X *= InverseTextureSize.X;
					SourceUV.Y *= InverseTextureSize.Y;

					const float WX0 = FVector::DotProduct(TopLeftCornerOfTile, PaperAxisX);
					const float WY0 = FVector::DotProduct(TopLeftCornerOfTile, PaperAxisY);

					const int32 Flags = TileInfo.GetFlagsAsIndex();

					const FVector2f TileSizeWithFlip = TileInfo.HasFlag(EPaperTileFlags::FlipDiagonal) ? FVector2f(TileSizeXY.Y, TileSizeXY.X) : TileSizeXY;
					const float UValues[4] = { SourceUV.X, SourceUV.X + SourceDimensionsUV.X, SourceUV.X + SourceDimensionsUV.X, SourceUV.X };
					const float VValues[4] = { SourceUV.Y + SourceDimensionsUV.Y, SourceUV.Y + SourceDimensionsUV.Y, SourceUV.Y, SourceUV.Y };

					const uint8 UVIndex0 = PermutationTable[Flags][0];
					const uint8 UVIndex1 = PermutationTable[Flags][1];
					const uint8 UVIndex2 = PermutationTable[Flags][2];
					const uint8 UVIndex3 = PermutationTable[Flags][3];

					const FVector4f BottomLeft(WX0, WY0 - TileSizeWithFlip.Y, UValues[UVIndex0], VValues[UVIndex0]);
					const FVector4f BottomRight(WX0 + TileSizeWithFlip.X, WY0 - TileSizeWithFlip.Y, UValues[UVIndex1], VValues[UVIndex1]);
					const FVector4f TopRight(WX0 + TileSizeWithFlip.X, WY0, UValues[UVIndex2], VValues[UVIndex2]);
					const FVector4f TopLeft(WX0, WY0, UValues[UVIndex3], VValues[UVIndex3]);

					CurrentBatch->AddVertex(BottomRight.X, BottomRight.Y, BottomRight.Z, BottomRight.W, CurrentDestinationOrigin, DrawColor, Vertices);
					CurrentBatch->AddVertex(TopRight.X, TopRight.Y, TopRight.Z, TopRight.W, CurrentDestinationOrigin, DrawColor, Vertices);
					CurrentBatch->AddVertex(BottomLeft.X, BottomLeft.Y, BottomLeft.Z, BottomLeft.W, CurrentDestinationOrigin, DrawColor, Vertices);

					CurrentBatch->AddVertex(TopRight.X, TopRight.Y, TopRight.Z, TopRight.W, CurrentDestinationOrigin, DrawColor, Vertices);
					CurrentBatch->AddVertex(TopLeft.X, TopLeft.Y, TopLeft.Z, TopLeft.W, CurrentDestinationOrigin, DrawColor, Vertices);
					CurrentBatch->AddVertex(BottomLeft.X, BottomLeft.Y, BottomLeft.Z, BottomLeft.W, CurrentDestinationOrigin, DrawColor, Vertices);
				}
			}
		}
	}

#if WITH_EDITOR
	NumBatches = Sections.Num();
	NumTriangles = Vertices.Num() / 3;
#endif
}

void UPaperTileMapComponent::CreateNewOwnedTileMap()
{
	TGuardValue<TEnumAsByte<EComponentMobility::Type>> MobilitySaver(Mobility, EComponentMobility::Movable);

	UPaperTileMap* NewTileMap = NewObject<UPaperTileMap>(this);
	NewTileMap->SetFlags(RF_Transactional);
	NewTileMap->InitializeNewEmptyTileMap();

	SetTileMap(NewTileMap);
}

void UPaperTileMapComponent::CreateNewTileMap(int32 MapWidth, int32 MapHeight, int32 TileWidth, int32 TileHeight, float PixelsPerUnrealUnit, bool bCreateLayer)
{
	TGuardValue<TEnumAsByte<EComponentMobility::Type>> MobilitySaver(Mobility, EComponentMobility::Movable);

	UPaperTileMap* NewTileMap = NewObject<UPaperTileMap>(this);
	NewTileMap->SetFlags(RF_Transactional);
	NewTileMap->MapWidth = MapWidth;
	NewTileMap->MapHeight = MapHeight;
	NewTileMap->TileWidth = TileWidth;
	NewTileMap->TileHeight = TileHeight;
	NewTileMap->PixelsPerUnrealUnit = PixelsPerUnrealUnit;

	if (bCreateLayer)
	{
		NewTileMap->AddNewLayer();
	}

	SetTileMap(NewTileMap);
}

bool UPaperTileMapComponent::OwnsTileMap() const
{
	return (TileMap != nullptr) && (TileMap->GetOuter() == this);
}

bool UPaperTileMapComponent::SetTileMap(class UPaperTileMap* NewTileMap)
{
	if (NewTileMap != TileMap)
	{
		// Don't allow changing the tile map if we are "static".
		AActor* ComponentOwner = GetOwner();
		if ((ComponentOwner == nullptr) || AreDynamicDataChangesAllowed())
		{
			TileMap = NewTileMap;

			// Need to send this to render thread at some point
			MarkRenderStateDirty();

			// Update physics representation right away
			RecreatePhysicsState();

			// Since we have new mesh, we need to update bounds
			UpdateBounds();

			return true;
		}
	}

	return false;
}

void UPaperTileMapComponent::GetMapSize(int32& MapWidth, int32& MapHeight, int32& NumLayers)
{
	if (TileMap != nullptr)
	{
		MapWidth = TileMap->MapWidth;
		MapHeight = TileMap->MapHeight;
		NumLayers = TileMap->TileLayers.Num();
	}
	else
	{
		MapWidth = 1;
		MapHeight = 1;
		NumLayers = 1;
	}
}

FPaperTileInfo UPaperTileMapComponent::GetTile(int32 X, int32 Y, int32 Layer) const
{
	FPaperTileInfo Result;
	if (TileMap != nullptr)
	{
		if (TileMap->TileLayers.IsValidIndex(Layer))
		{
			Result = TileMap->TileLayers[Layer]->GetCell(X, Y);
		}
	}

	return Result;
}

void UPaperTileMapComponent::SetTile(int32 X, int32 Y, int32 Layer, FPaperTileInfo NewValue)
{
	if (OwnsTileMap())
	{
		if (TileMap->TileLayers.IsValidIndex(Layer))
		{
			TileMap->TileLayers[Layer]->SetCell(X, Y, NewValue);

			MarkRenderStateDirty();
		}
		else
		{
			UE_LOGF(LogPaper2D, Warning, "Invalid layer index %d for %ls", Layer, *TileMap->GetPathName());
		}
	}
}

void UPaperTileMapComponent::ResizeMap(int32 NewWidthInTiles, int32 NewHeightInTiles)
{
	if (OwnsTileMap())
	{
		TileMap->ResizeMap(NewWidthInTiles, NewHeightInTiles);
		
		MarkRenderStateDirty();
		RecreatePhysicsState();
		UpdateBounds();
	}
}

UPaperTileLayer* UPaperTileMapComponent::AddNewLayer()
{
	UPaperTileLayer* Result = nullptr;

	if (OwnsTileMap())
	{
		Result = TileMap->AddNewLayer();

		MarkRenderStateDirty();
		RecreatePhysicsState();
		UpdateBounds();
	}

	return Result;
}

FLinearColor UPaperTileMapComponent::GetTileMapColor() const
{
	return TileMapColor;
}

void UPaperTileMapComponent::SetTileMapColor(FLinearColor NewColor)
{
	TileMapColor = NewColor;
	MarkRenderStateDirty();
}

FLinearColor UPaperTileMapComponent::GetLayerColor(int32 Layer) const
{
	if ((TileMap != nullptr) && TileMap->TileLayers.IsValidIndex(Layer))
	{
		return TileMap->TileLayers[Layer]->GetLayerColor();
	}
	else
	{
		return FLinearColor::White;
	}
}

void UPaperTileMapComponent::SetLayerColor(FLinearColor NewColor, int32 Layer)
{
	if (OwnsTileMap())
	{
		if (TileMap->TileLayers.IsValidIndex(Layer))
		{
			TileMap->TileLayers[Layer]->SetLayerColor(NewColor);
			MarkRenderStateDirty();
		}
	}
}

FLinearColor UPaperTileMapComponent::GetWireframeColor() const
{
	return TileMapColor;
}

void UPaperTileMapComponent::MakeTileMapEditable()
{
	if ((TileMap != nullptr) && !OwnsTileMap())
	{
		SetTileMap(TileMap->CloneTileMap(this));
	}
}

#if WITH_EDITOR
void UPaperTileMapComponent::GetRenderingStats(int32& OutNumTriangles, int32& OutNumBatches) const
{
	OutNumBatches = NumBatches;
	OutNumTriangles = NumTriangles;
}
#endif

FVector UPaperTileMapComponent::GetTileCornerPosition(int32 TileX, int32 TileY, int32 LayerIndex, bool bWorldSpace) const
{
	FVector Result(ForceInitToZero);

	if (TileMap != nullptr)
	{
		Result = TileMap->GetTilePositionInLocalSpace(TileX, TileY, LayerIndex);
	}

	if (bWorldSpace)
	{
		Result = GetComponentTransform().TransformPosition(Result);
	}
	return Result;
}

FVector UPaperTileMapComponent::GetTileCenterPosition(int32 TileX, int32 TileY, int32 LayerIndex, bool bWorldSpace) const
{
	FVector Result(ForceInitToZero);

	if (TileMap != nullptr)
	{
		Result = TileMap->GetTileCenterInLocalSpace(TileX, TileY, LayerIndex);
	}

	if (bWorldSpace)
	{
		Result = GetComponentTransform().TransformPosition(Result);
	}
	return Result;
}

void UPaperTileMapComponent::GetTilePolygon(int32 TileX, int32 TileY, TArray<FVector>& Points, int32 LayerIndex, bool bWorldSpace) const
{
	Points.Reset();

	if (TileMap != nullptr)
	{
		TileMap->GetTilePolygon(TileX, TileY, LayerIndex, /*out*/ Points);
	}

	if (bWorldSpace)
	{
		const FTransform& ComponentTransform = GetComponentTransform();
		for (FVector& Point : Points)
		{
			Point = ComponentTransform.TransformPosition(Point);
		}
	}
}

void UPaperTileMapComponent::SetDefaultCollisionThickness(float Thickness, bool bRebuildCollision)
{
	if (OwnsTileMap())
	{
		TileMap->SetCollisionThickness(Thickness);

		if (bRebuildCollision)
		{
			RebuildCollision();
		}
	}
}

void UPaperTileMapComponent::SetLayerCollision(int32 Layer, bool bHasCollision, bool bOverrideThickness, float CustomThickness, bool bOverrideOffset, float CustomOffset, bool bRebuildCollision)
{
	if (OwnsTileMap())
	{
		if (TileMap->TileLayers.IsValidIndex(Layer))
		{
			UPaperTileLayer* TileLayer = TileMap->TileLayers[Layer];

			TileLayer->SetLayerCollides(bHasCollision);
			TileLayer->SetLayerCollisionThickness(bOverrideThickness, CustomThickness);
			TileLayer->SetLayerCollisionOffset(bOverrideOffset, CustomThickness);

			if (bRebuildCollision)
			{
				RebuildCollision();
			}
		}
		else
		{
			UE_LOGF(LogPaper2D, Warning, "Invalid layer index %d for %ls", Layer, *GetPathNameSafe(TileMap));
		}
	}
}

void UPaperTileMapComponent::RebuildCollision()
{
	if (OwnsTileMap())
	{
		TileMap->RebuildCollision();
	}

	RecreatePhysicsState();
	UpdateBounds();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

