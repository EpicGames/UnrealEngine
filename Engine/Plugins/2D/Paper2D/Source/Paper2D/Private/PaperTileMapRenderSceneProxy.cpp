// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileMapRenderSceneProxy.h"
#include "Materials/Material.h"
#include "SceneManagement.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/Engine.h"
#include "Materials/MaterialRenderProxy.h"
#include "PaperTileMap.h"
#include "PaperTileMapComponent.h"
#include "SceneInterface.h"

#include "SceneView.h"

DECLARE_CYCLE_STAT(TEXT("Tile Map Proxy"), STAT_TileMap_GetDynamicMeshElements, STATGROUP_Paper2D);
DECLARE_CYCLE_STAT(TEXT("Tile Map Editor Grid"), STAT_TileMap_EditorWireDrawing, STATGROUP_Paper2D);

//////////////////////////////////////////////////////////////////////////
// FPaperTileMapRenderSceneProxy

FPaperTileMapRenderSceneProxy::FPaperTileMapRenderSceneProxy(const UPaperTileMapComponent* InComponent)
	: FPaperRenderSceneProxy(InComponent)
#if WITH_EDITOR
	, bShowPerTileGridWhenSelected(false)
	, bShowPerTileGridWhenUnselected(false)
	, bShowPerLayerGridWhenSelected(false)
	, bShowPerLayerGridWhenUnselected(false)
	, bShowOutlineWhenUnselected(false)
#endif
	, TileMap(nullptr)
	, OnlyLayerIndex(InComponent->bUseSingleLayer ? InComponent->UseSingleLayerIndex : INDEX_NONE)
	, WireDepthBias(0.0001f)
{
	check(InComponent);

	SetWireframeColor(InComponent->GetWireframeColor());
	TileMap = InComponent->TileMap;
	MaterialRelevance = InComponent->GetMaterialRelevance(GetScene().GetShaderPlatform());

#if WITH_EDITORONLY_DATA
	bShowPerTileGridWhenSelected = InComponent->bShowPerTileGridWhenSelected;
	bShowPerTileGridWhenUnselected = InComponent->bShowPerTileGridWhenUnselected;
	bShowPerLayerGridWhenSelected = InComponent->bShowPerLayerGridWhenSelected;
	bShowPerLayerGridWhenUnselected = InComponent->bShowPerLayerGridWhenUnselected;
	bShowOutlineWhenUnselected = InComponent->bShowOutlineWhenUnselected;
#endif
}

SIZE_T FPaperTileMapRenderSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FPaperTileMapRenderSceneProxy* FPaperTileMapRenderSceneProxy::CreateTileMapProxy(const UPaperTileMapComponent* InComponent, TArray<FSpriteRenderSection>*& OutSections, TArray<FDynamicMeshVertex>*& OutVertices)
{
	FPaperTileMapRenderSceneProxy* NewProxy = new FPaperTileMapRenderSceneProxy(InComponent);

	OutVertices = &(NewProxy->Vertices);
	OutSections = &(NewProxy->BatchedSections);

	return NewProxy;
}

void FPaperTileMapRenderSceneProxy::FinishConstruction_GameThread()
{
}

void FPaperTileMapRenderSceneProxy::DrawBoundsForLayer(FPrimitiveDrawInterface* PDI, const FLinearColor& Color, int32 LayerIndex) const
{
	const FMatrix& LocalToWorldMat = GetLocalToWorld();
	const FVector TL(LocalToWorldMat.TransformPosition(TileMap->GetTilePositionInLocalSpace(0, 0, LayerIndex)));
	const FVector TR(LocalToWorldMat.TransformPosition(TileMap->GetTilePositionInLocalSpace(TileMap->MapWidth, 0, LayerIndex)));
	const FVector BL(LocalToWorldMat.TransformPosition(TileMap->GetTilePositionInLocalSpace(0, TileMap->MapHeight, LayerIndex)));
	const FVector BR(LocalToWorldMat.TransformPosition(TileMap->GetTilePositionInLocalSpace(TileMap->MapWidth, TileMap->MapHeight, LayerIndex)));

	PDI->DrawLine(TL, TR, Color, SDPG_Foreground, 0.0f, WireDepthBias);
	PDI->DrawLine(TR, BR, Color, SDPG_Foreground, 0.0f, WireDepthBias);
	PDI->DrawLine(BR, BL, Color, SDPG_Foreground, 0.0f, WireDepthBias);
	PDI->DrawLine(BL, TL, Color, SDPG_Foreground, 0.0f, WireDepthBias);
}

void FPaperTileMapRenderSceneProxy::DrawNormalGridLines(FPrimitiveDrawInterface* PDI, const FLinearColor& PerTileColor, const FLinearColor& MultiTileColor, int32 MultiTileGridWidth, int32 MultiTileGridHeight, int32 MultiTileGridOffsetX, int32 MultiTileGridOffsetY, int32 LayerIndex) const
{
	FLinearColor Color = PerTileColor;
	const FMatrix& LocalToWorldMat = GetLocalToWorld();
	const uint8 DPG = SDPG_Foreground;//GetDepthPriorityGroup(View);

	// Draw horizontal lines on the selection
	for (int32 Y = 0; Y <= TileMap->MapHeight; ++Y)
	{
		int32 X = 0;
		const FVector Start(TileMap->GetTilePositionInLocalSpace(X, Y, LayerIndex));

		X = TileMap->MapWidth;
		const FVector End(TileMap->GetTilePositionInLocalSpace(X, Y, LayerIndex));

		if (MultiTileGridHeight > 0 && int32(Y - MultiTileGridOffsetY) % int32(MultiTileGridHeight) == 0)
		{
			Color = MultiTileColor;
		}
		else
		{
			Color = PerTileColor;
		}

		PDI->DrawLine(LocalToWorldMat.TransformPosition(Start), LocalToWorldMat.TransformPosition(End), Color, DPG, 0.0f, WireDepthBias);
	}

	// Draw vertical lines
	for (int32 X = 0; X <= TileMap->MapWidth; ++X)
	{
		int32 Y = 0;
		const FVector Start(TileMap->GetTilePositionInLocalSpace(X, Y, LayerIndex));

		Y = TileMap->MapHeight;
		const FVector End(TileMap->GetTilePositionInLocalSpace(X, Y, LayerIndex));

		if (MultiTileGridWidth > 0 && int32(X - MultiTileGridOffsetX) % int32(MultiTileGridWidth) == 0)
		{
			Color = MultiTileColor;
		}
		else
		{
			Color = PerTileColor;
		}

		PDI->DrawLine(LocalToWorldMat.TransformPosition(Start), LocalToWorldMat.TransformPosition(End), Color, DPG, 0.0f, WireDepthBias);
	}
}

void FPaperTileMapRenderSceneProxy::DrawStaggeredGridLines(FPrimitiveDrawInterface* PDI, const FLinearColor& PerTileColor, const FLinearColor& MultiTileColor, int32 MultiTileGridWidth, int32 MultiTileGridHeight, int32 MultiTileGridOffsetX, int32 MultiTileGridOffsetY, int32 LayerIndex) const
{
	FLinearColor Color = PerTileColor;
	TArray<FVector> Poly;
	Poly.Empty(4);

	const FMatrix& LocalToWorldMat = GetLocalToWorld();
	const uint8 DPG = SDPG_Foreground;//GetDepthPriorityGroup(View);

	FVector CornerPosition;
	FVector OffsetYFactor;
	FVector StepX;
	FVector StepY;

	TileMap->GetTileToLocalParameters(/*out*/ CornerPosition, /*out*/ StepX, /*out*/ StepY, /*out*/ OffsetYFactor);

	const FVector PartialZ = (TileMap->SeparationPerLayer * LayerIndex) * PaperAxisZ;
	const FVector TotalOffset = CornerPosition + PartialZ;

	const bool bStaggerEven = false;

	const FVector TopCenterStart = TotalOffset + StepX * 0.5f + (StepX * -0.5f) + StepY;
	for (int32 X = 0-((TileMap->MapHeight+1)/2); X < TileMap->MapWidth; ++X)
	{
		int32 XTop = FMath::Max(X, 0);
		int32 YTop = FMath::Max(-2 * X, 0);

		if (X < 0)
		{
			XTop--;
			YTop--;
		}

		// A is top of center top row cell
		Poly.Reset();
		TileMap->GetTilePolygon(XTop, YTop, LayerIndex, Poly);
		const FVector LSA = Poly[0];

		// Determine the bottom row cell
		int32 YBottom = TileMap->MapHeight - 1;
		int32 XBottom = X + (TileMap->MapHeight +  1) / 2;
		const int32 XExcess = FMath::Max(XBottom - TileMap->MapWidth, 0);
		XBottom -= XExcess;
		YBottom -= XExcess * 2;

		if (XBottom == TileMap->MapWidth)
		{
			YBottom -= ((TileMap->MapHeight & 1) != 0) ? 0 : 1;
		}

		// Bottom center
		Poly.Reset();
		TileMap->GetTilePolygon(XBottom, YBottom, LayerIndex, Poly);
		const FVector LSB = Poly[2];

		if (MultiTileGridHeight > 0 && int32(X - MultiTileGridOffsetY) % int32(MultiTileGridHeight) == 0)
		{
			Color = MultiTileColor;
		}
		else
		{
			Color = PerTileColor;
		}

		PDI->DrawLine(LocalToWorldMat.TransformPosition(LSA), LocalToWorldMat.TransformPosition(LSB), Color, DPG, 0.0f, WireDepthBias);
	}

	for (int32 X = 0; X < TileMap->MapWidth + ((TileMap->MapHeight + 1) / 2) + 1; ++X)
	{
		const int32 XTop = FMath::Min(X, TileMap->MapWidth);
		const int32 YTop = FMath::Max(2 * (X - TileMap->MapWidth), 0);

		// A is top center of top row cell
		Poly.Reset();
		TileMap->GetTilePolygon(XTop, YTop, LayerIndex, Poly);
		const FVector LSA = Poly[0];

		// Determine the bottom row cell
		int32 YBottom = TileMap->MapHeight;
		int32 XBottom = X - ((TileMap->MapHeight+1) / 2);
		const int32 XExcess = FMath::Max(-XBottom, 0);
		XBottom += XExcess;
		YBottom -= XExcess * 2;

		if (XExcess > 0)
		{
			YBottom += (TileMap->MapHeight & 1);
		}

		// Bottom left
		Poly.Reset();
		TileMap->GetTilePolygon(XBottom, YBottom, LayerIndex, Poly);
		const FVector LSB = Poly[3];

		if (MultiTileGridWidth > 0 && int32(X - MultiTileGridOffsetX) % int32(MultiTileGridWidth) == 0)
		{
			Color = MultiTileColor;
		}
		else
		{
			Color = PerTileColor;
		}

		PDI->DrawLine(LocalToWorldMat.TransformPosition(LSA), LocalToWorldMat.TransformPosition(LSB), Color, DPG, 0.0f, WireDepthBias);
	}
}

void FPaperTileMapRenderSceneProxy::DrawHexagonalGridLines(FPrimitiveDrawInterface* PDI, const FLinearColor& PerTileColor, const FLinearColor& MultiTileColor, int32 MultiTileGridWidth, int32 MultiTileGridHeight, int32 MultiTileGridOffsetX, int32 MultiTileGridOffsetY, int32 LayerIndex) const
{
	FLinearColor Color = FLinearColor::White;
	//@TODO: This isn't very efficient
	const FMatrix& LocalToWorldMat = GetLocalToWorld();
	const uint8 DPG = SDPG_Foreground;//GetDepthPriorityGroup(View);

	TArray<FVector> Poly;
	Poly.Empty(6);
	for (int32 Y = 0; Y < TileMap->MapHeight; ++Y)
	{
		for (int32 X = 0; X < TileMap->MapWidth; ++X)
		{
			Poly.Reset();
			TileMap->GetTilePolygon(X, Y, LayerIndex, Poly);

			FVector LastVertexWS = LocalToWorldMat.TransformPosition(Poly[5]);
			for (int32 VI = 0; VI < Poly.Num(); ++VI)
			{
				FVector ThisVertexWS = LocalToWorldMat.TransformPosition(Poly[VI]);
				PDI->DrawLine(LastVertexWS, ThisVertexWS, Color, DPG, 0.0f, WireDepthBias);
				LastVertexWS = ThisVertexWS;
			}
		}
	}
}

void FPaperTileMapRenderSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	SCOPE_CYCLE_COUNTER(STAT_TileMap_GetDynamicMeshElements);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		SCOPE_CYCLE_COUNTER(STAT_TileMap_EditorWireDrawing);

		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

			// Draw the tile maps
			//@TODO: RenderThread race condition
			if (TileMap != nullptr)
			{
				if ((View->Family->EngineShowFlags.Collision /*@TODO: && bIsCollisionEnabled*/) && AllowDebugViewmodes())
				{
					if (UBodySetup* BodySetup = TileMap->BodySetup)
					{
						if (FMath::Abs(GetLocalToWorld().Determinant()) < SMALL_NUMBER)
						{
							// Catch this here or otherwise GeomTransform below will assert
							// This spams so commented out
							//UE_LOGF(LogStaticMesh, Log, "Zero scaling not supported (%ls)", *StaticMesh->GetPathName());
						}
						else
						{
							// Make a material for drawing solid collision stuff
							const UMaterial* LevelColorationMaterial = View->Family->EngineShowFlags.Lighting
								? GEngine->ShadedLevelColorationLitMaterial : GEngine->ShadedLevelColorationUnlitMaterial;

							auto CollisionMaterialInstance = new FColoredMaterialRenderProxy(
								LevelColorationMaterial->GetRenderProxy(),
								GetWireframeColor()
								);

							// Draw the static mesh's body setup.

							// Get transform without scaling.
							FTransform GeomTransform(GetLocalToWorld());

							// In old wireframe collision mode, always draw the wireframe highlighted (selected or not).
							bool bDrawWireSelected = IsSelected();
							if (View->Family->EngineShowFlags.Collision)
							{
								bDrawWireSelected = true;
							}

							// Differentiate the color based on bBlockNonZeroExtent.  Helps greatly with skimming a level for optimization opportunities.
							FColor CollisionColor = FColor(157, 149, 223, 255);

							const bool bPerHullColor = false;
							const bool bDrawSimpleSolid = false;
							BodySetup->AggGeom.GetAggGeom(GeomTransform, GetSelectionColor(CollisionColor, bDrawWireSelected, IsHovered()).ToFColor(true), CollisionMaterialInstance, bPerHullColor, bDrawSimpleSolid, AlwaysHasVelocity(), ViewIndex, Collector);
						}
					}
				}

				// Draw the bounds
				RenderBounds(PDI, View->Family->EngineShowFlags, GetBounds(), IsSelected());

#if WITH_EDITOR
				const bool bShowAsSelected = IsSelected();
				const bool bEffectivelySelected = bShowAsSelected || IsHovered();

				const uint8 DPG = SDPG_Foreground;//GetDepthPriorityGroup(View);


				// Draw the debug outline
				if (bEffectivelySelected)
				{
					const int32 SelectedLayerIndex = (OnlyLayerIndex != INDEX_NONE) ? OnlyLayerIndex : TileMap->SelectedLayerIndex;

					// Draw separation wires if selected
					const FLinearColor OverrideColor = GetSelectionColor(FLinearColor::White, bShowAsSelected, IsHovered(), /*bUseOverlayIntensity=*/ false);

					if (bShowPerLayerGridWhenSelected)
					{
						if (OnlyLayerIndex == INDEX_NONE)
						{
							// Draw a bound for every layer but the selected one (and even that one if the per-tile grid is off)
							for (int32 LayerIndex = 0; LayerIndex < TileMap->TileLayers.Num(); ++LayerIndex)
							{
								if ((LayerIndex != SelectedLayerIndex) || !bShowPerTileGridWhenSelected)
								{
									DrawBoundsForLayer(PDI, OverrideColor, LayerIndex);
								}
							}
						}
						else if (!bShowPerTileGridWhenSelected)
						{
							DrawBoundsForLayer(PDI, OverrideColor, OnlyLayerIndex);
						}
					}

					if (bShowPerTileGridWhenSelected && (SelectedLayerIndex != INDEX_NONE))
					{
						switch (TileMap->ProjectionMode)
						{
						default:
						case ETileMapProjectionMode::Orthogonal:
						case ETileMapProjectionMode::IsometricDiamond:
							DrawNormalGridLines(PDI, OverrideColor, OverrideColor, 0, 0, 0, 0, SelectedLayerIndex);
							break;
						case ETileMapProjectionMode::IsometricStaggered:
							DrawStaggeredGridLines(PDI, OverrideColor, OverrideColor, 0, 0, 0, 0, SelectedLayerIndex);
							break;
						case ETileMapProjectionMode::HexagonalStaggered:
							DrawHexagonalGridLines(PDI, OverrideColor, OverrideColor, 0, 0, 0, 0, SelectedLayerIndex);
							break;
						}
					}
				}
				else if (View->Family->EngineShowFlags.Grid)
				{
					const int32 SelectedLayerIndex = (OnlyLayerIndex != INDEX_NONE) ? OnlyLayerIndex : TileMap->SelectedLayerIndex;

					if (bShowOutlineWhenUnselected)
					{
						// Draw a layer rectangle even when not selected, so you can see where the tile map is in the editor
						DrawBoundsForLayer(PDI, GetWireframeColor(), /*LayerIndex=*/ (OnlyLayerIndex != INDEX_NONE) ? OnlyLayerIndex : 0);
					}
					if (bShowPerLayerGridWhenUnselected)
					{
						const FLinearColor LayerGridColor = TileMap->LayerGridColor;

						if (OnlyLayerIndex == INDEX_NONE)
						{
							// Draw a bound for every layer but the selected one (and even that one if the per-tile grid is off)
							for (int32 LayerIndex = 0; LayerIndex < TileMap->TileLayers.Num(); ++LayerIndex)
							{
								if ((LayerIndex != SelectedLayerIndex) || !bShowPerTileGridWhenUnselected)
								{
									DrawBoundsForLayer(PDI, LayerGridColor, LayerIndex);
								}
							}
						}
						else if (!bShowPerTileGridWhenUnselected)
						{
							DrawBoundsForLayer(PDI, LayerGridColor, OnlyLayerIndex);
						}
					}
					if (bShowPerTileGridWhenUnselected)
					{
						const FLinearColor TileGridColor = TileMap->TileGridColor;
						const FLinearColor MultiTileGridColor = TileMap->MultiTileGridColor;
						int32 MultiTileGridWidth = TileMap->MultiTileGridWidth;
						int32 MultiTileGridHeight = TileMap->MultiTileGridHeight;
						int32 MultiTileGridOffsetX = TileMap->MultiTileGridOffsetX;
						int32 MultiTileGridOffsetY = TileMap->MultiTileGridOffsetY;

						switch (TileMap->ProjectionMode)
						{
						default:
						case ETileMapProjectionMode::Orthogonal:
						case ETileMapProjectionMode::IsometricDiamond:
							DrawNormalGridLines(PDI, TileGridColor, MultiTileGridColor, MultiTileGridWidth, MultiTileGridHeight, MultiTileGridOffsetX, MultiTileGridOffsetY, SelectedLayerIndex);
							break;
						case ETileMapProjectionMode::IsometricStaggered:
							DrawStaggeredGridLines(PDI, TileGridColor, MultiTileGridColor, MultiTileGridWidth, MultiTileGridHeight, MultiTileGridOffsetX, MultiTileGridOffsetY, SelectedLayerIndex);
							break;
						case ETileMapProjectionMode::HexagonalStaggered:
							DrawHexagonalGridLines(PDI, TileGridColor, MultiTileGridColor, MultiTileGridWidth, MultiTileGridHeight, MultiTileGridOffsetX, MultiTileGridOffsetY, SelectedLayerIndex);
							break;
						}
					}
				}
#endif
			}
		}
	}

	// Draw all of the queued up sprites
	FPaperRenderSceneProxy::GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector);
}

