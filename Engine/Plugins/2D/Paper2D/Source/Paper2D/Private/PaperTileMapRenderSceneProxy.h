// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperRenderSceneProxy.h"

class FMeshElementCollector;
class FPrimitiveDrawInterface;
class UPaperTileMap;
class UPaperTileMapComponent;

//////////////////////////////////////////////////////////////////////////
// FPaperTileMapRenderSceneProxy

class FPaperTileMapRenderSceneProxy final : public FPaperRenderSceneProxy
{
public:
	SIZE_T GetTypeHash() const override;

	// FPrimitiveSceneProxy interface.
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	// End of FPrimitiveSceneProxy interface.

	// Construct a tile map scene proxy
	static FPaperTileMapRenderSceneProxy* CreateTileMapProxy(const UPaperTileMapComponent* InComponent, TArray<FSpriteRenderSection>*& OutSections, TArray<FDynamicMeshVertex>*& OutVertices);
	
	// Call this once the tile map sections/vertices are finished
	void FinishConstruction_GameThread();

protected:
	FPaperTileMapRenderSceneProxy(const UPaperTileMapComponent* InComponent);

	void DrawBoundsForLayer(FPrimitiveDrawInterface* PDI, const FLinearColor& Color, int32 LayerIndex) const;
	void DrawNormalGridLines(FPrimitiveDrawInterface* PDI, const FLinearColor& PerTileColor, const FLinearColor& MultiTileColor, int32 MultiTileGridWidth, int32 MultiTileGridHeight, int32 MultiTileGridOffsetX, int32 MultiTileGridOffsetY, int32 LayerIndex) const;
	void DrawStaggeredGridLines(FPrimitiveDrawInterface* PDI, const FLinearColor& PerTileColor, const FLinearColor& MultiTileColor, int32 MultiTileGridWidth, int32 MultiTileGridHeight, int32 MultiTileGridOffsetX, int32 MultiTileGridOffsetY, int32 LayerIndex) const;
	void DrawHexagonalGridLines(FPrimitiveDrawInterface* PDI, const FLinearColor& PerTileColor, const FLinearColor& MultiTileColor, int32 MultiTileGridWidth, int32 MultiTileGridHeight, int32 MultiTileGridOffsetX, int32 MultiTileGridOffsetY, int32 LayerIndex) const;

protected:

#if WITH_EDITOR
	bool bShowPerTileGridWhenSelected;
	bool bShowPerTileGridWhenUnselected;
	bool bShowPerLayerGridWhenSelected;
	bool bShowPerLayerGridWhenUnselected;
	bool bShowOutlineWhenUnselected;
#endif

	//@TODO: Not thread safe
	const UPaperTileMap* TileMap;

	// The only layer to draw, or INDEX_NONE if the filter is unset
	const int32 OnlyLayerIndex;

	// Slight depth bias so that the wireframe grid overlay doesn't z-fight with the tiles themselves
	const float WireDepthBias;
};
