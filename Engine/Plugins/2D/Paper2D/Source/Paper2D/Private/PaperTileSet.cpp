// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileSet.h"
#include "PaperCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileSet)

//////////////////////////////////////////////////////////////////////////
// UPaperTileSet

UPaperTileSet::UPaperTileSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TileSize(32, 32)
{
	TileWidth_DEPRECATED = 32;
	TileHeight_DEPRECATED = 32;

#if WITH_EDITORONLY_DATA
	BackgroundColor = FColor(0, 0, 127);
#endif
}

int32 UPaperTileSet::GetTileCount() const
{
	if (TileSheet != nullptr)
	{
		checkSlow((TileSize.X > 0) && (TileSize.Y > 0));
		const FIntPoint TextureSize = TileSheet->GetImportedSize();
		const int32 TextureWidth = TextureSize.X;
		const int32 TextureHeight = TextureSize.Y;

		const FIntPoint SumMargin = BorderMargin.GetDesiredSize();

		const int32 CellsX = (TextureWidth - SumMargin.X + PerTileSpacing.X) / (TileSize.X + PerTileSpacing.X);
		const int32 CellsY = (TextureHeight - SumMargin.Y + PerTileSpacing.Y) / (TileSize.Y + PerTileSpacing.Y);

		return CellsX * CellsY;
	}
	else
	{
		return 0;
	}
}

int32 UPaperTileSet::GetTileCountX() const
{
	if (TileSheet != nullptr)
	{
		checkSlow(TileSize.X > 0);
		const int32 TextureWidth = TileSheet->GetImportedSize().X;
		const int32 CellsX = (TextureWidth - (BorderMargin.Left + BorderMargin.Right) + PerTileSpacing.X) / (TileSize.X + PerTileSpacing.X);
		return CellsX;
	}
	else
	{
		return 0;
	}
}

int32 UPaperTileSet::GetTileCountY() const
{
	if (TileSheet != nullptr)
	{
		checkSlow(TileSize.Y > 0);
		const int32 TextureHeight = TileSheet->GetImportedSize().Y;
		const int32 CellsY = (TextureHeight - (BorderMargin.Top + BorderMargin.Bottom) + PerTileSpacing.Y) / (TileSize.Y + PerTileSpacing.Y);
		return CellsY;
	}
	else
	{
		return 0;
	}
}

FPaperTileMetadata* UPaperTileSet::GetMutableTileMetadata(int32 TileIndex)
{
	if (PerTileData.IsValidIndex(TileIndex))
	{
		return &(PerTileData[TileIndex]);
	}
	else
	{
		return nullptr;
	}
}

const FPaperTileMetadata* UPaperTileSet::GetTileMetadata(int32 TileIndex) const
{
	if (PerTileData.IsValidIndex(TileIndex))
	{
		return &(PerTileData[TileIndex]);
	}
	else
	{
		return nullptr;
	}
}

bool UPaperTileSet::GetTileUV(int32 TileIndex, /*out*/ FVector2D& Out_TileUV) const
{
	const int32 NumCells = GetTileCount();

	if ((TileIndex < 0) || (TileIndex >= NumCells))
	{
		return false;
	}
	else
	{
		const int32 CellsX = GetTileCountX();

		const FIntPoint XY(TileIndex % CellsX, TileIndex / CellsX);

		Out_TileUV = GetTileUVFromTileXY(XY);
		return true;
	}
}

FIntPoint UPaperTileSet::GetTileUVFromTileXY(const FIntPoint& TileXY) const
{
	return FIntPoint(TileXY.X * (TileSize.X + PerTileSpacing.X) + BorderMargin.Left, TileXY.Y * (TileSize.Y + PerTileSpacing.Y) + BorderMargin.Top);
}

FIntPoint UPaperTileSet::GetTileXYFromTextureUV(const FVector2D& TextureUV, bool bRoundUp) const
{
	const float DividendX = TextureUV.X - BorderMargin.Left;
	const float DividendY = TextureUV.Y - BorderMargin.Top;
	const int32 DivisorX = TileSize.X + PerTileSpacing.X;
	const int32 DivisorY = TileSize.Y + PerTileSpacing.Y;
	const int32 X = bRoundUp ? FMath::DivideAndRoundUp<int32>(FMath::CeilToInt(DividendX), DivisorX) : FMath::DivideAndRoundDown<int32>(DividendX, DivisorX);
	const int32 Y = bRoundUp ? FMath::DivideAndRoundUp<int32>(FMath::CeilToInt(DividendY), DivisorY) : FMath::DivideAndRoundDown<int32>(DividendY, DivisorY);
	return FIntPoint(X, Y);
}

bool UPaperTileSet::AddTerrainDescription(FPaperTileSetTerrain NewTerrain)
{
	if (Terrains.Num() < 254)
	{
		Terrains.Add(NewTerrain);
		return true;
	}
	else
	{
		return false;
	}
}

int32 UPaperTileSet::GetTerrainMembership(const FPaperTileInfo& TileInfo) const
{
	return INDEX_NONE; //@TODO: TileMapTerrain: Implement this
}

void UPaperTileSet::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);
}

void UPaperTileSet::PostLoad()
{
	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);

	if (PaperVer < FPaperCustomVersion::AllowNonUniformPaddingInTileSets)
	{
		BorderMargin = FIntMargin(Margin_DEPRECATED);
		PerTileSpacing = FIntPoint(Spacing_DEPRECATED, Spacing_DEPRECATED);
		TileSize = FIntPoint(TileWidth_DEPRECATED, TileHeight_DEPRECATED);
	}

	if (TileSheet != nullptr)
	{
		TileSheet->ConditionalPostLoad();

		WidthInTiles = GetTileCountX();
		HeightInTiles = GetTileCountY();
		ReallocateAndCopyTileData();
	}

	Super::PostLoad();
}


#if WITH_EDITOR

#include "PaperTileMapComponent.h"
#include "ComponentReregisterContext.h"

void UPaperTileSet::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	BorderMargin.Left = FMath::Max<int32>(BorderMargin.Left, 0);
	BorderMargin.Right = FMath::Max<int32>(BorderMargin.Right, 0);
	BorderMargin.Top = FMath::Max<int32>(BorderMargin.Top, 0);
	BorderMargin.Bottom = FMath::Max<int32>(BorderMargin.Bottom, 0);
	PerTileSpacing.X = FMath::Max<int32>(PerTileSpacing.X, 0);
	PerTileSpacing.Y = FMath::Max<int32>(PerTileSpacing.Y, 0);

	TileSize.X = FMath::Max<int32>(TileSize.X, 1);
	TileSize.Y = FMath::Max<int32>(TileSize.Y, 1);

	WidthInTiles = GetTileCountX();
	HeightInTiles = GetTileCountY();
	ReallocateAndCopyTileData();

	// Rebuild any tile map components that may have been relying on us
	if ((PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) == 0)
	{
		//@TODO: Currently tile maps have no fast list of referenced tile sets, so we just rebuild all of them
		TComponentReregisterContext<UPaperTileMapComponent> ReregisterAllTileMaps;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UPaperTileSet::DestructiveAllocateTileData(int32 NewWidth, int32 NewHeight)
{
	const int32 NumCells = NewWidth * NewHeight;
	PerTileData.Empty(NumCells);
	for (int32 Index = 0; Index < NumCells; ++Index)
	{
		PerTileData.Add(FPaperTileMetadata());
	}

	AllocatedWidth = NewWidth;
	AllocatedHeight = NewHeight;
}

void UPaperTileSet::ReallocateAndCopyTileData()
{
	if ((AllocatedWidth == WidthInTiles) && (AllocatedHeight == HeightInTiles))
	{
		return;
	}

	const int32 SavedWidth = AllocatedWidth;
	const int32 SavedHeight = AllocatedHeight;
	TArray<FPaperTileMetadata> SavedDesignedMap(PerTileData);

	DestructiveAllocateTileData(WidthInTiles, HeightInTiles);

	const int32 CopyWidth = FMath::Min<int32>(WidthInTiles, SavedWidth);
	const int32 CopyHeight = FMath::Min<int32>(HeightInTiles, SavedHeight);

	for (int32 Y = 0; Y < CopyHeight; ++Y)
	{
		for (int32 X = 0; X < CopyWidth; ++X)
		{
			const int32 SrcIndex = Y*SavedWidth + X;
			const int32 DstIndex = Y*WidthInTiles + X;

			PerTileData[DstIndex] = SavedDesignedMap[SrcIndex];
		}
	}
}

FName UPaperTileSet::GetTileUserData(int32 TileIndex) const
{
	if (const FPaperTileMetadata* Metadata = GetTileMetadata(TileIndex))
	{
		return Metadata->UserDataName;
	}
	else
	{
		return NAME_None;
	}
}

