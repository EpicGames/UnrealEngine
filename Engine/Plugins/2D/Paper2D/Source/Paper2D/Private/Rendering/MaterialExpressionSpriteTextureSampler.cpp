// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionSpriteTextureSampler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionSpriteTextureSampler)

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSpriteTextureSampler

UMaterialExpressionSpriteTextureSampler::UMaterialExpressionSpriteTextureSampler(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ParameterName = TEXT("SpriteTexture");
	bSampleAdditionalTextures = false;
	AdditionalSlotIndex = 0;
}

#if WITH_EDITOR
void UMaterialExpressionSpriteTextureSampler::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Paper2D Sprite"));

	if (!SlotDisplayName.IsEmpty())
	{
		OutCaptions.Add(SlotDisplayName.ToString());
	}

	if (bSampleAdditionalTextures)
	{
		FNumberFormattingOptions NoCommas;
		NoCommas.UseGrouping = false;
		const FText SlotDesc = FText::Format(LOCTEXT("SpriteSamplerTitle_AdditionalSlot", "Additional Texture #{0}"), FText::AsNumber(AdditionalSlotIndex, &NoCommas));
		OutCaptions.Add(SlotDesc.ToString());
	}
	else
	{
		OutCaptions.Add(LOCTEXT("SpriteSamplerTitle_BasicSlot", "Source Texture").ToString());
	}
}

FText UMaterialExpressionSpriteTextureSampler::GetKeywords() const
{
	FText ParentKeywords = Super::GetKeywords();

	FFormatNamedArguments Args;
	Args.Add(TEXT("ParentKeywords"), ParentKeywords);
	return FText::Format(LOCTEXT("SpriteTextureSamplerKeywords", "{ParentKeywords} Paper2D Sprite"), Args);
}

bool UMaterialExpressionSpriteTextureSampler::CanRenameNode() const
{
	// The parameter name is read only
	return false;
}

void UMaterialExpressionSpriteTextureSampler::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	const FText MyTooltip = GetClass()->GetToolTipText(/*bShortTooltip=*/ true);
	OutToolTip.Add(MyTooltip.ToString());
}

void UMaterialExpressionSpriteTextureSampler::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Clamp the slot index to something reasonably sane
	AdditionalSlotIndex = FMath::Clamp<int32>(AdditionalSlotIndex, 0, 127);

	// Ensure that the parameter name never changes
	if (bSampleAdditionalTextures)
	{
		ParameterName = FName(TEXT("SpriteAdditionalTexture"), AdditionalSlotIndex + 1);
	}
	else
	{
		ParameterName = TEXT("SpriteTexture");
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

#undef LOCTEXT_NAMESPACE

