// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTerrainActor.h"
#include "PaperTerrainComponent.h"
#include "PaperTerrainSplineComponent.h"
#include "PaperTerrainMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTerrainActor)

//////////////////////////////////////////////////////////////////////////
// APaperTerrainActor

APaperTerrainActor::APaperTerrainActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DummyRoot = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));
	SplineComponent = CreateDefaultSubobject<UPaperTerrainSplineComponent>(TEXT("SplineComponent"));
 	RenderComponent = CreateDefaultSubobject<UPaperTerrainComponent>(TEXT("RenderComponent"));
 
	SplineComponent->SetupAttachment(DummyRoot);
	RenderComponent->SetupAttachment(DummyRoot);
	RenderComponent->AssociatedSpline = SplineComponent;
	RootComponent = DummyRoot;
}

#if WITH_EDITOR
bool APaperTerrainActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (RenderComponent->TerrainMaterial != nullptr)
	{
		Objects.Add(RenderComponent->TerrainMaterial);
	}
	return true;
}
#endif

