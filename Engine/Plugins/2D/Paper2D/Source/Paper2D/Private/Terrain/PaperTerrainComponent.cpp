// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTerrainComponent.h"
#include "UObject/ConstructorHelpers.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "PaperCustomVersion.h"

#include "PaperRenderSceneProxy.h"
#include "GeomTools.h"
#include "PhysicsEngine/BodySetup.h"
#include "PaperTerrainSplineComponent.h"
#include "PaperTerrainMaterial.h"
#include "Paper2DPrivate.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTerrainComponent)

#define PAPER_USE_MATERIAL_SLOPES 1
#define PAPER_TERRAIN_DRAW_DEBUG 0

//////////////////////////////////////////////////////////////////////////

static FBox2D GetSpriteRenderDataBounds2D(const TArray<FVector4>& Data)
{
	FBox2D Bounds(ForceInit);
	for (const FVector4& XYUV : Data)
	{
		Bounds += FVector2D(XYUV.X, XYUV.Y);
	}

	return Bounds;
}


//////////////////////////////////////////////////////////////////////////

FTerrainSpriteStamp::FTerrainSpriteStamp(const UPaperSprite* InSprite, float InTime, bool bIsEndCap)
	: Sprite(InSprite)
	, Time(InTime)
	, Scale(1.0f)
	, bCanStretch(!bIsEndCap)
{
	const FBox2D Bounds2D = GetSpriteRenderDataBounds2D(InSprite->BakedRenderData);
	NominalWidth = FMath::Max<float>(Bounds2D.GetSize().X, 1.0f);
}


FTerrainSegment::FTerrainSegment() : Rule(nullptr)
{

}

void FTerrainSegment::RepositionStampsToFillSpace()
{

}

//////////////////////////////////////////////////////////////////////////
// FPaperTerrainSceneProxy

class FPaperTerrainSceneProxy final : public FPaperRenderSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FPaperTerrainSceneProxy(const UPaperTerrainComponent* InComponent, const TArray<FPaperTerrainSpriteGeometry>& InDrawingData);
};

FPaperTerrainSceneProxy::FPaperTerrainSceneProxy(const UPaperTerrainComponent* InComponent, const TArray<FPaperTerrainSpriteGeometry>& InDrawingData)
	: FPaperRenderSceneProxy(InComponent)
{
	//@TODO: PAPER2D: Build/store the data in the 'new' batch format directly rather than converting it here
	BatchedSections.AddDefaulted(InDrawingData.Num());
	FSpriteRenderSection* DstBatch = BatchedSections.GetData();
	for (const FPaperTerrainSpriteGeometry& SrcBatch : InDrawingData)
	{
		DstBatch->Material = SrcBatch.Material;
		for (const FSpriteDrawCallRecord& SrcRecord : SrcBatch.Records)
		{
			DstBatch->AddVerticesFromDrawCallRecord(SrcRecord, 0, SrcRecord.RenderVerts.Num(), Vertices);
		}
		++DstBatch;
	}

	// Combine the material relevance for all materials
	for (const FPaperTerrainSpriteGeometry& Batch : InDrawingData)
	{
		const UMaterialInterface* MaterialInterface = (Batch.Material != nullptr) ? Batch.Material : UMaterial::GetDefaultMaterial(MD_Surface);
		MaterialRelevance |= MaterialInterface->GetRelevance_Concurrent(GetScene().GetShaderPlatform());
	}
}

//////////////////////////////////////////////////////////////////////////
// UPaperTerrainComponent

UPaperTerrainComponent::UPaperTerrainComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bFilledSpline(true)
	, SegmentOverlapAmount(100.0f)
	, TerrainColor(FLinearColor::White)
	, ReparamStepsPerSegment(8)
	, SpriteCollisionDomain(ESpriteCollisionMode::Use3DPhysics)
	, CollisionThickness(200.0f)
{
	bCanEverAffectNavigation = true;

	static ConstructorHelpers::FObjectFinder<UPaperTerrainMaterial> DefaultMaterialRef(TEXT("/Paper2D/DefaultPaperTerrainMaterial"));
	TerrainMaterial = DefaultMaterialRef.Object;
}

const UObject* UPaperTerrainComponent::AdditionalStatObject() const
{
	return TerrainMaterial;
}

void UPaperTerrainComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPaperCustomVersion::GUID);

	if (SpriteCollisionDomain == ESpriteCollisionMode::Use2DPhysics)
	{
		UE_LOGF(LogPaper2D, Warning, "PaperTerrainComponent '%ls' was using 2D physics which has been removed, it has been switched to 3D physics.", *GetPathName());
		SpriteCollisionDomain = ESpriteCollisionMode::Use3DPhysics;
	}
}

void UPaperTerrainComponent::PostLoad()
{
	Super::PostLoad();

	const int32 PaperVer = GetLinkerCustomVersion(FPaperCustomVersion::GUID);

	if (PaperVer < FPaperCustomVersion::FixVertexColorSpace)
	{
		const FColor SRGBColor = TerrainColor.ToFColor(/*bSRGB=*/ true);
		TerrainColor = SRGBColor.ReinterpretAsLinear();
	}
}

#if WITH_EDITOR
void UPaperTerrainComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		const FName PropertyName(PropertyChangedEvent.Property->GetFName());
		if (PropertyName == GET_MEMBER_NAME_CHECKED(USplineComponent, ReparamStepsPerSegment))
		{
			if (AssociatedSpline != nullptr)
			{
				AssociatedSpline->ReparamStepsPerSegment = ReparamStepsPerSegment;
				AssociatedSpline->UpdateSpline();
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UPaperTerrainComponent::OnRegister()
{
	Super::OnRegister();

	if (AssociatedSpline != nullptr)
	{
		AssociatedSpline->OnSplineEdited = FSimpleDelegate::CreateUObject(this, &UPaperTerrainComponent::OnSplineEdited);
	}

	OnSplineEdited();
}

void UPaperTerrainComponent::OnUnregister()
{
	Super::OnUnregister();

	if (AssociatedSpline != nullptr)
	{
		AssociatedSpline->OnSplineEdited.Unbind();
	}
}

FPrimitiveSceneProxy* UPaperTerrainComponent::CreateSceneProxy()
{
	FPaperTerrainSceneProxy* NewProxy = new FPaperTerrainSceneProxy(this, GeneratedSpriteGeometry);
	return NewProxy;
}

FBoxSphereBounds UPaperTerrainComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// Determine the rendering bounds
	FBoxSphereBounds LocalRenderBounds;
	{
		FBox BoundingBox(ForceInit);

		for (const FPaperTerrainSpriteGeometry& DrawCall : GeneratedSpriteGeometry)
		{
			for (const FSpriteDrawCallRecord& Record : DrawCall.Records)
			{
				for (const FVector4& VertXYUV : Record.RenderVerts)
				{
					const FVector Vert((PaperAxisX * VertXYUV.X) + (PaperAxisY * VertXYUV.Y));
					BoundingBox += Vert;
				}
			}
		}

		// Make the whole thing a single unit 'deep'
		const FVector HalfThicknessVector = 0.5f * PaperAxisZ;
		BoundingBox.Min -= HalfThicknessVector;
		BoundingBox.Max += HalfThicknessVector;

		LocalRenderBounds = FBoxSphereBounds(BoundingBox);
	}

	// Graphics bounds.
	FBoxSphereBounds NewBounds = LocalRenderBounds.TransformBy(LocalToWorld);

	// Add bounds of collision geometry (if present).
	if (CachedBodySetup != nullptr)
	{
		const FBox AggGeomBox = CachedBodySetup->AggGeom.CalcAABB(LocalToWorld);
		if (AggGeomBox.IsValid)
		{
			NewBounds = Union(NewBounds, FBoxSphereBounds(AggGeomBox));
		}
	}

	// Apply bounds scale
	NewBounds.BoxExtent *= BoundsScale;
	NewBounds.SphereRadius *= BoundsScale;

	return NewBounds;
}

UBodySetup* UPaperTerrainComponent::GetBodySetup()
{
	return CachedBodySetup;
}

FTransform UPaperTerrainComponent::GetTransformAtDistance(float InDistance) const
{
	const float SplineLength = AssociatedSpline->GetSplineLength();
	InDistance = FMath::Clamp<float>(InDistance, 0.0f, SplineLength);

	const float Param = AssociatedSpline->GetInputKeyValueAtDistanceAlongSpline(InDistance);
	const FVector Position3D = AssociatedSpline->GetLocationAtSplineInputKey(Param, ESplineCoordinateSpace::Local);

	const FVector Tangent = AssociatedSpline->GetTangentAtSplineInputKey(Param, ESplineCoordinateSpace::Local).GetSafeNormal();
	const FVector NormalEst = StaticCastPtr<const UPaperTerrainSplineComponent, UPaperTerrainSplineComponent>(AssociatedSpline)->GetSplinePointsPosition().EvalSecondDerivative(Param, FVector(0.0f, 1.0f, 0.0f)).GetSafeNormal();
	const FVector Bitangent = FVector::CrossProduct(Tangent, NormalEst);
	const FVector Normal = FVector::CrossProduct(Bitangent, Tangent);
	const FVector Floop = FVector::CrossProduct(PaperAxisZ, Tangent);

	FTransform LocalTransform(Tangent, PaperAxisZ, Floop, Position3D);

	LocalTransform = FTransform(FRotator(0.0f, 180.0f, 0.0f), FVector::ZeroVector) * LocalTransform;

#if PAPER_TERRAIN_DRAW_DEBUG
	FTransform WorldTransform = LocalTransform * GetComponentToWorld();

	const float Time = 2.5f;

	DrawDebugCoordinateSystem(GetWorld(), WorldTransform.GetLocation(), FRotator(WorldTransform.GetRotation()), 15.0f, true, Time, SDPG_Foreground);
// 	FVector WorldPos = GetComponentTransform().TransformPosition(Position3D);
// 	WorldPos.Y -= 0.01;
// 
// 	//DrawDebugLine(GetWorld(), WorldPos, WorldPos + GetComponentTransform().TransformVector(Tangent) * 10.0f, FColor::Red, true, Time);
// 	// 		DrawDebugLine(GetWorld(), WorldPos, WorldPos + GetComponentTransform().TransformVector(NormalEst) * 10.0f, FColor::Green, true, Time);
// 	// 		DrawDebugLine(GetWorld(), WorldPos, WorldPos + GetComponentTransform().TransformVector(Bitangent) * 10.0f, FColor::Blue, true, Time);
// 	//DrawDebugLine(GetWorld(), WorldPos, WorldPos + GetComponentTransform().TransformVector(Floop) * 10.0f, FColor::Yellow, true, Time);
// 	// 		DrawDebugPoint(GetWorld(), WorldPos, 4.0f, FColor::White, true, 1.0f);
#endif

	return LocalTransform;
}

// Simplify spline by removing vertices from nearly flat edges
// Currently don't allow merging points when the offset is different
// SplineEdgeOffsetAmounts is ignored if empty
static void SimplifyPolygon(TArray<FVector2D> &SplinePolyVertices2D, TArray<float> &SplineEdgeOffsetAmounts, float FlatEdgeDistance = 10)
{
	const bool bHasSplineEdgeOffsets = SplineEdgeOffsetAmounts.Num() == SplinePolyVertices2D.Num();
	const float FlatEdgeDistanceThreshold = FlatEdgeDistance * FlatEdgeDistance;
	for (int i = 0; i < SplinePolyVertices2D.Num() && SplinePolyVertices2D.Num() > 3; ++i)
	{
		int StartRemoveIndex = (i + 1) % SplinePolyVertices2D.Num();
		int EndRemoveIndex = StartRemoveIndex;
		FVector2D& A = SplinePolyVertices2D[i];
		float SplineEdgeOffsetA = bHasSplineEdgeOffsets ? SplineEdgeOffsetAmounts[i] : 0;

		// Keep searching to find if any of the vector rejections fail in subsequent points on the polygon
		// A B C D E F (eg. when testing A B C, test rejection for BA, CA)
		// When testing A E F, test rejection for AB-AF, AC-AF, AD-AF, AE-AF
		// When one of these fails we discard all verts between A and one before the current vertex being tested
		for (int j = i; j < SplinePolyVertices2D.Num(); ++j)
		{
			int IndexC = (j + 2) % SplinePolyVertices2D.Num();
			FVector2D& C = SplinePolyVertices2D[IndexC];
			float SplineEdgeOffsetC = bHasSplineEdgeOffsets ? SplineEdgeOffsetAmounts[IndexC] : 0;
			bool bSmallOffsetFailed = SplineEdgeOffsetA != SplineEdgeOffsetC;

			for (int k = i; k <= j && !bSmallOffsetFailed; ++k)
			{
				int IndexB = (k + 1) % SplinePolyVertices2D.Num();
				FVector2D& B = SplinePolyVertices2D[IndexB];
				float SplineEdgeOffsetB = bHasSplineEdgeOffsets ? SplineEdgeOffsetAmounts[IndexB] : 0;
				if (SplineEdgeOffsetA != SplineEdgeOffsetB)
				{
					bSmallOffsetFailed = true;
					break;
				}

				FVector2D CA = C - A;
				FVector2D BA = B - A;
				FVector2D Rejection_BA_CA = BA - (FVector2D::DotProduct(BA, CA) / FVector2D::DotProduct(CA, CA)) * CA;
				float RejectionLengthSquared = Rejection_BA_CA.SizeSquared();
				if (RejectionLengthSquared > FlatEdgeDistanceThreshold)
				{
					bSmallOffsetFailed = true;
					break;
				}
			}

			if (bSmallOffsetFailed)
			{
				break;
			}
			else
			{
				EndRemoveIndex = (EndRemoveIndex + 1) % SplinePolyVertices2D.Num();
			}
		}

		// Remove the vertices that we deemed "too flat"
		if (EndRemoveIndex > StartRemoveIndex)
		{
			SplinePolyVertices2D.RemoveAt(StartRemoveIndex, EndRemoveIndex - StartRemoveIndex);
			if (bHasSplineEdgeOffsets)
			{
				SplineEdgeOffsetAmounts.RemoveAt(StartRemoveIndex, EndRemoveIndex - StartRemoveIndex);
			}
		}
		else if (EndRemoveIndex < StartRemoveIndex)
		{
			SplinePolyVertices2D.RemoveAt(StartRemoveIndex, SplinePolyVertices2D.Num() - StartRemoveIndex);
			SplinePolyVertices2D.RemoveAt(0, EndRemoveIndex);
			if (bHasSplineEdgeOffsets)
			{
				SplineEdgeOffsetAmounts.RemoveAt(StartRemoveIndex, SplineEdgeOffsetAmounts.Num() - StartRemoveIndex);
				SplineEdgeOffsetAmounts.RemoveAt(0, EndRemoveIndex);
			}
			// The search has wrapped around, no more vertices to test
			break;
		}
	}
}

// Makes sure all spline points are constrained to the XZ plane
void UPaperTerrainComponent::ConstrainSplinePointsToXZ()
{
	if (AssociatedSpline != nullptr)
	{
		bool bSplineChanged = false;
		const int32 NumPoints = AssociatedSpline->GetNumberOfSplinePoints();
		for (int PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const FSplinePoint Point = AssociatedSpline->GetSplinePointAt(PointIndex, ESplineCoordinateSpace::Local);
			
			if (Point.ArriveTangent.Y != 0 || Point.LeaveTangent.Y != 0 || Point.Position.Y != 0)
			{
				FVector NewArriveTangent = Point.ArriveTangent; NewArriveTangent.Y = 0;
				FVector NewLeaveTangent = Point.LeaveTangent; NewLeaveTangent.Y = 0;
				AssociatedSpline->SetTangentsAtSplinePoint(PointIndex, NewArriveTangent, NewLeaveTangent, ESplineCoordinateSpace::Local, false);
				
				FVector NewPosition = Point.Position; NewPosition.Y = 0;
				AssociatedSpline->SetLocationAtSplinePoint(PointIndex, NewPosition, ESplineCoordinateSpace::Local, false);
				
				bSplineChanged = true;
			}
		}

		if (bSplineChanged)
		{
			AssociatedSpline->UpdateSpline();
		}
	}
}

void UPaperTerrainComponent::OnSplineEdited()
{
	ConstrainSplinePointsToXZ();

	// Ensure we have the data structure for the desired collision method
	if (SpriteCollisionDomain == ESpriteCollisionMode::Use3DPhysics)
	{
		CachedBodySetup = NewObject<UBodySetup>(this);
	}
	else
	{
		CachedBodySetup = nullptr;
	}

	const float SlopeAnalysisTimeRate = 10.0f;
	const float FillRasterizationTimeRate = 100.0f;

	GeneratedSpriteGeometry.Empty();

	if ((AssociatedSpline != nullptr) && (TerrainMaterial != nullptr))
	{
		if (AssociatedSpline->ReparamStepsPerSegment != ReparamStepsPerSegment)
		{
			AssociatedSpline->ReparamStepsPerSegment = ReparamStepsPerSegment;
			AssociatedSpline->UpdateSpline();
		}

		FRandomStream RandomStream(RandomSeed);

		float SplineLength = AssociatedSpline->GetSplineLength();


		struct FTerrainRuleHelper
		{
		public:
			FTerrainRuleHelper(const FPaperTerrainMaterialRule* Rule)
				: StartWidth(0.0f)
				, EndWidth(0.0f)
			{
				for (const UPaperSprite* Sprite : Rule->Body)
				{
					if (Sprite != nullptr)
					{
						const float Width = GetSpriteRenderDataBounds2D(Sprite->BakedRenderData).GetSize().X;
						if (Width > 0.0f)
						{
							ValidBodies.Add(Sprite);
							ValidBodyWidths.Add(Width);
						}
					}
				}

				if (Rule->StartCap != nullptr)
				{
					const float Width = GetSpriteRenderDataBounds2D(Rule->StartCap->BakedRenderData).GetSize().X;
					if (Width > 0.0f)
					{
						StartWidth = Width;
					}
				}

				if (Rule->EndCap != nullptr)
				{
					const float Width = GetSpriteRenderDataBounds2D(Rule->EndCap->BakedRenderData).GetSize().X;
					if (Width > 0.0f)
					{
						EndWidth = Width;
					}
				}
			}

			float StartWidth;
			float EndWidth;

			TArray<const UPaperSprite*> ValidBodies;
			TArray<float> ValidBodyWidths;

			int32 GenerateBodyIndex(FRandomStream& InRandomStream) const
			{
				check(ValidBodies.Num() > 0);
				return InRandomStream.GetUnsignedInt() % ValidBodies.Num();
			}
		};

		// Split the spline into segments based on the slope rules in the material
		TArray<FTerrainSegment> Segments;

		FTerrainSegment* ActiveSegment = new (Segments) FTerrainSegment();
		ActiveSegment->StartTime = 0.0f;
		ActiveSegment->EndTime = SplineLength;

		{
			float CurrentTime = 0.0f;
			while (CurrentTime < SplineLength)
			{
				const FTransform Frame(GetTransformAtDistance(CurrentTime));
				const FVector UnitTangent = Frame.GetUnitAxis(EAxis::X);
				const float RawSlopeAngleRadians = FMath::Atan2(FVector::DotProduct(UnitTangent, PaperAxisY), FVector::DotProduct(UnitTangent, PaperAxisX));
				const float RawSlopeAngle = FMath::RadiansToDegrees(RawSlopeAngleRadians);
				const float SlopeAngle = FMath::Fmod(FMath::UnwindDegrees(RawSlopeAngle) + 360.0f, 360.0f);

				const FPaperTerrainMaterialRule* DesiredRule = (TerrainMaterial->Rules.Num() > 0) ? &(TerrainMaterial->Rules[0]) : nullptr;
				for (const FPaperTerrainMaterialRule& TestRule : TerrainMaterial->Rules)
				{
					if ((SlopeAngle >= TestRule.MinimumAngle) && (SlopeAngle < TestRule.MaximumAngle))
					{
						DesiredRule = &TestRule;
					}
				}

				if (ActiveSegment->Rule != DesiredRule)
				{
					if (ActiveSegment->Rule == nullptr)
					{
						ActiveSegment->Rule = DesiredRule;
					}
					else
					{
						ActiveSegment->EndTime = CurrentTime;
						
						// Segment is too small, delete it
						if (ActiveSegment->EndTime < ActiveSegment->StartTime + 2.0f * SegmentOverlapAmount)
						{
							Segments.Pop(EAllowShrinking::No);
						}

						ActiveSegment = new (Segments)FTerrainSegment();
						ActiveSegment->StartTime = CurrentTime;
						ActiveSegment->EndTime = SplineLength;
						ActiveSegment->Rule = DesiredRule;
					}
				}

				CurrentTime += SlopeAnalysisTimeRate;
			}
		}

		// Account for overlap
		for (FTerrainSegment& Segment : Segments)
		{
			Segment.StartTime -= SegmentOverlapAmount;
			Segment.EndTime += SegmentOverlapAmount;
		}

		// Convert those segments to actual geometry
		for (FTerrainSegment& Segment : Segments)
		{
			check(Segment.Rule);
			FTerrainRuleHelper RuleHelper(Segment.Rule);

			float RemainingSegStart = Segment.StartTime + RuleHelper.StartWidth;
			float RemainingSegEnd = Segment.EndTime - RuleHelper.EndWidth;
			const float BodyDistance = RemainingSegEnd - RemainingSegStart;
			float DistanceBudget = BodyDistance;

			bool bUseBodySegments = (DistanceBudget > 0.0f) && (RuleHelper.ValidBodies.Num() > 0);

			// Add the start cap
			if (RuleHelper.StartWidth > 0.0f)
			{
				new (Segment.Stamps) FTerrainSpriteStamp(Segment.Rule->StartCap, Segment.StartTime + RuleHelper.StartWidth * 0.5f, /*bIsEndCap=*/ bUseBodySegments);
			}

			// Add body segments
			if (bUseBodySegments)
			{
				int32 NumSegments = 0;
				float Position = RemainingSegStart;
				
				while (DistanceBudget > 0.0f)
				{
					const int32 BodyIndex = RuleHelper.GenerateBodyIndex(RandomStream);
					const UPaperSprite* Sprite = RuleHelper.ValidBodies[BodyIndex];
					const float Width = RuleHelper.ValidBodyWidths[BodyIndex];

					if ((NumSegments > 0) && ((Width * 0.5f) > DistanceBudget))
					{
						break;
					}
					new (Segment.Stamps) FTerrainSpriteStamp(Sprite, Position + (Width * 0.5f), /*bIsEndCap=*/ false);

					DistanceBudget -= Width;
					Position += Width;
					++NumSegments;
				}

				const float UsedSpace = (BodyDistance - DistanceBudget);
				const float OverallScaleFactor = BodyDistance / UsedSpace;
				
				// Stretch body segments
				float PositionCorrectionSum = 0.0f;
				for (int32 Index = 0; Index < NumSegments; ++Index)
				{
					FTerrainSpriteStamp& Stamp = Segment.Stamps[Index + (Segment.Stamps.Num() - NumSegments)];
					
					const float WidthChange = (OverallScaleFactor - 1.0f) * Stamp.NominalWidth;
					const float FirstGapIsSmallerFactor = (Index == 0) ? 0.5f : 1.0f;
					PositionCorrectionSum += WidthChange * FirstGapIsSmallerFactor;

					Stamp.Scale = OverallScaleFactor;
					Stamp.Time += PositionCorrectionSum;
				}
			}
			else
			{
				// Stretch endcaps
			}

			// Add the end cap
			if (RuleHelper.EndWidth > 0.0f)
			{
				new (Segment.Stamps) FTerrainSpriteStamp(Segment.Rule->EndCap, Segment.EndTime - RuleHelper.EndWidth * 0.5f, /*bIsEndCap=*/ bUseBodySegments);
			}
		}

		// Convert stamps into geometry
		SpawnSegments(Segments, !bClosedSpline || (bClosedSpline && !bFilledSpline));

		// Generate the background if the spline is closed
		if (bClosedSpline && bFilledSpline)
		{
			// Create a polygon from the spline
			FBox2D SplineBounds(ForceInit);
			TArray<FVector2D> SplinePolyVertices2D;
			TArray<float> SplineEdgeOffsetAmounts;
			{
				float CurrentTime = 0.0f;
				while (CurrentTime < SplineLength)
				{
					// note [nickolas.drake]: SplineLength is not measured in time, I am adapting this code in spite of the fact that it seems wrong at a glance.
					const float Param = AssociatedSpline->GetInputKeyValueAtDistanceAlongSpline(CurrentTime);
					const FVector Position3D = AssociatedSpline->GetLocationAtSplineInputKey(Param, ESplineCoordinateSpace::Local);
					const FVector2D Position2D = FVector2D(FVector::DotProduct(Position3D, PaperAxisX), FVector::DotProduct(Position3D, PaperAxisY));

					SplineBounds += Position2D;
					SplinePolyVertices2D.Add(Position2D);

					// Find the collision offset for this sample point
					float CollisionOffset = 0;
					for (int SegmentIndex = 0; SegmentIndex < Segments.Num(); ++SegmentIndex)
					{
						FTerrainSegment& Segment = Segments[SegmentIndex];
						if (CurrentTime >= Segment.StartTime && CurrentTime <= Segment.EndTime)
						{
							CollisionOffset = (Segment.Rule != nullptr) ? (Segment.Rule->CollisionOffset * 0.25f) : 0;
							break;
						}
					}
					SplineEdgeOffsetAmounts.Add(CollisionOffset);

					CurrentTime += FillRasterizationTimeRate;
				}
			}

			SimplifyPolygon(SplinePolyVertices2D, SplineEdgeOffsetAmounts);

			// Always CCW and facing forward regardless of spline winding
			TArray<FVector2D> CorrectedSplineVertices;
			FGeomTools2D::CorrectPolygonWinding(CorrectedSplineVertices, SplinePolyVertices2D, false);

			TArray<FVector2D> TriangulatedPolygonVertices;
			FGeomTools2D::TriangulatePoly(/*out*/TriangulatedPolygonVertices, CorrectedSplineVertices, false);

			GenerateCollisionDataFromPolygon(SplinePolyVertices2D, SplineEdgeOffsetAmounts, TriangulatedPolygonVertices);

			if (TerrainMaterial->InteriorFill != nullptr)
			{
				const UPaperSprite* FillSprite = TerrainMaterial->InteriorFill;
				FPaperTerrainSpriteGeometry& MaterialBatch = *new (GeneratedSpriteGeometry)FPaperTerrainSpriteGeometry(); //@TODO: Look up the existing one instead
				MaterialBatch.Material = FillSprite->GetDefaultMaterial();

				FSpriteDrawCallRecord& FillDrawCall = *new (MaterialBatch.Records) FSpriteDrawCallRecord();
				FillDrawCall.BuildFromSprite(FillSprite);
				FillDrawCall.RenderVerts.Empty();
				FillDrawCall.Color = TerrainColor.ToFColor(/*bSRGB=*/ false);
				FillDrawCall.Destination = PaperAxisZ * 0.1f;

				const FVector2D TextureSize = GetSpriteRenderDataBounds2D(FillSprite->BakedRenderData).GetSize();
				const FVector2D SplineSize = SplineBounds.GetSize();

				GenerateFillRenderDataFromPolygon(FillSprite, FillDrawCall, TextureSize, TriangulatedPolygonVertices);

				//@TODO: Add support for the fill sprite being smaller than the entire texture
#ifdef NOT_WORKING
				const float StartingDivisionPointX = FMath::CeilToFloat(SplineBounds.Min.X / TextureSize.X);
				const float StartingDivisionPointY = FMath::CeilToFloat(SplineBounds.Min.Y / TextureSize.Y);

				FPoly VerticalRemainder = SplineAsPolygon;
				for (float Y = StartingDivisionPointY; VerticalRemainder.Vertices.Num() > 0; Y += TextureSize.Y)
				{
					FPoly Top;
					FPoly Bottom;
					const FVector SplitBaseOuter = (Y * PaperAxisY);
					VerticalRemainder.SplitWithPlane(SplitBaseOuter, -PaperAxisY, &Top, &Bottom, 1);
					VerticalRemainder = Bottom;

					FPoly HorizontalRemainder = Top;
					for (float X = StartingDivisionPointX; HorizontalRemainder.Vertices.Num() > 0; X += TextureSize.X)
					{
						FPoly Left;
						FPoly Right;
						const FVector SplitBaseInner = (X * PaperAxisX) + (Y * PaperAxisY);
						HorizontalRemainder.SplitWithPlane(SplitBaseInner, -PaperAxisX, &Left, &Right, 1);
						HorizontalRemainder = Right;

						//BROKEN, function no longer exists (split into 2 parts)
						SpawnFromPoly(Segments, SplineEdgeOffsetAmounts, FillSprite, FillDrawCall, TextureSize, Left);
					}
				}
#endif
			}
		}

		// Draw debug frames at the start and end of the spline
#if PAPER_TERRAIN_DRAW_DEBUG
		{
			const float Time = 5.0f;
			{
				FTransform WorldTransform = GetTransformAtDistance(0.0f) * GetComponentTransform();
				DrawDebugCoordinateSystem(GetWorld(), WorldTransform.GetLocation(), FRotator(WorldTransform.GetRotation()), 30.0f, true, Time, SDPG_Foreground);
			}
			{
				FTransform WorldTransform = GetTransformAtDistance(SplineLength) * GetComponentTransform();
				DrawDebugCoordinateSystem(GetWorld(), WorldTransform.GetLocation(), FRotator(WorldTransform.GetRotation()), 30.0f, true, Time, SDPG_Foreground);
			}
		}
#endif
	}

	if (CachedBodySetup != nullptr)
	{
		// Finalize the BodySetup
		CachedBodySetup->InvalidatePhysicsData();
		CachedBodySetup->CreatePhysicsMeshes();
	}

	RecreateRenderState_Concurrent();
}

#define USE_SIMPLIFIED_POLYGON_COLLIDERS_FOR_SEGMENTS

void UPaperTerrainComponent::SpawnSegments(const TArray<FTerrainSegment>& TerrainSegments, bool bGenerateSegmentColliders)
{
#ifdef USE_SIMPLIFIED_POLYGON_COLLIDERS_FOR_SEGMENTS
	TArray<FVector2D> CollisionPolygonPoints;
#endif

	// The tangent from the first box added in this segment
	FVector2D StartTangent;

	for (const FTerrainSegment& Segment : TerrainSegments)
	{
		for (const FTerrainSpriteStamp& Stamp : Segment.Stamps)
		{
			const class UPaperSprite* NewSprite = Stamp.Sprite;
			float Position = Stamp.Time;
			float HorizontalScale = Stamp.Scale;
			float NominalWidth = Stamp.NominalWidth;
			const struct FPaperTerrainMaterialRule* Rule = Segment.Rule;

			if (bGenerateSegmentColliders && (Rule->bEnableCollision) && (CachedBodySetup != nullptr))
			{
				const FTransform LocalTransformAtCenter(GetTransformAtDistance(Position));

#ifdef USE_SIMPLIFIED_POLYGON_COLLIDERS_FOR_SEGMENTS
				// Check note be low Re: bClosedSplines
				FVector2D BoxExtents( 0.5f * NominalWidth * HorizontalScale, 0.5f * FMath::Max<float>(1.0f, FMath::Abs<float>(Rule->CollisionOffset * 0.5f)) );
				
				FVector BoxPoints[4];
				BoxPoints[0] = LocalTransformAtCenter.TransformPosition(FVector(BoxExtents.X, 0, BoxExtents.Y));
				BoxPoints[1] = LocalTransformAtCenter.TransformPosition(FVector(-BoxExtents.X, 0, BoxExtents.Y));
				BoxPoints[2] = LocalTransformAtCenter.TransformPosition(FVector(-BoxExtents.X, 0, -BoxExtents.Y));
				BoxPoints[3] = LocalTransformAtCenter.TransformPosition(FVector(BoxExtents.X, 0, -BoxExtents.Y));

				FVector2D BoxPoints2D[4];
				BoxPoints2D[0] = FVector2D(BoxPoints[0].X, BoxPoints[0].Z);
				BoxPoints2D[1] = FVector2D(BoxPoints[1].X, BoxPoints[1].Z);
				BoxPoints2D[2] = FVector2D(BoxPoints[2].X, BoxPoints[2].Z);
				BoxPoints2D[3] = FVector2D(BoxPoints[3].X, BoxPoints[3].Z);

				// If there is a previous polygon, try to merge
				if (CollisionPolygonPoints.Num() >= 4)
				{
					int InsertPoint = CollisionPolygonPoints.Num() / 2 - 1;
					float LengthV0 = FVector2D::Distance(CollisionPolygonPoints[InsertPoint], BoxPoints2D[0]);
					float LengthV1 = FVector2D::Distance(CollisionPolygonPoints[InsertPoint + 1], BoxPoints2D[3]);

					FVector2D CurrentSegmentTangent = BoxPoints2D[1] - BoxPoints2D[0];
					CurrentSegmentTangent.Normalize();

					bool bNewSegmentStraightEnough = FVector2D::DotProduct(CurrentSegmentTangent, StartTangent) > FMath::Acos(45.0f);

					// TODO: Arbitray number needs to come from somewhere...
					float MergeThreshold = 10;
					bool bMergeIntoPolygon = LengthV0 < MergeThreshold && LengthV1 < MergeThreshold;

					if (bNewSegmentStraightEnough && bMergeIntoPolygon)
					{
						CollisionPolygonPoints.Insert(BoxPoints2D[2], InsertPoint + 1);
						CollisionPolygonPoints.Insert(BoxPoints2D[1], InsertPoint + 1);
					}
					else
					{
						InsertConvexCollisionDataFromPolygon(CollisionPolygonPoints);
						CollisionPolygonPoints.Empty(0);
						CollisionPolygonPoints.Add(BoxPoints2D[0]);
						CollisionPolygonPoints.Add(BoxPoints2D[1]);
						CollisionPolygonPoints.Add(BoxPoints2D[2]);
						CollisionPolygonPoints.Add(BoxPoints2D[3]);
						StartTangent = BoxPoints2D[1] - BoxPoints2D[0];
						StartTangent.Normalize();
					}
				}
				else
				{
					CollisionPolygonPoints.Add(BoxPoints2D[0]);
					CollisionPolygonPoints.Add(BoxPoints2D[1]);
					CollisionPolygonPoints.Add(BoxPoints2D[2]);
					CollisionPolygonPoints.Add(BoxPoints2D[3]);
					StartTangent = BoxPoints2D[1] - BoxPoints2D[0];
					StartTangent.Normalize();
				}
#else
				FKBoxElem Box;
				// The spline is never "closed" properly right now
				//if (bClosedSpline)
				//{
				//	//@TODO: Add proper support for this!
				//	Box.SetTransform(LocalTransformAtCenter);
				//	Box.X = NominalWidth * HorizontalScale;
				//	Box.Y = CollisionThickness;
				//	Box.Z = 30.0f;
				//}
				//else
				{
					Box.SetTransform(LocalTransformAtCenter);
					Box.X = NominalWidth * HorizontalScale;
					Box.Y = CollisionThickness;

					Box.Z = FMath::Max<float>(1.0f, FMath::Abs<float>(Rule->CollisionOffset * 0.5f));
				}
				CachedBodySetup->AggGeom.BoxElems.Add(Box);
#endif
			}

			if (NewSprite)
			{
				FPaperTerrainSpriteGeometry& MaterialBatch = *new (GeneratedSpriteGeometry) FPaperTerrainSpriteGeometry(); //@TODO: Look up the existing one instead
				MaterialBatch.Material = NewSprite->GetDefaultMaterial();
				MaterialBatch.DrawOrder = Rule->DrawOrder;

				FSpriteDrawCallRecord& Record = *new (MaterialBatch.Records) FSpriteDrawCallRecord();
				Record.BuildFromSprite(NewSprite);
				Record.Color = TerrainColor.ToFColor(/*bSRGB=*/ false);

				// Work out if the sprite is likely to be bent > X deg (folded over itself)
				const FVector ForwardVector(1, 0, 0);
				const FTransform LocalTransformAtBack(GetTransformAtDistance(Position - 0.5f * NominalWidth * HorizontalScale));
				FVector StartForwardVector = LocalTransformAtBack.TransformVector(ForwardVector).GetSafeNormal();
				const FTransform LocalTransformAtFront(GetTransformAtDistance(Position + 0.5f * NominalWidth * HorizontalScale));
				FVector EndForwardVector = LocalTransformAtFront.TransformVector(ForwardVector).GetSafeNormal();
				bool bIsSpriteBent = FVector::DotProduct(StartForwardVector, EndForwardVector) < 0.0f;// 0.7071f; // (45deg looks worse)

				for (FVector4& XYUV : Record.RenderVerts)
				{
					FTransform LocalTransformAtX(GetTransformAtDistance(Position + (XYUV.X * HorizontalScale)));

					// When the quad is overly bent, inherit rotation from the start of the quad to unfold it
					if (bIsSpriteBent)
					{
						LocalTransformAtX.SetRotation(LocalTransformAtFront.GetRotation());
					}

					const FVector SourceVector = (XYUV.Y * PaperAxisY);
					const FVector NewVector = LocalTransformAtX.TransformPosition(SourceVector);

					const float NewX = FVector::DotProduct(NewVector, PaperAxisX);
					const float NewY = FVector::DotProduct(NewVector, PaperAxisY);

					XYUV.X = NewX;
					XYUV.Y = NewY;
				}
			}
		}
	}

	//@TODO: Sort by draw order first, materials next - Merge batches with the same material
	GeneratedSpriteGeometry.Sort([](const FPaperTerrainSpriteGeometry& A, const FPaperTerrainSpriteGeometry& B) { return B.DrawOrder > A.DrawOrder; });

#ifdef USE_SIMPLIFIED_POLYGON_COLLIDERS_FOR_SEGMENTS
	// For whatever is remaining
	if (CollisionPolygonPoints.Num() > 0)
	{
		InsertConvexCollisionDataFromPolygon(CollisionPolygonPoints);
	}
#endif
}

// Create an extruded convex hull resulting from extruding edges by the amount defined in EdgeExtrudeAmount
// Each edge is extruded in the normal direction to the edge
static void CreateExtrudedConvexHull(TArray<FVector2D>& OutConvexHull, TArray<FVector2D>& SourcePoints, TArray<float> EdgeExtrudeAmount)
{
	TArray<FVector2D> ExtrudedPoints;
	int SourcePointsCount = SourcePoints.Num();
	for (int i = 0; i < SourcePointsCount; ++i)
	{
		FVector2D& A = SourcePoints[i];
		FVector2D& B = SourcePoints[(i + 1) % SourcePointsCount];
		FVector2D N(B.Y - A.Y, A.X - B.X);
		N.Normalize();
		float extrude = EdgeExtrudeAmount[i];

		// Each edge is pushed forwards and backwards, and the points added
		ExtrudedPoints.Add(A + N * extrude);
		ExtrudedPoints.Add(A - N * extrude);
		ExtrudedPoints.Add(B + N * extrude);
		ExtrudedPoints.Add(B - N * extrude);
	}

	FGeomTools2D::GenerateConvexHullFromPoints(OutConvexHull, ExtrudedPoints);
}

void UPaperTerrainComponent::GenerateFillRenderDataFromPolygon(const class UPaperSprite* NewSprite, FSpriteDrawCallRecord& FillDrawCall, const FVector2D& TextureSize, const TArray<FVector2D>& TriangulatedPolygonVertices)
{
	const FVector2D TextureSizeInUnits = TextureSize * NewSprite->GetUnrealUnitsPerPixel();

	// Pack vertex data
	if (TriangulatedPolygonVertices.Num() >= 3)
	{
		for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < TriangulatedPolygonVertices.Num(); ++TriangleVertexIndex)
		{
			const FVector2D& TriangleVertex = TriangulatedPolygonVertices[TriangleVertexIndex];
			new (FillDrawCall.RenderVerts) FVector4(TriangleVertex.X, TriangleVertex.Y, TriangleVertex.X / TextureSizeInUnits.X, -TriangleVertex.Y / TextureSizeInUnits.Y);
		}
	}
}

void UPaperTerrainComponent::GenerateCollisionDataFromPolygon(const TArray<FVector2D>& SplinePolyVertices2D, const TArray<float>& TerrainOffsets, const TArray<FVector2D>& TriangulatedPolygonVertices)
{
	if (CachedBodySetup != nullptr && TriangulatedPolygonVertices.Num() >= 3)
	{
		// Generate polygon collider
		TArray<TArray<FVector2D>> ConvexHulls;
		FGeomTools2D::GenerateConvexPolygonsFromTriangles(ConvexHulls, TriangulatedPolygonVertices);
		TArray<float> ConvexHullEdgeExtrusionAmount;

		for (TArray<FVector2D> ConvexHull : ConvexHulls)
		{
			ConvexHullEdgeExtrusionAmount.Empty(ConvexHull.Num());

			// Find distances for each edge in this convex hull from the pair of points forming the edge
			// Not all edges will match edges in the original concave geometry, eg. newly created internal edges
			//TODO: Speed this up by using indices / vertex & edge identifiers instead of brute force search
			for (int ConvexHullPoint = 0; ConvexHullPoint < ConvexHull.Num(); ++ConvexHullPoint)
			{
				const FVector2D& A = ConvexHull[ConvexHullPoint];
				const FVector2D& B = ConvexHull[(ConvexHullPoint + 1) % ConvexHull.Num()];
				bool bFound = false;
				for (int Vertex = 0; Vertex < SplinePolyVertices2D.Num(); ++Vertex)
				{
					// The winding is might be different to the source polygon, compare both ways
					int NextVertexIndex = (Vertex + 1) % SplinePolyVertices2D.Num();
					if ((A.Equals(SplinePolyVertices2D[Vertex], THRESH_POINTS_ARE_SAME) && B.Equals(SplinePolyVertices2D[NextVertexIndex], THRESH_POINTS_ARE_SAME)) ||
						(B.Equals(SplinePolyVertices2D[Vertex], THRESH_POINTS_ARE_SAME) && A.Equals(SplinePolyVertices2D[NextVertexIndex], THRESH_POINTS_ARE_SAME)))
					{
						// Found an edge that matches the 2 vertex points
						ConvexHullEdgeExtrusionAmount.Add(TerrainOffsets[Vertex]);
						bFound = true;
						break;
					}
				}
				if (!bFound)
				{
					// Couldn't find this edge in the original polygon 
					ConvexHullEdgeExtrusionAmount.Add(0);
				}
			}
				
			TArray<FVector2D> ExtrudedConvexHull;
			CreateExtrudedConvexHull(ExtrudedConvexHull, ConvexHull, ConvexHullEdgeExtrusionAmount);
//			ExtrudedConvexHull = ConvexHull;

			// Generate convex hull
			FKConvexElem Convex;
			for (int J = 0; J < ExtrudedConvexHull.Num(); ++J)
			{
				FVector2D& Vert = ExtrudedConvexHull[J];
				new(Convex.VertexData) FVector(Vert.X, -0.5f * CollisionThickness, Vert.Y);
				new(Convex.VertexData) FVector(Vert.X, 0.5f * CollisionThickness, Vert.Y);
			}
			Convex.UpdateElemBox();

			CachedBodySetup->AggGeom.ConvexElems.Add(Convex);
		}
	}
}

void UPaperTerrainComponent::InsertConvexCollisionDataFromPolygon(const TArray<FVector2D>& ClosedPolyVertices2D)
{
	if (CachedBodySetup != nullptr && ClosedPolyVertices2D.Num() >= 3)
	{
		// Simplify polygon
		TArray<float> EmptyOffsetsList;
		TArray<FVector2D> LocalPolyVertices = ClosedPolyVertices2D;

		// The merge / weld threshold should not be any lower / less than half the thickness
		float PolygonThickness = (ClosedPolyVertices2D[0] - ClosedPolyVertices2D[ClosedPolyVertices2D.Num() - 1]).Size();
		float SimplifyThreshold = PolygonThickness * 0.5f;
		SimplifyPolygon(LocalPolyVertices, EmptyOffsetsList, SimplifyThreshold);

		// Always CCW and facing forward regardless of spline winding
		TArray<FVector2D> CorrectedSplineVertices;
		FGeomTools2D::CorrectPolygonWinding(CorrectedSplineVertices, LocalPolyVertices, false);

		TArray<FVector2D> TriangulatedPolygonVertices;
		if (!FGeomTools2D::TriangulatePoly(/*out*/TriangulatedPolygonVertices, CorrectedSplineVertices, false))
		{
			// Triangulation failed, try triangulating the original non simplified polygon
			CorrectedSplineVertices.Empty();
			FGeomTools2D::CorrectPolygonWinding(/*out*/CorrectedSplineVertices, ClosedPolyVertices2D, false);
			TriangulatedPolygonVertices.Empty();
			FGeomTools2D::TriangulatePoly(/*out*/TriangulatedPolygonVertices, CorrectedSplineVertices, false);
		}

		TArray<TArray<FVector2D>> ConvexHulls;
		FGeomTools2D::GenerateConvexPolygonsFromTriangles(ConvexHulls, TriangulatedPolygonVertices);

		for (TArray<FVector2D> ConvexHull : ConvexHulls)
		{
			FKConvexElem Convex;
			for (int J = 0; J < ConvexHull.Num(); ++J)
			{
				FVector2D& Vert = ConvexHull[J];
				new(Convex.VertexData) FVector(Vert.X, -0.5f * CollisionThickness, Vert.Y);
				new(Convex.VertexData) FVector(Vert.X, 0.5f * CollisionThickness, Vert.Y);
			}
			Convex.UpdateElemBox();

			CachedBodySetup->AggGeom.ConvexElems.Add(Convex);
		}
	}
}

void UPaperTerrainComponent::SetTerrainColor(FLinearColor NewColor)
{
	// Can't set color on a static component
	if (AreDynamicDataChangesAllowed() && (TerrainColor != NewColor))
	{
		TerrainColor = NewColor;

		const FColor TerrainColorQuantized = TerrainColor.ToFColor(/*bSRGB=*/ false);

		// Update the color in the game-thread copy of the render geometry
		for (FPaperTerrainSpriteGeometry& Batch : GeneratedSpriteGeometry)
		{
			for (FSpriteDrawCallRecord& DrawCall : Batch.Records)
			{
				DrawCall.Color = TerrainColorQuantized;
			}
		}

		// Update the render thread copy
		RecreateRenderState_Concurrent();
	}
}

