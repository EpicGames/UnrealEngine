// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTerrainMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTerrainMaterial)

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// UPaperTerrainMaterial

UPaperTerrainMaterial::UPaperTerrainMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Add a default rule
	FPaperTerrainMaterialRule& DefaultRule = *new (Rules) FPaperTerrainMaterialRule();
	DefaultRule.MinimumAngle = 0.0f;
	DefaultRule.MaximumAngle = 360.0f;
#if WITH_EDITORONLY_DATA
	DefaultRule.Description = LOCTEXT("TerrainDefaultRuleDesc", "Top");
#endif
}

#if WITH_EDITOR
void UPaperTerrainMaterial::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	//@TODO: PAPER2D: Ensure that the angle ranges of rules don't overlap
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

