// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTerrainSplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTerrainSplineComponent)

//////////////////////////////////////////////////////////////////////////
// UPaperTerrainSplineComponent

UPaperTerrainSplineComponent::UPaperTerrainSplineComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
void UPaperTerrainSplineComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	OnSplineEdited.ExecuteIfBound();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

