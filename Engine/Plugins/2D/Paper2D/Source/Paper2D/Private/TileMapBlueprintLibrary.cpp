// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapBlueprintLibrary.h"
#include "PaperTileLayer.h"
#include "PaperTileSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TileMapBlueprintLibrary)

//////////////////////////////////////////////////////////////////////////

FName UTileMapBlueprintLibrary::GetTileUserData(FPaperTileInfo Tile)
{
	if (Tile.TileSet != nullptr)
	{
		return Tile.TileSet->GetTileUserData(Tile.GetTileIndex());
	}
	else
	{
		return NAME_None;
	}
}

FTransform UTileMapBlueprintLibrary::GetTileTransform(FPaperTileInfo Tile)
{
	return UPaperTileLayer::GetTileTransform(Tile.GetFlagsAsIndex());
}

void UTileMapBlueprintLibrary::BreakTile(FPaperTileInfo Tile, int32& TileIndex, UPaperTileSet*& TileSet, bool& bFlipH, bool& bFlipV, bool& bFlipD)
{
	TileIndex = Tile.GetTileIndex();
	TileSet = Tile.TileSet;
	bFlipH = Tile.HasFlag(EPaperTileFlags::FlipHorizontal);
	bFlipV = Tile.HasFlag(EPaperTileFlags::FlipVertical);
	bFlipD = Tile.HasFlag(EPaperTileFlags::FlipDiagonal);
}

FPaperTileInfo UTileMapBlueprintLibrary::MakeTile(int32 TileIndex, UPaperTileSet* TileSet, bool bFlipH, bool bFlipV, bool bFlipD)
{
	FPaperTileInfo Result;
	Result.TileSet = TileSet;
	Result.PackedTileIndex = TileIndex;
	Result.SetFlagValue(EPaperTileFlags::FlipHorizontal, bFlipH);
	Result.SetFlagValue(EPaperTileFlags::FlipVertical, bFlipV);
	Result.SetFlagValue(EPaperTileFlags::FlipDiagonal, bFlipD);
	return Result;
}

