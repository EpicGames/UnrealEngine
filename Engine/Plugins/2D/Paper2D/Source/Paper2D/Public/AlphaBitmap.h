// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"

//////////////////////////////////////////////////////////////////////////
// FAlphaBitmap

struct FAlphaBitmap
{
	FAlphaBitmap(UTexture* SourceTexture, uint8 InDefaultValue = 0)
		: Width(0)
		, Height(0)
		, DefaultValue(InDefaultValue)
	{
		ExtractFromTexture(SourceTexture);
	}

	void ExtractFromTexture(UTexture* SourceTexture)
	{
#if WITH_EDITORONLY_DATA
		// use the source art if it exists
		FTextureSource* TextureSource = nullptr;
		if ((SourceTexture != nullptr) && SourceTexture->Source.IsValid())
		{
			switch (SourceTexture->Source.GetFormat())
			{
			case TSF_G8:
			case TSF_BGRA8:
				TextureSource = &(SourceTexture->Source);
				break;
			default:
				break;
			};
		}

		if (TextureSource != nullptr)
		{
			TArray64<uint8> TextureRawData;
			verify( TextureSource->GetMipData(TextureRawData, 0) );
			int32 BytesPerPixel = TextureSource->GetBytesPerPixel();
			ETextureSourceFormat PixelFormat = TextureSource->GetFormat();
			Width = TextureSource->GetSizeX();
			Height = TextureSource->GetSizeY();
			RawData.SetNumZeroed(Width * Height);

			for (int Y = 0; Y < Height; ++Y)
			{
				for (int X = 0; X < Width; ++X)
				{
					int32 PixelByteOffset = (X + Y * Width) * BytesPerPixel;
					const uint8* PixelPtr = TextureRawData.GetData() + PixelByteOffset;
					FColor Color;
					if (PixelFormat == TSF_BGRA8)
					{
						Color = *((FColor*)PixelPtr);
					}
					else
					{
						checkSlow(PixelFormat == TSF_G8);
						const uint8 Intensity = *PixelPtr;
						Color = FColor(Intensity, Intensity, Intensity, Intensity);
					}

					RawData[Y * Width + X] = Color.A;
				}
			}
		}
#endif
	}

	// Create an empty bitmap
	FAlphaBitmap(int InWidth, int InHeight, int InDefaultValue = 0)
		: Width(InWidth)
		, Height(InHeight)
		, DefaultValue(InDefaultValue)
	{
		RawData.SetNumZeroed(Width * Height);
		ClearToDefaultValue();
	}

	bool IsValid() const
	{
		return (Width != 0) && (Height != 0);
	}

	uint8 GetPixel(int X, int Y) const
	{
		if ((X >= 0) && (X < Width) && (Y >= 0) && (Y < Height))
		{
			return RawData[Y * Width + X];
		}
		else
		{
			return DefaultValue;
		}
	}

	void SetPixel(int X, int Y, uint8 Value)
	{
		if ((X >= 0) && (X < Width) && (Y >= 0) && (Y < Height))
		{
			RawData[Y * Width + X] = Value;
		}
	}

	void ClearToDefaultValue()
	{
		int RawDataCount = Width * Height;
		for (int RawDataIndex = 0; RawDataIndex < RawDataCount; ++RawDataIndex)
		{
			RawData[RawDataIndex] = DefaultValue;
		}
	}

	bool IsColumnEqual(int X, int Y0, int Y1, int Target) const
	{
		for (int Y = Y0; Y <= Y1; ++Y)
		{
			if (GetPixel(X, Y) != Target)
			{
				return false;
			}
		}
		return true;
	}

	bool IsRowEqual(int X0, int X1, int Y, int Target) const
	{
		for (int X = X0; X <= X1; ++X)
		{
			if (GetPixel(X, Y) != Target)
			{
				return false;
			}
		}
		return true;
	}

	// Is the rectangle empty (X0..X1, Y0..Y1 inclusive)
	bool IsRegionEqual(int32 X0, int32 Y0, int32 X1, int32 Y1, int32 Target) const
	{
		bool bEqual = true;
		for (int32 Y = Y0; (Y < Y1) && bEqual; ++Y)
		{
			bEqual = bEqual && IsRowEqual(X0, X1, Y, Target);
		}
		return bEqual;
	}

	bool IsColumnEmpty(int X, int Y0, int Y1) const
	{
		return IsColumnEqual(X, Y0, Y1, 0);
	}

	bool IsRowEmpty(int X0, int X1, int Y) const
	{
		return IsRowEqual(X0, X1, Y, 0);
	}

	// Is the rectangle empty (X0..X1, Y0..Y1 inclusive)
	bool IsRegionEmpty(int32 X0, int32 Y0, int32 X1, int32 Y1)
	{
		return IsRegionEqual(X0, Y0, X1, Y1, 0);
	}

	// Returns the tight bounding box around pixels that are not 0
	void GetTightBounds(FIntPoint& OutOrigin, FIntPoint& OutDimension) const
	{
		OutOrigin.X = 0;
		OutOrigin.Y = 0;
		OutDimension.X = Width;
		OutDimension.Y = Height;
		TightenBounds(OutOrigin, OutDimension);
	}

	// Returns the tight bounding box around pixels that are not 0
	void TightenBounds(FIntPoint& InOutOrigin, FIntPoint& InOutDimension) const
	{
		int Top = InOutOrigin.Y;
		int Bottom = InOutOrigin.Y + InOutDimension.Y - 1;
		int Left = InOutOrigin.X;
		int Right = InOutOrigin.X + InOutDimension.X - 1;

		while (Top < Bottom && IsRowEmpty(Left, Right, Top))
		{
			++Top;
		}
		while (Bottom >= Top && IsRowEmpty(Left, Right, Bottom))
		{
			--Bottom;
		}
		while (Left < Right && IsColumnEmpty(Left, Top, Bottom))
		{
			++Left;
		}
		while (Right >= Left && IsColumnEmpty(Right, Top, Bottom))
		{
			--Right;
		}

		InOutOrigin.X = Left;
		InOutOrigin.Y = Top;
		InOutDimension.X = Right - Left + 1;
		InOutDimension.Y = Bottom - Top + 1;
	}

	// Sets pixels to 1 marking the rectangle outline
	void DrawRectOutline(int StartX, int StartY, int InWidth, int InHeight)
	{
		const int X0 = StartX;
		const int Y0 = StartY;
		const int X1 = StartX + InWidth - 1;
		const int Y1 = StartY + InHeight - 1;
		for (int Y = Y0; Y <= Y1; ++Y)
		{
			SetPixel(X0, Y, 1);
			SetPixel(X1, Y, 1);
		}
		for (int X = X0; X <= X1; ++X)
		{
			SetPixel(X, Y0, 1);
			SetPixel(X, Y1, 1);
		}
	}

	void FillRect(int StartX, int StartY, int InWidth, int InHeight)
	{
		const int X0 = StartX;
		const int Y0 = StartY;
		const int X1 = StartX + InWidth - 1;
		const int Y1 = StartY + InHeight - 1;
		for (int Y = Y0; Y <= Y1; ++Y)
		{
			for (int X = X0; X <= X1; ++X)
			{
				SetPixel(X, Y, 1);
			}
		}
	}

	// Wind through the bitmap from StartX, StartY to find the closest hit point
	bool FoundClosestValidPoint(int StartX, int StartY, const int MaxAllowedSearchDistance, FIntPoint& OutHitPoint) const
	{
		// Constrain the point within the image bounds
		StartX = FMath::Clamp(StartX, 0, Width - 1);
		StartY = FMath::Clamp(StartY, 0, Height - 1);
		OutHitPoint.X = StartX;
		OutHitPoint.Y = StartY;

		// Should probably calculate a better max based on StartX and StartY
		int RequiredSearchDistance = (FMath::Max(Width, Height) + 1) / 2;
		int MaxSearchDistance = FMath::Min(RequiredSearchDistance, MaxAllowedSearchDistance);
		int SearchDistance = 0;
		while (SearchDistance < MaxSearchDistance)
		{
			int X0 = FMath::Max(StartX - SearchDistance, 0);
			int X1 = FMath::Min(StartX + SearchDistance, Width - 1);
			int Y0 = FMath::Max(StartY - SearchDistance, 0);
			int Y1 = FMath::Min(StartY + SearchDistance, Height - 1);
			// Search along the rectangular edge
			for (int Y = Y0; Y <= Y1; ++Y)
			{
				if (GetPixel(X0, Y) != 0) 
				{
					OutHitPoint.X = X0;
					OutHitPoint.Y = Y;
					return true;
				}
				if (GetPixel(X1, Y) != 0) 
				{
					OutHitPoint.X = X1;
					OutHitPoint.Y = Y;
					return true;
				}
			}
			for (int X = X0; X <= X1; ++X)
			{
				if (GetPixel(X, Y0) != 0) 
				{
					OutHitPoint.X = X;
					OutHitPoint.Y = Y0;
					return true;
				}
				if (GetPixel(X, Y1) != 0) 
				{
					OutHitPoint.X = X;
					OutHitPoint.Y = Y1;
					return true;
				}
			}
			SearchDistance += 1;
		}
		return false;
	}

	void ThresholdImage(int32 AlphaThreshold)
	{
		for (uint8& Pixel : RawData)
		{
			if (Pixel > AlphaThreshold)
			{
				Pixel = 1;
			}
			else
			{
				Pixel = DefaultValue;
			}
		}
	}

	// Flushes values smaller or equal to LowAlphaThreshold to 0, and values greater or equal to HighAlphaThreshold to 255
	void ThresholdImageBothWays(int32 LowAlphaThreshold, int32 HighAlphaThreshold)
	{
		for (uint8& Pixel : RawData)
		{
			if (Pixel <= LowAlphaThreshold)
			{
				Pixel = 0;
			}
			else if (Pixel >= HighAlphaThreshold)
			{
				Pixel = 255;
			}
		}
	}

	// Checks the image to determine if it is suitable for opaque, masked or translucent rendering
	void AnalyzeImage(int32 StartX, int32 StartY, int32 InWidth, int32 InHeight, bool& bOutHasZeros, bool& bOutHasIntermediateValues) const
	{
		bOutHasZeros = false;
		bOutHasIntermediateValues = false;

		const int32 X0 = StartX;
		const int32 Y0 = StartY;
		const int32 X1 = StartX + InWidth;
		const int32 Y1 = StartY + InHeight;
		for (int Y = Y0; Y < Y1; ++Y)
		{
			for (int X = X0; X < X1; ++X)
			{
				const uint8 Value = GetPixel(X, Y);

				if ((Value > 0) && (Value < 255))
				{
					bOutHasIntermediateValues = true;
				}

				if (Value == 0)
				{
					bOutHasZeros = true;
				}
			}
		}
	}

public:
	TArray<uint8> RawData;
	int32 Width;
	int32 Height;
	uint8 DefaultValue;
};
