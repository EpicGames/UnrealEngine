// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "MaterialExpressionSpriteTextureSampler.generated.h"

#define UE_API PAPER2D_API

struct FPropertyChangedEvent;

// This is a texture sampler 2D with a special automatically defined parameter name. The texture specified here will be replaced by the SourceTexture or an AdditionalSourceTextures entry of a Paper2D sprite if this material is used on a sprite.
UCLASS(MinimalAPI, hideCategories=(MaterialExpressionTextureSampleParameter, MaterialExpressionSpriteTextureSampler))
class UMaterialExpressionSpriteTextureSampler : public UMaterialExpressionTextureSampleParameter2D
{
	GENERATED_UCLASS_BODY()

	// Is this a sampler for the default SourceTexture or the AdditionalSourceTextures list?
	UPROPERTY(EditAnywhere, Category=Paper2D)
	bool bSampleAdditionalTextures;

	// This is the slot index into the AdditionalSourceTextures array
	UPROPERTY(EditAnywhere, Category=Paper2D, meta=(EditCondition=bSampleAdditionalTextures))
	int32 AdditionalSlotIndex;

	// Friendly label for the texture slot, displayed in the Sprite Editor if not empty
	UPROPERTY(EditAnywhere, Category=Paper2D)
	FText SlotDisplayName;

	// UMaterialExpression interface
#if WITH_EDITOR
	UE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual bool CanRenameNode() const override;
	UE_API virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End of UMaterialExpression interface
};

#undef UE_API
