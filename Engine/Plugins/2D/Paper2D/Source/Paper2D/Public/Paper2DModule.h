// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Modules/ModuleInterface.h"
#include "Stats/Stats.h"

enum class EStatFlags : uint8;

#define PAPER2D_MODULE_NAME "Paper2D"

//////////////////////////////////////////////////////////////////////////
// IPaper2DModuleInterface

class IPaper2DModuleInterface : public IModuleInterface
{
};

//////////////////////////////////////////////////////////////////////////

// NOTE: Experimental: Do not adjust these values, many aspects of the editor or runtime assume PaperX=Right(+X) PaperY=Up(+Z) PaperZ=PaperX cross PaperY=Out(-Y)
// The ability to adjust these values may be removed entirely in the future, or made to work properly
//@TODO: PAPER2D: Synchronize with the Box2D axes in the engine (or remove the ability to customize them)

// The 3D vector that corresponds to the sprite X axis
extern PAPER2D_API FVector PaperAxisX;

// The 3D vector that corresponds to the sprite Y axis
extern PAPER2D_API FVector PaperAxisY;

// The 3D vector that corresponds to the sprite Z axis
extern PAPER2D_API FVector PaperAxisZ;

//////////////////////////////////////////////////////////////////////////
// Paper2D stats

DECLARE_STATS_GROUP(TEXT("Paper2D"), STATGROUP_Paper2D, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("SetSprite (RT)"), STAT_PaperRender_SetSpriteRT, STATGROUP_Paper2D, PAPER2D_API);
