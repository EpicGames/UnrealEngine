// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "PaperFlipbook.h"
#include "PaperFlipbookFactory.generated.h"

#define UE_API PAPER2DEDITOR_API

/**
 * Factory for flipbooks
 */

UCLASS(MinimalAPI)
class UPaperFlipbookFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	TArray<FPaperFlipbookKeyFrame> KeyFrames;

	// UFactory interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};

#undef UE_API
