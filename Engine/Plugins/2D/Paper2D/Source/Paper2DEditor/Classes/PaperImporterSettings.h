// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperImporterSettings.generated.h"

#define UE_API PAPER2DEDITOR_API

class UTexture;
enum TextureCompressionSettings : int;
enum TextureGroup : int;

class UMaterialInterface;
class UPaperTileMap;
class UPaperTileSet;
class UTexture2D;
struct FSpriteAssetInitParameters;

enum class ESpriteInitMaterialLightingMode : uint8
{
	// Use the default in the importer settings
	Automatic,

	// Force unlit
	ForceUnlit,

	// Force lit
	ForceLit
};

enum class ESpriteInitMaterialType : uint8
{
	// Leave the material alone
	LeaveAsIs,

	// Use the default in the importer settings (typically auto-analyze)
	Automatic,

	// Force masked
	Masked,

	// Force translucent
	Translucent,

	// Force opaque
	Opaque
};

/**
 * Implements the settings for imported Paper2D assets, such as sprite sheet textures.
 */
UCLASS(MinimalAPI, config=Editor, defaultconfig)
class UPaperImporterSettings : public UObject
{
	GENERATED_BODY()

protected:
	// Should the source texture be scanned when creating new sprites to determine the appropriate material? (if false, the Default Masked Material is always used)
	UPROPERTY(config, EditAnywhere, Category=NewAssetSettings)
	bool bPickBestMaterialWhenCreatingSprites;

	// Should the source texture be scanned when creating new tile maps (from a tile set or via importing) to determine the appropriate material? (if false, the Default Masked Material is always used)
	UPROPERTY(config, EditAnywhere, Category=NewAssetSettings)
	bool bPickBestMaterialWhenCreatingTileMaps;

	// Can opaque materials be applied as part of the 'best material' analysis?
	UPROPERTY(config, EditAnywhere, Category=NewAssetSettings)
	bool bAnalysisCanUseOpaque;

	// The default scaling factor between pixels and Unreal units (cm) to use for newly created sprite assets (e.g., 0.64 would make a 64 pixel wide sprite take up 100 cm)
	UPROPERTY(config, EditAnywhere, Category=NewAssetSettings)
	float DefaultPixelsPerUnrealUnit;

	// A list of default suffixes to use when looking for associated normal maps while importing sprites or creating sprites from textures
	UPROPERTY(config, EditAnywhere, Category=ImportSettings)
	TArray<FString> NormalMapTextureSuffixes;

	// The default suffix to remove (if present) from a texture name before looking for an associated normal map using NormalMapTextureSuffixes
	UPROPERTY(config, EditAnywhere, Category=ImportSettings)
	TArray<FString> BaseMapTextureSuffixes;

	// The default texture group for imported sprite textures, tile sheets, etc... (typically set to UI for 'modern 2D' or 2D pixels for 'retro 2D')
	UPROPERTY(config, EditAnywhere, Category=ImportSettings)
	TEnumAsByte<TextureGroup> DefaultSpriteTextureGroup;

	// Should texture compression settings be overridden on imported sprite textures, tile sheets, etc...?
	UPROPERTY(config, EditAnywhere, Category=ImportSettings)
	bool bOverrideTextureCompression;
	
	// Compression settings to use when building the texture.
	// The default texture group for imported sprite textures, tile sheets, etc... (typically set to UI for 'modern 2D' or 2D pixels for 'retro 2D')
	UPROPERTY(config, EditAnywhere, Category=ImportSettings, meta=(EditCondition=bOverrideTextureCompression))
	TEnumAsByte<TextureCompressionSettings> DefaultSpriteTextureCompression;

	// The unlit default masked material for newly created sprites (masked means binary opacity: things are either opaque or see-thru, with nothing in between)
	UPROPERTY(config, EditAnywhere, Category=MaterialSettings, meta=(AllowedClasses="/Script/Engine.MaterialInterface", DisplayName="Unlit Default Masked Material"))
	FSoftObjectPath UnlitDefaultMaskedMaterialName;

	// The unlit default translucent material for newly created sprites (translucent means smooth opacity which can vary continuously from 0..1, but translucent rendering is more expensive that opaque or masked rendering and has different sorting rules)
	UPROPERTY(config, EditAnywhere, Category=MaterialSettings, meta=(AllowedClasses="/Script/Engine.MaterialInterface", DisplayName="Unlit Default Translucent Material"))
	FSoftObjectPath UnlitDefaultTranslucentMaterialName;

	// The unlit default opaque material for newly created sprites
	UPROPERTY(config, EditAnywhere, Category=MaterialSettings, meta=(AllowedClasses="/Script/Engine.MaterialInterface", DisplayName="Unlit Default Opaque Sprite Material"))
	FSoftObjectPath UnlitDefaultOpaqueMaterialName;

	// The lit default masked material for newly created sprites (masked means binary opacity: things are either opaque or see-thru, with nothing in between)
	UPROPERTY(config, EditAnywhere, Category=MaterialSettings, meta=(AllowedClasses="/Script/Engine.MaterialInterface", DisplayName="Lit Default Masked Material"))
	FSoftObjectPath LitDefaultMaskedMaterialName;

	// The lit default translucent material for newly created sprites (translucent means smooth opacity which can vary continuously from 0..1, but translucent rendering is more expensive that opaque or masked rendering and has different sorting rules)
	UPROPERTY(config, EditAnywhere, Category=MaterialSettings, meta=(AllowedClasses="/Script/Engine.MaterialInterface", DisplayName="Lit Default Translucent Material"))
	FSoftObjectPath LitDefaultTranslucentMaterialName;

	// The lit default opaque material for newly created sprites
	UPROPERTY(config, EditAnywhere, Category=MaterialSettings, meta=(AllowedClasses="/Script/Engine.MaterialInterface", DisplayName="Lit Default Opaque Material"))
	FSoftObjectPath LitDefaultOpaqueMaterialName;

public:
	UE_API UPaperImporterSettings();

	// Should the source texture be scanned when creating new tile maps (from a tile set or via importing) to determine the appropriate material? (if false, the Default Masked Material is always used)
	bool ShouldPickBestMaterialWhenCreatingTileMaps() const { return bPickBestMaterialWhenCreatingTileMaps; }

	// Removes the suffix from the specified name if it matches something in BaseMapTextureSuffixes
	UE_API FString RemoveSuffixFromBaseMapName(const FString& InName) const;

	// Generates names to test for a normal map using NormalMapTextureSuffixes
	UE_API void GenerateNormalMapNamesToTest(const FString& InRoot, TArray<FString>& InOutNames) const;

	// Applies the compression settings to the specified texture
	UE_API void ApplyTextureSettings(UTexture2D* Texture) const;

	// Fills out the sprite init parameters with the default settings given the desired material type and lighting mode (which can both be automatic)
	// Note: This should be called after the texture has been set, as that will be analyzed if the lighting/material type flags are set to automatic
	UE_API void ApplySettingsForSpriteInit(FSpriteAssetInitParameters& InitParams, ESpriteInitMaterialLightingMode LightingMode = ESpriteInitMaterialLightingMode::Automatic, ESpriteInitMaterialType MaterialTypeMode = ESpriteInitMaterialType::Automatic) const;

	// Fills out the tile map with the default settings given the desired material type and lighting mode (which can both be automatic)
	UE_API void ApplySettingsForTileMapInit(UPaperTileMap* TileMap, UPaperTileSet* DefaultTileSet = nullptr, ESpriteInitMaterialLightingMode LightingMode = ESpriteInitMaterialLightingMode::Automatic, ESpriteInitMaterialType MaterialTypeMode = ESpriteInitMaterialType::Automatic, bool bCreateEmptyLayer = true) const;

	// Analyzes the specified texture in the Offset..Offset+Dimensions region and returns the best kind of material
	// to represent the alpha content in the texture (typically masked or translucent, but can return opaque if bAnalysisCanUseOpaque is true
	// Note: Will return Automatic if the texture is null.
	UE_API ESpriteInitMaterialType AnalyzeTextureForDesiredMaterialType(UTexture* Texture, const FIntPoint& Offset, const FIntPoint& Dimensions) const;

	// Returns the default translucent material
	UE_API UMaterialInterface* GetDefaultTranslucentMaterial(bool bLit) const;

	// Returns the default opaque material
	UE_API UMaterialInterface* GetDefaultOpaqueMaterial(bool bLit) const;

	// Returns the default masked material
	UE_API UMaterialInterface* GetDefaultMaskedMaterial(bool bLit) const;

	// Returns the default material for the specified material type.
	// Input should be Masked, Opaque, or Translucent.  Automatic and LeaveAsIs will be treated like masked!
	UE_API UMaterialInterface* GetDefaultMaterial(ESpriteInitMaterialType MaterialType, bool bUseLitMaterial) const;

	// Returns the default pixels/uu setting
	float GetDefaultPixelsPerUnrealUnit() const { return DefaultPixelsPerUnrealUnit; }
};

#undef UE_API
