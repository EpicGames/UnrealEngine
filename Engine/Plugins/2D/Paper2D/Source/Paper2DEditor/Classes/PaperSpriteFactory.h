// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Factory for sprites
 */

#include "Factories/Factory.h"
#include "PaperSpriteFactory.generated.h"

UCLASS()
class UPaperSpriteFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Initial texture to create the sprite from (Can be nullptr)
	class UTexture2D* InitialTexture;

	// Set bUseSourceRegion to get it to use/set initial sourceUV and dimensions
	bool bUseSourceRegion;
	FIntPoint InitialSourceUV;
	FIntPoint InitialSourceDimension;

	// UFactory interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};
