// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "PaperSpriteThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS()
class UPaperSpriteThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// UThumbnailRenderer interface
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End of UThumbnailRenderer interface

protected:
	void DrawFrame(class UPaperSprite* Sprite, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, FBoxSphereBounds* OverrideRenderBounds);

	void DrawGrid(int32 X, int32 Y, uint32 Width, uint32 Height, FCanvas* Canvas);
};
