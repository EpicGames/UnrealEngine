// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Factory for tile maps
 */

#include "Factories/Factory.h"
#include "PaperTileMapFactory.generated.h"

UCLASS()
class UPaperTileMapFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Initial tile set to create the tile map from (Can be nullptr)
	class UPaperTileSet* InitialTileSet;

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};
