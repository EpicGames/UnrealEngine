// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Factory used to promote an instance-specific object from instance to asset by renaming it into an asset package
 */

#include "Factories/Factory.h"
#include "PaperTileMapPromotionFactory.generated.h"

UCLASS()
class UPaperTileMapPromotionFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Object being promoted to an asset
	UPROPERTY()
	TObjectPtr<class UPaperTileMap> AssetToRename;

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};
