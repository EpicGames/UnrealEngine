// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Factory for tile sets
 */

#include "Factories/Factory.h"
#include "PaperTileSetFactory.generated.h"

UCLASS()
class UPaperTileSetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Initial texture to create the tile set from (Can be nullptr)
	class UTexture2D* InitialTexture;

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};
