// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "PaperTileSetThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS()
class UPaperTileSetThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// UThumbnailRenderer interface
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End of UThumbnailRenderer interface
};
