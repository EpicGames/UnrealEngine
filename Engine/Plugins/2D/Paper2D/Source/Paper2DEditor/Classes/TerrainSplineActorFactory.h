// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "TerrainSplineActorFactory.generated.h"

class AActor;
struct FAssetData;

UCLASS()
class UTerrainSplineActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	// UActorFactory interface
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	// End of UActorFactory interface
};
