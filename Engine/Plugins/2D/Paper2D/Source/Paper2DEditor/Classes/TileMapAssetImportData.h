// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorFramework/AssetImportData.h"
#include "TileMapAssetImportData.generated.h"

#define UE_API PAPER2DEDITOR_API

class UPaperTileSet;

USTRUCT()
struct FTileSetImportMapping
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString SourceName;

	UPROPERTY()
	TWeakObjectPtr<class UPaperTileSet> ImportedTileSet;

	UPROPERTY()
	TWeakObjectPtr<class UTexture> ImportedTexture;
};

/**
 * Base class for import data and options used when importing a tile map
 */
UCLASS(MinimalAPI)
class UTileMapAssetImportData : public UAssetImportData
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FTileSetImportMapping> TileSetMap;

	static UE_API UTileMapAssetImportData* GetImportDataForTileMap(class UPaperTileMap* TileMap);
};

#undef UE_API
