// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Paper2DEditor : ModuleRules
{
	public Paper2DEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"AssetTools",
				"Slate",
				"SlateCore",
				"Engine",
				"InputCore",
				"ImageCore",
				"EditorFramework",
				"UnrealEd", // for AssetEditorSubsystem
				"KismetWidgets",
				"Kismet",  // for FWorkflowCentricApplication
				"PropertyEditor",
				"RenderCore",
				"Paper2D",
				"ContentBrowser",
				"WorkspaceMenuStructure",	
				"MeshPaint",
				"EditorWidgets",
				"Projects",
				"NavigationSystem",
				"ToolMenus",
				"ToolWidgets"
            });

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Json",
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
				"LevelEditor"
			});

		// This module needs AutoRTFM so disable the auto disable since its in a plugin.
		// The call to the delegate `OnObjectPropertyChanged` calls into here.
		bDisableAutoRTFMInstrumentation = false;
	}
}
