// Copyright Epic Games, Inc. All Rights Reserved.

#include "Atlasing/AtlasAssetTypeActions.h"

#include "PaperSpriteAtlas.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

//////////////////////////////////////////////////////////////////////////
// FAtlasAssetTypeActions

FAtlasAssetTypeActions::FAtlasAssetTypeActions(EAssetTypeCategories::Type InAssetCategory)
	: MyAssetCategory(InAssetCategory)
{
}

FText FAtlasAssetTypeActions::GetName() const
{
	return LOCTEXT("FAtlasAssetTypeActionsName", "Sprite Atlas Group");
}

FColor FAtlasAssetTypeActions::GetTypeColor() const
{
	return FColor::Cyan;
}

UClass* FAtlasAssetTypeActions::GetSupportedClass() const
{
	return UPaperSpriteAtlas::StaticClass();
}

void FAtlasAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	//@TODO: Atlas will need a custom editor at some point
	FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, EditWithinLevelEditor, InObjects);
}

uint32 FAtlasAssetTypeActions::GetCategories()
{
	return MyAssetCategory;
}

FText FAtlasAssetTypeActions::GetAssetDescription(const FAssetData& AssetData) const
{
	FString Description = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UPaperSpriteAtlas, AtlasDescription));
	if (!Description.IsEmpty())
	{
		Description.ReplaceInline(TEXT("\\n"), TEXT("\n"));
		return FText::FromString(MoveTemp(Description));
	}
	
	return FText::GetEmpty();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
