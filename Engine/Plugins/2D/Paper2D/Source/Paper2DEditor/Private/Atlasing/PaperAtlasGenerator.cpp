// Copyright Epic Games, Inc. All Rights Reserved.

#include "Atlasing/PaperAtlasGenerator.h"
#include "Paper2DEditorLog.h"
#include "Misc/MessageDialog.h"


//////////////////////////////////////////////////////////////////////////
// 

#include "Atlasing/PaperAtlasTextureHelpers.h"
#include "Atlasing/PaperAtlasHelpers.h"

//////////////////////////////////////////////////////////////////////////
// FPaperAtlasGenerator

void FPaperAtlasGenerator::HandleAssetChangedEvent(UPaperSpriteAtlas* Atlas)
{
	const int32 MaxAtlasDimension = 4096;
	static_assert(MaxAtlasDimension < 16384, "PaperAtlasGenerator MaxAtlasDimension exceeds multiplier in sort key");
	Atlas->MaxWidth = FMath::Clamp(Atlas->MaxWidth, 16, MaxAtlasDimension);
	Atlas->MaxHeight = FMath::Clamp(Atlas->MaxHeight, 16, MaxAtlasDimension);
	Atlas->MipCount = FPaperAtlasTextureHelpers::ClampMips(Atlas->MaxWidth, Atlas->MaxHeight, Atlas->MipCount);

	bool bTestForAtlasImprovement = true;

	// Have the atlas settings changed? This will trigger a full rebuild
	bool bAtlasDimensionsChanged = Atlas->MaxWidth != Atlas->BuiltWidth || Atlas->MaxHeight != Atlas->BuiltHeight;
	if (bAtlasDimensionsChanged || Atlas->Padding != Atlas->BuiltPadding)
	{
		Atlas->bRebuildAtlas = true;
	}

	// Save the settings this atlas is being built with
	Atlas->BuiltWidth = Atlas->MaxWidth;
	Atlas->BuiltHeight = Atlas->MaxHeight;
	Atlas->BuiltPadding = Atlas->Padding;

	// Force rebuild an atlas by deleting history
	if (Atlas->bRebuildAtlas)
	{
		Atlas->AtlasSlots.Empty();
		Atlas->NumIncrementalBuilds = 0;
		Atlas->bRebuildAtlas = false;
	}
	else
	{
		// Keep track of incremental builds
		Atlas->NumIncrementalBuilds++;
	}

	// Load all sprites that were used in building this atlas
	TArray<UPaperSprite*> SpritesInPreviousAtlas;
	LoadAllReferencedSprites(Atlas, SpritesInPreviousAtlas);

	// Load all sprites that currently reference this atlas
	TArray<UPaperSprite*> SpritesInNewAtlas;
	LoadAllSpritesWithAtlasGroupGUID(Atlas, SpritesInNewAtlas);

	// Find sprites removed from this atlas, but not null (i.e. deliberately removed from the atlas)
	bool bWasTextureRemoved = false;
	for (UPaperSprite* OriginalSprite : SpritesInPreviousAtlas)
	{
		if (!SpritesInNewAtlas.Contains(OriginalSprite))
		{
			RemoveTextureSlotWithSprite(Atlas, OriginalSprite);
			bWasTextureRemoved = true;
		}
	}

	if (bWasTextureRemoved)
	{
		MergeAdjacentRects(Atlas);
	}

	// Sort sprites by name first, so that we have a consistent sorting, before sorting by size, so that we have a repeatable
	// atlas generation.
	SpritesInNewAtlas.StableSort([](UPaperSprite& A, UPaperSprite& B) { return A.GetPathName() > B.GetPathName(); } );

	// Sort new sprites by size
	struct Local
	{
		static int32 SpriteSortValue(const UPaperSprite& Sprite)
		{
			const FVector2D SpriteSizeFloat = Sprite.GetSourceSize();
			const FIntPoint SpriteSize(FMath::TruncToInt(SpriteSizeFloat.X), FMath::TruncToInt(SpriteSizeFloat.Y));
			return SpriteSize.X * 16384 + SpriteSize.Y; // Sort wider textures first
		}
	};
	SpritesInNewAtlas.StableSort( [](UPaperSprite& A, UPaperSprite& B) { return Local::SpriteSortValue(A) > Local::SpriteSortValue(B); } );

	// Add new sprites
	TArray<FPaperSpriteAtlasSlot> ImprovementTestAtlas; // A second atlas to compare wastage
	for (UPaperSprite* Sprite : SpritesInNewAtlas)
	{
		const FVector2D SpriteSizeFloat = Sprite->GetSourceSize();
		const FIntPoint SpriteSize(FMath::TruncToInt(SpriteSizeFloat.X), FMath::TruncToInt(SpriteSizeFloat.Y));
		const FIntPoint PaddedSpriteSize(SpriteSize.X + Atlas->Padding * 2, SpriteSize.Y + Atlas->Padding * 2);

		if (Sprite->GetSourceTexture() == nullptr)
		{
			UE_LOGF(LogPaper2DEditor, Error, "Sprite %ls has no source texture and cannot be packed", *(Sprite->GetPathName()));
			continue;
		}

		//TODO: Padding should only be considered by the slot finder to allow atlasing
		// textures flush to the edge
		if ((PaddedSpriteSize.X > Atlas->MaxWidth) || (PaddedSpriteSize.Y > Atlas->MaxHeight))
		{
			// This sprite cannot ever fit into an atlas page
			UE_LOGF(LogPaper2DEditor, Error, "Sprite %ls (%d x %d) can never fit into atlas %ls (%d x %d) due to maximum page size restrictions",
				*(Sprite->GetPathName()),
				SpriteSize.X,
				SpriteSize.Y,
				*(Atlas->GetPathName()),
				Atlas->MaxWidth,
				Atlas->MaxHeight);
			continue;
		}

		bool bSpriteChanged = false;
		FindBestSlotForTexture(Atlas->AtlasSlots, Atlas->MaxWidth, Atlas->MaxHeight, Sprite, PaddedSpriteSize.X, PaddedSpriteSize.Y, bSpriteChanged);
		if (bSpriteChanged)
		{
			//TODO: keep track of sprite moving about in the atlas?
		}

		if (bTestForAtlasImprovement)
		{
			// Pack into a second test atlas in parallel
			bool bUnused = false;
			FindBestSlotForTexture(ImprovementTestAtlas, Atlas->MaxWidth, Atlas->MaxHeight, Sprite, PaddedSpriteSize.X, PaddedSpriteSize.Y, bSpriteChanged);
		}
	}

	// Test for improvement if necessary
	// An "improvement" is defined as less atlases overall, but could be extended to check for atlas area once we support resizing atlases
	if (bTestForAtlasImprovement)
	{
		if (NumTexturesUsedInAtlasSlots(ImprovementTestAtlas) < NumTexturesUsedInAtlasSlots(Atlas->AtlasSlots))
		{
			const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::YesNo, NSLOCTEXT("PaperEditor", "AtlasPackingImprovement", "Atlas packing can be improved significantly by repacking the entire atlas. This will require re-saving most or all sprites in this atlas.\nDo you want to do this now?"));
			if (Choice == EAppReturnType::Yes)
			{
				// Likely to mark most sprites dirty
				Atlas->AtlasSlots = ImprovementTestAtlas;
			}
		}
	}

	// Update atlas textures
	TArray<UTexture*> RemappedAtlasTextures; // Will only contain valid and used textures after this
	TArray<bool> RemappedAtlasForceDirty; // If any atlases were missing (due to user deleting bits), all the dependent sprites are considered dirty
	TArray<int32> AtlasLookupIndex; // To correct mismatched atlas numbers, gaps in arrays, etc
	for (FPaperSpriteAtlasSlot& Slot : Atlas->AtlasSlots)
	{
		// Get the "correct" atlas index
		while (!AtlasLookupIndex.IsValidIndex(Slot.AtlasIndex))
		{
			AtlasLookupIndex.Add(-1); // add an uninitialized lookup entry
			RemappedAtlasForceDirty.Add(false);
		}

		if (AtlasLookupIndex[Slot.AtlasIndex] == -1)
		{
			AtlasLookupIndex[Slot.AtlasIndex] = RemappedAtlasTextures.Num();
			if (Atlas->GeneratedTextures.IsValidIndex(Slot.AtlasIndex) && Atlas->GeneratedTextures[Slot.AtlasIndex] != nullptr)
			{
				RemappedAtlasTextures.Add(Atlas->GeneratedTextures[Slot.AtlasIndex]);
				RemappedAtlasForceDirty.Add(false);
			}
			else
			{
				// The texture never existed - all sprites referencing this MUST be dirty and MUST be updated
				RemappedAtlasTextures.Add(NewObject<UTexture2D>(Atlas, NAME_None, RF_Public));
				RemappedAtlasForceDirty.Add(true);
			}
		}

		// Now atlas index refers to the RemappedAtlasTexures
		Slot.AtlasIndex = AtlasLookupIndex[Slot.AtlasIndex];
	}

	// Now fill the atlases and update sprite data where needed
	for (int32 AtlasIndex = 0; AtlasIndex < RemappedAtlasTextures.Num(); ++AtlasIndex)
	{
		UTexture2D* AtlasTexture = Cast<UTexture2D>(RemappedAtlasTextures[AtlasIndex]);
		check(AtlasTexture); // this should not be null

		// An atlas is ALSO forced dirty if the dimensions have changed
		// We're just grabbing a fixed atlas size here for now
		const int32 AtlasWidth = Atlas->MaxWidth;
		const int32 AtlasHeight = Atlas->MaxHeight;
		const int32 BytesPerPixel = sizeof(FColor);

		const bool bAtlasDirty = RemappedAtlasForceDirty[AtlasIndex] || (AtlasTexture->GetImportedSize() != FIntPoint(AtlasWidth, AtlasHeight));

		AtlasTexture->PreEditChange(nullptr);
		// Propagate texture settings
		AtlasTexture->CompressionSettings = Atlas->CompressionSettings;
		AtlasTexture->Filter = Atlas->Filter;
		AtlasTexture->AddressX = TextureAddress::TA_Clamp;
		AtlasTexture->AddressY = TextureAddress::TA_Clamp;
		AtlasTexture->MipGenSettings = TextureMipGenSettings::TMGS_LeaveExistingMips;

		// Allocate enough space for all mips
		TArray<uint8> AtlasTextureData;
		int32 CurrentAtlasWidth = AtlasWidth;
		int32 CurrentAtlasHeight = AtlasHeight;
		int32 TotalPixels = 0;
		for (int32 MipIndex = 0; MipIndex < Atlas->MipCount; ++MipIndex)
		{
			TotalPixels += CurrentAtlasHeight * CurrentAtlasWidth;
		}
		AtlasTextureData.AddZeroed(TotalPixels * BytesPerPixel);

		// Atlas sprites
		TArray<FPaperSpriteAtlasSlot> SlotsForAtlas;
		for (FPaperSpriteAtlasSlot& Slot : Atlas->AtlasSlots)
		{
			UPaperSprite* SpriteBeingBuilt = Slot.SpriteRef.Get();

			// Only for the sprites in this atlas
			if (Slot.AtlasIndex == AtlasIndex && SpriteBeingBuilt != nullptr)
			{
				SlotsForAtlas.Add(Slot);
				FPaperAtlasTextureHelpers::CopySpriteToAtlasTextureData(AtlasTextureData, AtlasWidth, AtlasHeight, BytesPerPixel, Atlas->PaddingType, Atlas->Padding, SpriteBeingBuilt, Slot);
			}
		}

		// Mipmaps
		if (Atlas->MipCount > 1)
		{
			FPaperAtlasTextureHelpers::GenerateMipChainARGB(SlotsForAtlas, AtlasTextureData, Atlas->MipCount, AtlasWidth, AtlasHeight);
		}


		AtlasTexture->Source.Init(AtlasWidth, AtlasHeight, /*NumSlices=*/ 1, Atlas->MipCount, ETextureSourceFormat::TSF_BGRA8, AtlasTextureData.GetData());
		AtlasTexture->PostEditChange();
	}

	// Rebuild sprites that have changed position in the atlas
	for (FPaperSpriteAtlasSlot& Slot : Atlas->AtlasSlots)
	{
		UPaperSprite* SpriteBeingBuilt = Slot.SpriteRef.Get();
		if (SpriteBeingBuilt)
		{
			UTexture2D* BakedSourceTexture = Cast<UTexture2D>(RemappedAtlasTextures[Slot.AtlasIndex]);
			FVector2D BakedSourceUV = FVector2D(Slot.X + Atlas->Padding, Slot.Y + Atlas->Padding);

			if (SpriteBeingBuilt->BakedSourceTexture != BakedSourceTexture || SpriteBeingBuilt->BakedSourceUV != BakedSourceUV || bAtlasDimensionsChanged)
			{
				SpriteBeingBuilt->Modify();
				SpriteBeingBuilt->BakedSourceTexture = Cast<UTexture2D>(RemappedAtlasTextures[Slot.AtlasIndex]);
				SpriteBeingBuilt->BakedSourceUV = FVector2D(Slot.X + Atlas->Padding, Slot.Y + Atlas->Padding);
				SpriteBeingBuilt->BakedSourceDimension = SpriteBeingBuilt->GetSourceSize();
				SpriteBeingBuilt->RebuildRenderData();

				// Propagate changes to sprites in scene
				SpriteBeingBuilt->PostEditChange();
			}
		}
	}

	// Finalize changes
	Atlas->GeneratedTextures = RemappedAtlasTextures;

	// Dirty
	Atlas->MarkPackageDirty();
}
