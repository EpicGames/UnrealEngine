// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperSprite.h"
#include "PaperSpriteAtlas.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

class IModuleInterface;

// Loads and returns a list of sprites referenced in the atlas group (i.e. sprites used in the previous build)
static void LoadAllReferencedSprites(UPaperSpriteAtlas* AtlasGroup, TArray<UPaperSprite*>& AllSprites)
{
	for (FPaperSpriteAtlasSlot& Slot : AtlasGroup->AtlasSlots)
	{
		FSoftObjectPath SpriteStringRef = Slot.SpriteRef.ToSoftObjectPath();
		if (!SpriteStringRef.ToString().IsEmpty())
		{
			UPaperSprite* Sprite = Cast<UPaperSprite>(StaticLoadObject(UPaperSprite::StaticClass(), nullptr, *SpriteStringRef.ToString(), nullptr, LOAD_None, nullptr));
			if (Sprite != nullptr)
			{
				AllSprites.Add(Sprite);
			}
		}
	}
}

// Find all sprites that reference the atlas and force them loaded
static void LoadAllSpritesWithAtlasGroupGUID(UPaperSpriteAtlas* Atlas, TArray<UPaperSprite*>& SpritesInNewAtlas)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> AssetList;
	TMultiMap<FName, FString> TagsAndValues;
	TagsAndValues.Add(TEXT("AtlasGroupGUID"), Atlas->AtlasGUID.ToString(EGuidFormats::Digits));
	AssetRegistryModule.Get().GetAssetsByTagValues(TagsAndValues, AssetList);

	for (const FAssetData& SpriteRef : AssetList)
	{
		if (UPaperSprite* Sprite = Cast<UPaperSprite>(SpriteRef.GetAsset()))
		{
			if (Sprite->GetAtlasGroup() == Atlas)
			{
				SpritesInNewAtlas.Add(Sprite);
			}
		}
	}
}

// Create an empty slot and return the slot index
// Returns -1 if the slot can't be created for any reason (eg. width and height too small)
static int32 CreateEmptySlot(TArray<FPaperSpriteAtlasSlot>& Slots, int32 AtlasIndex, int32 X, int32 Y, int32 Width, int32 Height)
{
	if (Width > 0 && Height > 0)
	{
		FPaperSpriteAtlasSlot& NewSlot = Slots.AddDefaulted_GetRef();
		NewSlot.AtlasIndex = AtlasIndex;
		NewSlot.X = X;
		NewSlot.Y = Y;
		NewSlot.Width = Width;
		NewSlot.Height = Height;
		return Slots.Num() - 1;
	}
	else
	{
		return -1;
	}
}

// Insert a sprite into a specified
// The sprite must be known to fit in the provided slot
static void InsertSpriteIntoSlot(TArray<FPaperSpriteAtlasSlot>& Slots, int SlotIndexToInsertInto, UPaperSprite* Sprite, int32 Width, int32 Height)
{
	FPaperSpriteAtlasSlot& Slot = Slots[SlotIndexToInsertInto];
	int32 RemainingWidth = Slot.Width - Width;
	int32 RemainingHeight = Slot.Height - Height;
	int32 OriginalSlotWidth = Slot.Width;
	int32 OriginalSlotHeight = Slot.Height;
	Slot.SpriteRef = Sprite;
	Slot.Width = Width;
	Slot.Height = Height;
	check(RemainingWidth >= 0 && RemainingHeight >= 0);

	if (RemainingHeight <= RemainingWidth)
	{
		CreateEmptySlot(Slots, Slot.AtlasIndex, Slot.X, Slot.Y + Height, Width, RemainingHeight);
		CreateEmptySlot(Slots, Slot.AtlasIndex, Slot.X + Width, Slot.Y, RemainingWidth, OriginalSlotHeight);
	}
	else
	{
		CreateEmptySlot(Slots, Slot.AtlasIndex, Slot.X, Slot.Y + Height, OriginalSlotWidth, RemainingHeight);
		CreateEmptySlot(Slots, Slot.AtlasIndex, Slot.X + Width, Slot.Y, RemainingWidth, Height);
	}
}

static void MergeAdjacentRects(UPaperSpriteAtlas* AtlasGroup)
{
	//@TODO: run through and merge
}

static void RemoveTextureSlotWithSprite(UPaperSpriteAtlas* AtlasGroup, UPaperSprite* Sprite)
{
	TArray<FPaperSpriteAtlasSlot>& AtlasSlots = AtlasGroup->AtlasSlots;
	for (int32 SlotIndex = 0; SlotIndex < AtlasSlots.Num(); ++SlotIndex)
	{
		FPaperSpriteAtlasSlot& AtlasSlot = AtlasSlots[SlotIndex];
		if (AtlasSlot.SpriteRef.Get() == Sprite)
		{
			AtlasSlot.SpriteRef = (UPaperSprite*)nullptr;
		}
	}
}

// Calculate the number of textures used in the atlas slots (atlas indices could be sparse) 
static int NumTexturesUsedInAtlasSlots(const TArray<FPaperSpriteAtlasSlot>& AtlasSlots)
{
	TSet<int> TextureIndex;
	for (const FPaperSpriteAtlasSlot& Slot : AtlasSlots)
	{
		TextureIndex.Add(Slot.AtlasIndex);
	}

	return TextureIndex.Num();
}

// FindBestSlotForTexture
// NOTE: Assumes all UPaperSprites have been loaded
static const FPaperSpriteAtlasSlot& FindBestSlotForTexture(TArray<FPaperSpriteAtlasSlot>& AtlasSlots, int32 AtlasWidth, int32 AtlasHeight, UPaperSprite* Sprite, int32 Width, int32 Height, bool& bOutSpriteChanged)
{
	int32 BestMatchIndex = -1;
	int32 BestMatchScore = MAX_int32; // Lower score = better match
	bool bImpossibleToFit = (Width > AtlasWidth || Height > AtlasHeight); // If impossible to fit, still continue to free up a slot if it already exists
	bOutSpriteChanged = false;

	// Asset pointer for the sprite we're currently trying to pack
	TSoftObjectPtr<UPaperSprite> SpriteAssetRef = Sprite;

	// 1. Find matching texture and see if we can fit it in the existing slot
	for (int32 SlotIndex = 0; SlotIndex < AtlasSlots.Num(); ++SlotIndex)
	{
		FPaperSpriteAtlasSlot& AtlasSlot = AtlasSlots[SlotIndex];
		if (AtlasSlot.SpriteRef == SpriteAssetRef)
		{
			if (Width <= AtlasSlot.Width && Height <= AtlasSlot.Height && !bImpossibleToFit)
			{
				// This is the best slot for an incremental update
				// The slot width and height should not be changed in case it grows to its original size later
				BestMatchIndex = SlotIndex;
				BestMatchScore = -1.0f;

				// The sprite UVs need updating if the actual dimensions have changed
				bOutSpriteChanged = Width != AtlasSlot.Width || Height != AtlasSlot.Height;
				break;
			}
			else
			{
				// The sprite used to be in this slot, but its too big for the slot now
				// Free the slot to be used later
				AtlasSlot.SpriteRef = (UPaperSprite*)nullptr;
				break;
			}
		}
	}

	// 2. Not found, try to fit in any empty slot
	if (BestMatchIndex == -1 && !bImpossibleToFit)
	{
		for (int32 SlotIndex = 0; SlotIndex < AtlasSlots.Num(); ++SlotIndex)
		{
			FPaperSpriteAtlasSlot& AtlasSlot = AtlasSlots[SlotIndex];
			if (Width <= AtlasSlot.Width && Height <= AtlasSlot.Height && AtlasSlot.SpriteRef.Get() == nullptr)
			{
				// The one with the least amount of wastage is the best bet?
				int32 RemainingWidth = AtlasSlot.Width - Width;
				int32 RemainingHeight = AtlasSlot.Height - Height;
				int32 CurrentScore = RemainingWidth * RemainingHeight;
				if (CurrentScore < BestMatchScore)
				{
					BestMatchIndex = SlotIndex;
					BestMatchScore = CurrentScore;
				}
			}
		}

		// Found a slot, insert the sprite
		// Split up and insert split new slots
		if (BestMatchIndex != -1)
		{
			InsertSpriteIntoSlot(AtlasSlots, BestMatchIndex, Sprite, Width, Height);
			bOutSpriteChanged = true;
		}
	}

	// Can't fit in any of the slots, need a new atlas
	if (BestMatchIndex == -1 && !bImpossibleToFit)
	{
		// Insert a new atlas at the end of the list, empty slots are cleared out later
		int32 LargestAtlasIndex = -1;
		for (int32 SlotIndex = 0; SlotIndex < AtlasSlots.Num(); ++SlotIndex)
		{
			const FPaperSpriteAtlasSlot& AtlasSlot = AtlasSlots[SlotIndex];
			LargestAtlasIndex = FMath::Max(LargestAtlasIndex, AtlasSlot.AtlasIndex);
		}

		// Insert an empty slot for the new atlas texture
		BestMatchIndex = CreateEmptySlot(AtlasSlots, LargestAtlasIndex + 1, 0, 0, AtlasWidth, AtlasHeight);
		check(BestMatchIndex != -1);

		// Insert the sprite into this newly created slot
		InsertSpriteIntoSlot(AtlasSlots, BestMatchIndex, Sprite, Width, Height);
		bOutSpriteChanged = true;
	}

	return AtlasSlots[BestMatchIndex];
}

