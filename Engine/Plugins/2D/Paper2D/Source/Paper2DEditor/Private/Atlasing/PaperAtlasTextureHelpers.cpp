// Copyright Epic Games, Inc. All Rights Reserved.

#include "Atlasing/PaperAtlasTextureHelpers.h"
#include "Paper2DEditorLog.h"
#include "PaperSprite.h"
#include "PaperSpriteAtlas.h"

#include "ImageCore.h"

//////////////////////////////////////////////////////////////////////////
// 

void FPaperAtlasTextureHelpers::CopyTextureData(const uint8* Source, uint8* Dest, uint32 SizeX, uint32 SizeY, uint32 BytesPerPixel, uint32 SourceStride, uint32 DestStride)
{
	const uint32 NumBytesPerRow = SizeX * BytesPerPixel;

	for (uint32 Y = 0; Y < SizeY; ++Y)
	{
		FMemory::Memcpy(
			Dest + (DestStride * Y),
			Source + (SourceStride * Y),
			NumBytesPerRow
			);
	}
}

bool FPaperAtlasTextureHelpers::ReadSpriteTexture(UTexture* SourceTexture, const FIntPoint& SourceXY, const FIntPoint& SourceSize, TArray<uint8>& TargetBuffer)
{
	{
		const int32 BytesPerPixel = 4;
		TargetBuffer.Empty();
		TargetBuffer.AddZeroed(SourceSize.X * SourceSize.Y * BytesPerPixel);
	}

	check(SourceTexture);
	FTextureSource& SourceData = SourceTexture->Source;
	FImage SourceMip0;
	const int32 MipIndex = 0;
	if (SourceData.GetMipImage(SourceMip0, MipIndex))
	{
		// It will only convert the texture if it is not in the right format
		SourceMip0.ChangeFormat(ERawImageFormat::BGRA8, EGammaSpace::sRGB);

		uint32 BytesPerPixel = SourceMip0.GetBytesPerPixel();
		const uint8* OffsetSource = SourceMip0.RawData.GetData() + (SourceXY.X + SourceXY.Y * SourceData.GetSizeX()) * BytesPerPixel;
		uint8* OffsetDest = TargetBuffer.GetData();
		CopyTextureData(OffsetSource, OffsetDest, SourceSize.X, SourceSize.Y, BytesPerPixel, SourceData.GetSizeX() * BytesPerPixel, SourceSize.X * BytesPerPixel);
	}
	else
	{
		UE_LOGF(LogPaper2DEditor, Error, "Couldn't retrive the source texture of the Sprite texture %ls while building the atlas for the sprite.", *SourceTexture->GetName());
		return false;
	}

	return true;
}

void FPaperAtlasTextureHelpers::PadSprite(const FPaperSpriteAtlasSlot& Slot, EPaperSpriteAtlasPadding PaddingType, int32 Padding, const FIntPoint& SpriteSize, int32 AtlasWidth, int32 AtlasBytesPerPixel, TArray<uint8>& TextureData)
{
#define UE_PIXEL(PIXEL__X, PIXEL__Y) TextureData[((Slot.Y + (PIXEL__Y)) * AtlasWidth + (Slot.X + (PIXEL__X))) * AtlasBytesPerPixel + (PixelIndex)]
	if (PaddingType == EPaperSpriteAtlasPadding::DilateBorder)
	{
		for (int32 X = 0; X < Padding; ++X)
		{
			for (int32 Y = 0; Y < SpriteSize.Y + Padding * 2; ++Y)
			{
				for (int32 PixelIndex = 0; PixelIndex < AtlasBytesPerPixel; ++PixelIndex)
				{
					const int32 ClampedY = Padding + FMath::Clamp(Y - Padding, 0, SpriteSize.Y - 1);
					UE_PIXEL(X, Y) = UE_PIXEL(Padding, ClampedY);
					UE_PIXEL(Padding + SpriteSize.X + X, Y) = UE_PIXEL(Padding + SpriteSize.X - 1, ClampedY);
				}
			}
		}
		for (int32 Y = 0; Y < Padding; ++Y)
		{
			for (int32 X = 0; X < SpriteSize.X + Padding * 2; ++X)
			{
				for (int32 PixelIndex = 0; PixelIndex < AtlasBytesPerPixel; ++PixelIndex)
				{
					const int32 ClampedX = Padding + FMath::Clamp(X - Padding, 0, SpriteSize.X - 1);
					UE_PIXEL(X, Y) = UE_PIXEL(ClampedX, Padding);
					UE_PIXEL(X, Padding + SpriteSize.Y + Y) = UE_PIXEL(ClampedX, Padding + SpriteSize.Y - 1);
				}
			}
		}
	}
	else if (PaddingType == EPaperSpriteAtlasPadding::PadWithZero)
	{
		for (int32 X = 0; X < Padding; ++X)
		{
			for (int32 Y = 0; Y < SpriteSize.Y + Padding * 2; ++Y)
			{
				for (int32 PixelIndex = 0; PixelIndex < AtlasBytesPerPixel; ++PixelIndex)
				{
					UE_PIXEL(X, Y) = 0;
					UE_PIXEL(Padding + SpriteSize.X + X, Y) = 0;
				}
			}
		}
		for (int32 Y = 0; Y < Padding; ++Y)
		{
			for (int32 X = 0; X < SpriteSize.X + Padding * 2; ++X)
			{
				for (int32 PixelIndex = 0; PixelIndex < AtlasBytesPerPixel; ++PixelIndex)
				{
					UE_PIXEL(X, Y) = 0;
					UE_PIXEL(X, Padding + SpriteSize.Y + Y) = 0;
				}
			}
		}
	}
#undef UE_PIXEL
}

void FPaperAtlasTextureHelpers::CopySpriteToAtlasTextureData(TArray<uint8>& TextureData, int32 AtlasWidth, int32 AtlasHeight, int32 AtlasBytesPerPixel, EPaperSpriteAtlasPadding PaddingType, int32 Padding, UPaperSprite* Sprite, const FPaperSpriteAtlasSlot& Slot)
{
	const FVector2D SpriteSizeFloat = Sprite->GetSourceSize();
	const FIntPoint SpriteSize(FMath::TruncToInt(Sprite->GetSourceSize().X), FMath::TruncToInt(Sprite->GetSourceSize().Y));
	const FIntPoint SpriteXY(FMath::TruncToInt(Sprite->GetSourceUV().X), FMath::TruncToInt(Sprite->GetSourceUV().Y));

	TArray<uint8> DummyBuffer;
	ReadSpriteTexture(Sprite->GetSourceTexture(), SpriteXY, SpriteSize, DummyBuffer);

	CopyTextureRegionToAtlasTextureData(TextureData, AtlasWidth, AtlasHeight, AtlasBytesPerPixel, PaddingType, Padding, DummyBuffer, SpriteSize, Slot);
}

void FPaperAtlasTextureHelpers::CopyTextureRegionToAtlasTextureData(TArray<uint8>& TextureData, int32 AtlasWidth, int32 AtlasHeight, int32 AtlasBytesPerPixel, EPaperSpriteAtlasPadding PaddingType, int32 Padding, TArray<uint8>& SourceData, const FIntPoint& SourceSize, const FPaperSpriteAtlasSlot& Slot)
{
	// Copy the texture into the texture buffer
	for (int32 Y = 0; Y < SourceSize.Y; ++Y)
	{
		for (int32 X = 0; X < SourceSize.X; ++X)
		{
			for (int32 PixelIndex = 0; PixelIndex < AtlasBytesPerPixel; ++PixelIndex)
			{
				TextureData[((Slot.Y + Y + Padding) * AtlasWidth + (Slot.X + X + Padding)) * AtlasBytesPerPixel + PixelIndex] = SourceData[(Y * SourceSize.X + X) * AtlasBytesPerPixel + PixelIndex];
			}
		}
	}

	// Padding
	PadSprite(Slot, PaddingType, Padding, SourceSize, AtlasWidth, AtlasBytesPerPixel, TextureData);
}

int32 FPaperAtlasTextureHelpers::ClampMips(int Width, int Height, int MipCount)
{
	int32 NumMips = 1;
	while (MipCount > 1 && (Width % 2) == 0 && (Height % 2) == 0)
	{
		Width /= 2;
		Height /= 2;
		MipCount--;
		NumMips++;
	}
	return NumMips;
}

void FPaperAtlasTextureHelpers::GenerateMipChainARGB(const TArray<FPaperSpriteAtlasSlot>& Slots, TArray<uint8>& AtlasTextureData, int32 MipCount, int32 Width, int32 Height)
{
	int32 SourceMipOffset = 0;
	int32 SourceMipWidth = Width;
	int32 SourceMipHeight = Height;

	// Mask bitmap stores all valid pixels in the image, i.e. of the rects that contain data
	TArray<int8> MaskBitmap;
	MaskBitmap.AddZeroed(SourceMipWidth * SourceMipHeight);
	for (const FPaperSpriteAtlasSlot& Slot : Slots)
	{
		for (int32 Y = 0; Y < Slot.Height; ++Y)
		{
			for (int32 X = 0; X < Slot.Width; ++X)
			{
				MaskBitmap[(Slot.Y + Y) * SourceMipWidth + (Slot.X + X)] = 1;
			}
		}
	}

	int32 BytesPerPixel = 4;
	check(BytesPerPixel == 4); // Only support 4 bytes per pixel

	// Mipmap offset to mip level being written into
	int32 TargetMipOffset = SourceMipWidth * SourceMipHeight * BytesPerPixel;
	int32 TargetMipWidth = SourceMipWidth / 2;
	int32 TargetMipHeight = SourceMipHeight / 2;

	for (int32 MipIndex = 1; MipIndex < MipCount; ++MipIndex)
	{
		int32 MipLevelSize = TargetMipHeight * TargetMipWidth * BytesPerPixel;

		for (int32 Y = 0; Y < TargetMipHeight; ++Y)
		{
			for (int32 X = 0; X < TargetMipWidth; ++X)
			{
				int32 TotalChannelValues[4] = { 0, 0, 0, 0 };
				check(BytesPerPixel == 4); // refer to above

				int32 ValidPixelCount = 0;
				int32 Mask0 = MaskBitmap[(Y * 2 + 0) * SourceMipWidth + (X * 2 + 0)];
				int32 Mask1 = MaskBitmap[(Y * 2 + 0) * SourceMipWidth + (X * 2 + 1)];
				int32 Mask2 = MaskBitmap[(Y * 2 + 1) * SourceMipWidth + (X * 2 + 0)];
				int32 Mask3 = MaskBitmap[(Y * 2 + 1) * SourceMipWidth + (X * 2 + 1)];
				ValidPixelCount = Mask0 + Mask1 + Mask2 + Mask3;

				for (int32 PixelIndex = 0; PixelIndex < BytesPerPixel; ++PixelIndex)
				{
					TotalChannelValues[PixelIndex] += AtlasTextureData[SourceMipOffset + ((Y * 2 + 0) * SourceMipWidth + (X * 2 + 0)) * BytesPerPixel + PixelIndex];
					TotalChannelValues[PixelIndex] += AtlasTextureData[SourceMipOffset + ((Y * 2 + 0) * SourceMipWidth + (X * 2 + 1)) * BytesPerPixel + PixelIndex];
					TotalChannelValues[PixelIndex] += AtlasTextureData[SourceMipOffset + ((Y * 2 + 1) * SourceMipWidth + (X * 2 + 0)) * BytesPerPixel + PixelIndex];
					TotalChannelValues[PixelIndex] += AtlasTextureData[SourceMipOffset + ((Y * 2 + 1) * SourceMipWidth + (X * 2 + 1)) * BytesPerPixel + PixelIndex];
				}

				for (int32 PixelIndex = 0; PixelIndex < BytesPerPixel; ++PixelIndex)
				{
					uint8 TargetPixelValue = (ValidPixelCount > 0) ? (TotalChannelValues[PixelIndex] / ValidPixelCount) : 0;
					AtlasTextureData[TargetMipOffset + (Y * TargetMipWidth + X) * BytesPerPixel + PixelIndex] = TargetPixelValue;
				}
			}
		}

		// Downsample mask in place, if any of the 4 mask pixels == 1, the target mask should be 1
		for (int32 Y = 0; Y < TargetMipHeight; ++Y)
		{
			for (int32 X = 0; X < TargetMipWidth; ++X)
			{
				uint8 TargetPixelValue = 0;
				TargetPixelValue |= MaskBitmap[(Y * 2 + 0) * SourceMipWidth + (X * 2 + 0)];
				TargetPixelValue |= MaskBitmap[(Y * 2 + 0) * SourceMipWidth + (X * 2 + 1)];
				TargetPixelValue |= MaskBitmap[(Y * 2 + 1) * SourceMipWidth + (X * 2 + 0)];
				TargetPixelValue |= MaskBitmap[(Y * 2 + 1) * SourceMipWidth + (X * 2 + 1)];
				MaskBitmap[Y * TargetMipWidth + X] = TargetPixelValue;
			}
		}

		// Update for next mip
		SourceMipOffset = TargetMipOffset;
		SourceMipWidth = TargetMipWidth;
		SourceMipHeight = TargetMipHeight;
		TargetMipOffset += MipLevelSize;
		TargetMipWidth /= 2;
		TargetMipHeight /= 2;
	}
}

