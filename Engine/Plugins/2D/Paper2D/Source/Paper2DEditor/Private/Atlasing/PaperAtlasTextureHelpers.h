// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class UPaperSprite;
class UTexture;
enum class EPaperSpriteAtlasPadding : uint8;
struct FPaperSpriteAtlasSlot;

struct FPaperAtlasTextureHelpers
{
public:
	// Copy texture data from one 32bpp image to another
	static void CopyTextureData(const uint8* Source, uint8* Dest, uint32 SizeX, uint32 SizeY, uint32 BytesPerPixel, uint32 SourceStride, uint32 DestStride);

	// Read sprite data into the 4 bytes per pixel target array
	static bool ReadSpriteTexture(UTexture* SourceTexture, const FIntPoint& SourceXY, const FIntPoint& SourceSize, TArray<uint8>& TargetBuffer);

	// Fills the padding space with the correct values
	static void PadSprite(const FPaperSpriteAtlasSlot& Slot, EPaperSpriteAtlasPadding PaddingType, int32 Padding, const FIntPoint& SpriteSize, int32 AtlasWidth, int32 AtlasBytesPerPixel, TArray<uint8>& TextureData);

	// Copies the sprite texture data at the position described by slot
	static void CopySpriteToAtlasTextureData(TArray<uint8>& TextureData, int32 AtlasWidth, int32 AtlasHeight, int32 AtlasBytesPerPixel, EPaperSpriteAtlasPadding PaddingType, int32 Padding, UPaperSprite* Sprite, const FPaperSpriteAtlasSlot& Slot);

	// Copies the texture data at the position described by slot
	static void CopyTextureRegionToAtlasTextureData(TArray<uint8>& TextureData, int32 AtlasWidth, int32 AtlasHeight, int32 AtlasBytesPerPixel, EPaperSpriteAtlasPadding PaddingType, int32 Padding, TArray<uint8>& SourceData, const FIntPoint& SourceSize, const FPaperSpriteAtlasSlot& Slot);

	static int32 ClampMips(int Width, int Height, int MipCount);

	static void GenerateMipChainARGB(const TArray<FPaperSpriteAtlasSlot>& Slots, TArray<uint8>& AtlasTextureData, int32 MipCount, int32 Width, int32 Height);

private:
	FPaperAtlasTextureHelpers() {}
};
