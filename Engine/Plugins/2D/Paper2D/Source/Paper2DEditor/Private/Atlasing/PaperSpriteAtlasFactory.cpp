// Copyright Epic Games, Inc. All Rights Reserved.

#include "Atlasing/PaperSpriteAtlasFactory.h"
#include "PaperSpriteAtlas.h"
#include "PaperRuntimeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteAtlasFactory)

//////////////////////////////////////////////////////////////////////////
// UPaperSpriteAtlasFactory

UPaperSpriteAtlasFactory::UPaperSpriteAtlasFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPaperSpriteAtlas::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UPaperSpriteAtlasFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UObject>(InParent, SupportedClass, InName, Flags | RF_Transactional);
}

bool UPaperSpriteAtlasFactory::CanCreateNew() const
{
	return GetDefault<UPaperRuntimeSettings>()->bEnableSpriteAtlasGroups;
}

