// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserExtensions/ContentBrowserExtensions.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackageName.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "PaperSprite.h"
#include "PaperTileSet.h"
#include "PaperSpriteFactory.h"
#include "PaperTileSetFactory.h"
#include "PaperImporterSettings.h"
#include "PaperStyle.h"

#include "ExtractSprites/SPaperExtractSpritesDialog.h"

#define LOCTEXT_NAMESPACE "Paper2D"

DECLARE_LOG_CATEGORY_EXTERN(LogPaperCBExtensions, Log, All);
DEFINE_LOG_CATEGORY(LogPaperCBExtensions);

//////////////////////////////////////////////////////////////////////////

static FContentBrowserMenuExtender_SelectedAssets ContentBrowserExtenderDelegate;
static FDelegateHandle ContentBrowserExtenderDelegateHandle;

//////////////////////////////////////////////////////////////////////////
// FContentBrowserSelectedAssetExtensionBase

struct FContentBrowserSelectedAssetExtensionBase
{
public:
	TArray<struct FAssetData> SelectedAssets;

public:
	virtual void Execute() {}
	virtual ~FContentBrowserSelectedAssetExtensionBase() {}
};

//////////////////////////////////////////////////////////////////////////
// FCreateSpriteFromTextureExtension

#include "IAssetTools.h"
#include "AssetToolsModule.h"


struct FCreateSpriteFromTextureExtension : public FContentBrowserSelectedAssetExtensionBase
{
	bool bExtractSprites;

	FCreateSpriteFromTextureExtension()
		: bExtractSprites(false)
	{
	}

	void CreateSpritesFromTextures(TArray<UTexture2D*>& Textures)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		TArray<UObject*> ObjectsToSync;

		for (auto TextureIt = Textures.CreateConstIterator(); TextureIt; ++TextureIt)
		{
			UTexture2D* Texture = *TextureIt;

			// Create the factory used to generate the sprite
			UPaperSpriteFactory* SpriteFactory = NewObject<UPaperSpriteFactory>();
			SpriteFactory->InitialTexture = Texture;

			// Create the sprite
			FString Name;
			FString PackageName;

			if (!bExtractSprites)
			{
				// Get a unique name for the sprite
				const FString DefaultSuffix = TEXT("_Sprite");
				AssetToolsModule.Get().CreateUniqueAssetName(Texture->GetOutermost()->GetName(), DefaultSuffix, /*out*/ PackageName, /*out*/ Name);
				const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

				if (UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, PackagePath, UPaperSprite::StaticClass(), SpriteFactory))
				{
					ObjectsToSync.Add(NewAsset);
				}
			}
			else
			{
				SPaperExtractSpritesDialog::ShowWindow(Texture);
			}
		}

		if (ObjectsToSync.Num() > 0)
		{
			ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

	virtual void Execute() override
	{
		// Create sprites for any selected textures
		TArray<UTexture2D*> Textures;
		for (auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FAssetData& AssetData = *AssetIt;
			if (UTexture2D* Texture = Cast<UTexture2D>(AssetData.GetAsset()))
			{
				Textures.Add(Texture);
			}
		}

		CreateSpritesFromTextures(Textures);
	}
};

struct FConfigureTexturesForSpriteUsageExtension : public FContentBrowserSelectedAssetExtensionBase
{
	virtual void Execute() override
	{
		// Change the compression settings and trigger a recompress
		for (auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FAssetData& AssetData = *AssetIt;
			if (UTexture2D* Texture = Cast<UTexture2D>(AssetData.GetAsset()))
			{
				GetDefault<UPaperImporterSettings>()->ApplyTextureSettings(Texture);
			}
		}
	}
};

struct FCreateTileSetFromTextureExtension : public FContentBrowserSelectedAssetExtensionBase
{
	void CreateTileSetsFromTextures(TArray<UTexture2D*>& Textures)
	{
		const FString DefaultSuffix = TEXT("_TileSet");

		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		TArray<UObject*> ObjectsToSync;

		for (auto TextureIt = Textures.CreateConstIterator(); TextureIt; ++TextureIt)
		{
			UTexture2D* Texture = *TextureIt;

			// Create the factory used to generate the tile set
			UPaperTileSetFactory* TileSetFactory = NewObject<UPaperTileSetFactory>();
			TileSetFactory->InitialTexture = Texture;

			// Get a unique name for the tile set
			FString Name;
			FString PackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(Texture->GetOutermost()->GetName(), DefaultSuffix, /*out*/ PackageName, /*out*/ Name);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

			if (UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, PackagePath, UPaperTileSet::StaticClass(), TileSetFactory))
			{
				ObjectsToSync.Add(NewAsset);
			}
		}

		if (ObjectsToSync.Num() > 0)
		{
			ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

	virtual void Execute() override
	{
		// Create sprites for any selected textures
		TArray<UTexture2D*> Textures;
		for (auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FAssetData& AssetData = *AssetIt;
			if (UTexture2D* Texture = Cast<UTexture2D>(AssetData.GetAsset()))
			{
				Textures.Add(Texture);
			}
		}

		CreateTileSetsFromTextures(Textures);
	}
};

enum class ECanCreatePaper2DFlags : uint8
{
	None = 0,
	PaperSprite = 1,
	PaperTileSet = 1 << 1,
};

ENUM_CLASS_FLAGS(ECanCreatePaper2DFlags);

//////////////////////////////////////////////////////////////////////////
// FPaperContentBrowserExtensions_Impl

class FPaperContentBrowserExtensions_Impl
{
public:
	static void ExecuteSelectedContentFunctor(TSharedPtr<FContentBrowserSelectedAssetExtensionBase> SelectedAssetFunctor)
	{
		SelectedAssetFunctor->Execute();
	}

	static void CreateSpriteActionsSubMenu(FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets, ECanCreatePaper2DFlags CanCreateFlags)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("SpriteActionsSubMenuLabel", "Sprite Actions"),
			LOCTEXT("SpriteActionsSubMenuToolTip", "Sprite-related actions for this texture."),
			FNewMenuDelegate::CreateStatic(&FPaperContentBrowserExtensions_Impl::PopulateSpriteActionsMenu, SelectedAssets, CanCreateFlags),
			false,
			FSlateIcon(FPaperStyle::Get()->GetStyleSetName(), "ClassIcon.PaperSprite")
		);
	}

	static void PopulateSpriteActionsMenu(FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets, ECanCreatePaper2DFlags CanCreateFlags)
	{
		const FName PaperStyleSetName = FPaperStyle::Get()->GetStyleSetName();

		if (EnumHasAnyFlags(CanCreateFlags, ECanCreatePaper2DFlags::PaperSprite))
		{
			// Create sprites
			TSharedPtr<FCreateSpriteFromTextureExtension> SpriteCreatorFunctor = MakeShareable(new FCreateSpriteFromTextureExtension());
			SpriteCreatorFunctor->SelectedAssets = SelectedAssets;

			FUIAction Action_CreateSpritesFromTextures(
				FExecuteAction::CreateStatic(&FPaperContentBrowserExtensions_Impl::ExecuteSelectedContentFunctor, StaticCastSharedPtr<FContentBrowserSelectedAssetExtensionBase>(SpriteCreatorFunctor)));


			MenuBuilder.AddMenuEntry(
				LOCTEXT("CB_Extension_Texture_CreateSprite", "Create Sprite"),
				LOCTEXT("CB_Extension_Texture_CreateSprite_Tooltip", "Create sprites from selected textures"),
				FSlateIcon(PaperStyleSetName, "AssetActions.CreateSprite"),
				Action_CreateSpritesFromTextures,
				NAME_None,
				EUserInterfaceActionType::Button);

			// Extract Sprites
			TSharedPtr<FCreateSpriteFromTextureExtension> SpriteExtractorFunctor = MakeShareable(new FCreateSpriteFromTextureExtension());
			SpriteExtractorFunctor->SelectedAssets = SelectedAssets;
			SpriteExtractorFunctor->bExtractSprites = true;

			FUIAction Action_ExtractSpritesFromTextures(
				FExecuteAction::CreateStatic(&FPaperContentBrowserExtensions_Impl::ExecuteSelectedContentFunctor, StaticCastSharedPtr<FContentBrowserSelectedAssetExtensionBase>(SpriteExtractorFunctor)));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CB_Extension_Texture_ExtractSprites", "Extract Sprites"),
				LOCTEXT("CB_Extension_Texture_ExtractSprites_Tooltip", "Extract sprites from selected textures"),
				FSlateIcon(PaperStyleSetName, "AssetActions.ExtractSprites"),
				Action_ExtractSpritesFromTextures,
				NAME_None,
				EUserInterfaceActionType::Button);
		}

		// Configure the selected textures according to the project settings (same as if it got imported from a sprite sheet)
		TSharedPtr<FConfigureTexturesForSpriteUsageExtension> TextureConfigFunctor = MakeShareable(new FConfigureTexturesForSpriteUsageExtension());
		TextureConfigFunctor->SelectedAssets = SelectedAssets;

		FUIAction Action_ConfigureTexturesForSprites(
			FExecuteAction::CreateStatic(&FPaperContentBrowserExtensions_Impl::ExecuteSelectedContentFunctor, StaticCastSharedPtr<FContentBrowserSelectedAssetExtensionBase>(TextureConfigFunctor)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CB_Extension_Texture_ConfigureTextureForSprites", "Apply Paper2D Texture Settings"),
			LOCTEXT("CB_Extension_Texture_ConfigureTextureForSprites_Tooltip", "Sets compression settings and sampling modes to the defaults specified in the 'Paper2D - Import' project settings"),
			FSlateIcon(PaperStyleSetName, "AssetActions.ConfigureForRetroSprites"),
			Action_ConfigureTexturesForSprites,
			NAME_None,
			EUserInterfaceActionType::Button);


		if (EnumHasAnyFlags(CanCreateFlags, ECanCreatePaper2DFlags::PaperTileSet))
		{
			// Create tile set
			TSharedPtr<FCreateTileSetFromTextureExtension> TileSetCreatorFunctor = MakeShareable(new FCreateTileSetFromTextureExtension());
			TileSetCreatorFunctor->SelectedAssets = SelectedAssets;

			FUIAction Action_CreateTileSetFromTextures(
				FExecuteAction::CreateStatic(&FPaperContentBrowserExtensions_Impl::ExecuteSelectedContentFunctor, StaticCastSharedPtr<FContentBrowserSelectedAssetExtensionBase>(TileSetCreatorFunctor)));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CB_Extension_Texture_CreateTileSet", "Create Tile Set"),
				LOCTEXT("CB_Extension_Texture_CreateTileSet_Tooltip", "Create tile set from selected texture"),
				FSlateIcon(PaperStyleSetName, "AssetActions.CreateTileSet"),
				Action_CreateTileSetFromTextures,
				NAME_None,
				EUserInterfaceActionType::Button);
		}
	}

	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
	{
		TSharedRef<FExtender> Extender(new FExtender());

		// Run thru the assets to determine if any meet our criteria
		bool bAnyTextures = false;
		for (auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FAssetData& Asset = *AssetIt;
			bAnyTextures = bAnyTextures || (Asset.AssetClassPath == UTexture2D::StaticClass()->GetClassPathName());
		}

		ECanCreatePaper2DFlags CanCreateFlags= ECanCreatePaper2DFlags::None;

		if (CanCreatePaperSpriteAsset())
		{
			EnumAddFlags(CanCreateFlags, ECanCreatePaper2DFlags::PaperSprite);
		}
		const bool bCanCreatePaperSprite = CanCreatePaperSpriteAsset();

		if (CanCreatePaperTileSetAsset())
		{
			EnumAddFlags(CanCreateFlags, ECanCreatePaper2DFlags::PaperTileSet);
		}

		if (bAnyTextures && CanCreateFlags != ECanCreatePaper2DFlags::None)
		{
			// Add the sprite actions sub-menu extender
			Extender->AddMenuExtension(
				"GetAssetActions",
				EExtensionHook::After,
				nullptr,
				FMenuExtensionDelegate::CreateStatic(&FPaperContentBrowserExtensions_Impl::CreateSpriteActionsSubMenu, SelectedAssets, CanCreateFlags));
		}

		return Extender;
	}

	static TArray<FContentBrowserMenuExtender_SelectedAssets>& GetExtenderDelegates()
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		return ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	}

	static bool CanCreatePaperSpriteAsset()
	{
		 return IAssetTools::Get().GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset)->PassesFilter(UPaperSprite::StaticClass()->GetClassPathName().ToString());
	}

	static bool CanCreatePaperTileSetAsset()
	{
		return IAssetTools::Get().GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset)->PassesFilter(UPaperTileSet::StaticClass()->GetClassPathName().ToString());
	}
};

//////////////////////////////////////////////////////////////////////////
// FPaperContentBrowserExtensions

void FPaperContentBrowserExtensions::InstallHooks()
{
	ContentBrowserExtenderDelegate = FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FPaperContentBrowserExtensions_Impl::OnExtendContentBrowserAssetSelectionMenu);

	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = FPaperContentBrowserExtensions_Impl::GetExtenderDelegates();
	CBMenuExtenderDelegates.Add(ContentBrowserExtenderDelegate);
	ContentBrowserExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

void FPaperContentBrowserExtensions::RemoveHooks()
{
	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = FPaperContentBrowserExtensions_Impl::GetExtenderDelegates();
	CBMenuExtenderDelegates.RemoveAll([](const FContentBrowserMenuExtender_SelectedAssets& Delegate){ return Delegate.GetHandle() == ContentBrowserExtenderDelegateHandle; });
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
