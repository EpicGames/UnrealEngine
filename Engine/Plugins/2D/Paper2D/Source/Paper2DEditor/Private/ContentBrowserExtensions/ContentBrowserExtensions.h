// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// Integrate Paper2D actions associated with existing engine types (e.g., Texture2D) into the content browser
class FPaperContentBrowserExtensions
{
public:
	static void InstallHooks();
	static void RemoveHooks();
};
