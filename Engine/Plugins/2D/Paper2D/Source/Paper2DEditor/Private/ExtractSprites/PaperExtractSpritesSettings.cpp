// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtractSprites/PaperExtractSpritesSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperExtractSpritesSettings)

UPaperExtractSpritesSettings::UPaperExtractSpritesSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	OutlineColor = FLinearColor::Yellow;
	ViewportTextureTint = FLinearColor::Gray;
	BackgroundColor = FLinearColor(0.1f, 0.1f, 0.1f);
}

UPaperExtractSpriteGridSettings::UPaperExtractSpriteGridSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

