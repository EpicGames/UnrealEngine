// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperExtractSpritesSettings.generated.h"

UENUM()
enum class ESpriteExtractMode : uint8
{
	/** Automatically extract sprites by detecting using alpha */
	Auto,

	/** Extract sprites in a grid defined in the properties below */
	Grid,	
};



/**
*
*/
UCLASS()
class UPaperExtractSpritesSettings : public UObject
{
	GENERATED_BODY()

public:
	// Sprite extract mode
	UPROPERTY(Category = Settings, EditAnywhere)
	ESpriteExtractMode SpriteExtractMode;

	// The color of the sprite boundary outlines
	UPROPERTY(Category = Settings, EditAnywhere, meta=(HideAlphaChannel))
	FLinearColor OutlineColor;

	// Apply a tint to the texture in the viewport to improve outline visibility in this editor
	UPROPERTY(Category = Settings, EditAnywhere)
	FLinearColor ViewportTextureTint;

	// The viewport background color
	UPROPERTY(Category = Settings, EditAnywhere, meta=(HideAlphaChannel))
	FLinearColor BackgroundColor;

	// The name of the sprite that will be created. {0} will get replaced by the sprite number.
	UPROPERTY(Category = Naming, EditAnywhere)
	FString NamingTemplate;

	// The number to start naming with
	UPROPERTY(Category = Naming, EditAnywhere)
	int32 NamingStartIndex;

	UPaperExtractSpritesSettings(const FObjectInitializer& ObjectInitializer);
};

UCLASS()
class UPaperExtractSpriteGridSettings : public UObject
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////
	// Grid mode

	// The width of each sprite in grid mode
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 1, ClampMin = 1))
	int32 CellWidth = 64;

	// The height of each sprite in grid mode
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 1, ClampMin = 1))
	int32 CellHeight = 64;

	// Number of cells extracted horizontally. Can be used to limit the number of sprites extracted. Set to 0 to extract all sprites
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 0, ClampMin = 0))
	int32 NumCellsX;

	// Number of cells extracted vertically. Can be used to limit the number of sprites extracted. Set to 0 to extract all sprites
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 0, ClampMin = 0))
	int32 NumCellsY;

	// Margin from the left of the texture to the first sprite
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 0, ClampMin = 0))
	int32 MarginX;

	// Margin from the top of the texture to the first sprite
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 0, ClampMin = 0))
	int32 MarginY;

	// Horizontal spacing between sprites
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 0, ClampMin = 0))
	int32 SpacingX;

	// Vertical spacing between sprites
	UPROPERTY(Category = Grid, EditAnywhere, meta = (UIMin = 0, ClampMin = 0))
	int32 SpacingY;

	UPaperExtractSpriteGridSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
};
