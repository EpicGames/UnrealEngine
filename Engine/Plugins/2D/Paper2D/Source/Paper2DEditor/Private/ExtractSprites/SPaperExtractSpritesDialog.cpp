// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtractSprites/SPaperExtractSpritesDialog.h"
#include "DetailsViewArgs.h"
#include "Misc/FeedbackContext.h"
#include "IDetailsView.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/PackageName.h"
#include "Framework/Application/SlateApplication.h"
#include "PaperEditorViewportClient.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Docking/TabManager.h"
#include "CanvasItem.h"
#include "ExtractSprites/PaperExtractSpritesSettings.h"

#include "PropertyEditorModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "CanvasTypes.h"
#include "PaperSprite.h"
#include "PaperSpriteFactory.h"
#include "TextureCompiler.h"
#include "TextureResource.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "PaperEditor"

//////////////////////////////////////////////////////////////////////////
// FTileSetEditorViewportClient 

class FPaperExtractSpritesViewportClient : public FPaperEditorViewportClient
{
public:
	FPaperExtractSpritesViewportClient(UTexture2D* Texture, const TArray<FPaperExtractedSprite>& InExtractedSprites, const UPaperExtractSpritesSettings* InSettings)
		: TextureBeingExtracted(Texture)
		, ExtractedSprites(InExtractedSprites)
		, Settings(InSettings)
	{
	}

	// FViewportClient interface
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual FLinearColor GetBackgroundColor() const override;
	// End of FViewportClient interface

private:
	// Tile set
	TWeakObjectPtr<UTexture2D> TextureBeingExtracted;
	const TArray<FPaperExtractedSprite>& ExtractedSprites;
	const UPaperExtractSpritesSettings* Settings;

	void DrawRectangle(FCanvas* Canvas, const FLinearColor& Color, const FIntRect& Rect);
};

void FPaperExtractSpritesViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	// Clear the viewport 
	Canvas->Clear(GetBackgroundColor());

	if (UTexture2D* Texture = TextureBeingExtracted.Get())
	{
		FTextureCompilingManager::Get().FinishCompilation({Texture});

		const bool bUseTranslucentBlend = Texture->HasAlphaChannel();

		// Fully stream in the texture before drawing it.
		Texture->SetForceMipLevelsToBeResident(30.0f);
		Texture->WaitForStreaming();

		FLinearColor TextureDrawColor = Settings->ViewportTextureTint;
		//FLinearColor RectOutlineColor = FLinearColor::Yellow;
		const FLinearColor RectOutlineColor = Settings->OutlineColor;

		const float XPos = -ZoomPos.X * ZoomAmount;
		const float YPos = -ZoomPos.Y * ZoomAmount;
		const float Width = Texture->GetSurfaceWidth() * ZoomAmount;
		const float Height = Texture->GetSurfaceHeight() * ZoomAmount;

		Canvas->DrawTile(XPos, YPos, Width, Height, 0.0f, 0.0f, 1.0f, 1.0f, TextureDrawColor, Texture->GetResource(), bUseTranslucentBlend);

		for (FPaperExtractedSprite Sprite : ExtractedSprites)
		{
			DrawRectangle(Canvas, RectOutlineColor, Sprite.Rect);
		}
	}
}

FLinearColor FPaperExtractSpritesViewportClient::GetBackgroundColor() const
{
	if (Settings != nullptr)
	{
		return Settings->BackgroundColor;
	}
	else
	{
		return FEditorViewportClient::GetBackgroundColor();
	}
}


void FPaperExtractSpritesViewportClient::DrawRectangle(FCanvas* Canvas, const FLinearColor& Color, const FIntRect& Rect)
{
	FVector2D TopLeft(-ZoomPos.X * ZoomAmount + Rect.Min.X * ZoomAmount, -ZoomPos.Y * ZoomAmount + Rect.Min.Y * ZoomAmount);
	FVector2D BottomRight(-ZoomPos.X * ZoomAmount + Rect.Max.X * ZoomAmount, -ZoomPos.Y * ZoomAmount + Rect.Max.Y * ZoomAmount);
	FVector2D RectVertices[4];
	RectVertices[0] = FVector2D(TopLeft.X, TopLeft.Y);
	RectVertices[1] = FVector2D(BottomRight.X, TopLeft.Y);
	RectVertices[2] = FVector2D(BottomRight.X, BottomRight.Y);
	RectVertices[3] = FVector2D(TopLeft.X, BottomRight.Y);
	for (int32 RectVertexIndex = 0; RectVertexIndex < 4; ++RectVertexIndex)
	{
		const int32 NextVertexIndex = (RectVertexIndex + 1) % 4;
		FCanvasLineItem RectLine(RectVertices[RectVertexIndex], RectVertices[NextVertexIndex]);
		RectLine.SetColor(Color);
		Canvas->DrawItem(RectLine);
	}
}

//////////////////////////////////////////////////////////////////////////
// SPaperExtractSpritesViewport

SPaperExtractSpritesViewport::~SPaperExtractSpritesViewport()
{
	TypedViewportClient = nullptr;
}

void SPaperExtractSpritesViewport::Construct(const FArguments& InArgs, UTexture2D* InTexture, const TArray<FPaperExtractedSprite>& ExtractedSprites, const UPaperExtractSpritesSettings* Settings, SPaperExtractSpritesDialog* InDialog)
{
	TexturePtr = InTexture;
	Dialog = InDialog;

	TypedViewportClient = MakeShareable(new FPaperExtractSpritesViewportClient(InTexture, ExtractedSprites, Settings));

	SPaperEditorViewport::Construct(
		SPaperEditorViewport::FArguments(),
		TypedViewportClient.ToSharedRef());

	// Make sure we get input instead of the viewport stealing it
	ViewportWidget->SetVisibility(EVisibility::HitTestInvisible);

	Invalidate();
}

FText SPaperExtractSpritesViewport::GetTitleText() const
{
	return FText::FromString(TexturePtr->GetName());
}

//////////////////////////////////////////////////////////////////////////
// SPaperExtractSpritesDialog

void SPaperExtractSpritesDialog::Construct(const FArguments& InArgs, UTexture2D* InSourceTexture)
{
	SourceTexture = InSourceTexture;
	bDirtyExtractedSprites = false;

	ExtractSpriteSettings = NewObject<UPaperExtractSpritesSettings>();
	ExtractSpriteSettings->AddToRoot();
	ExtractSpriteSettings->NamingTemplate = "Sprite_{0}";

	ExtractSpriteGridSettings = NewObject<UPaperExtractSpriteGridSettings>();
	ExtractSpriteGridSettings->AddToRoot();
	ExtractSpriteGridSettings->CellWidth = InSourceTexture->GetImportedSize().X;
	ExtractSpriteGridSettings->CellHeight = InSourceTexture->GetImportedSize().Y;

	PreviewExtractedSprites();

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	MainPropertyView = EditModule.CreateDetailView(DetailsViewArgs);
	MainPropertyView->SetObject(ExtractSpriteSettings);
	MainPropertyView->OnFinishedChangingProperties().AddSP(this, &SPaperExtractSpritesDialog::OnFinishedChangingProperties);
	
	DetailsPropertyView = EditModule.CreateDetailView(DetailsViewArgs);
	DetailsPropertyView->OnFinishedChangingProperties().AddSP(this, &SPaperExtractSpritesDialog::OnFinishedChangingProperties);

	SetDetailsViewForActiveMode();

	TSharedRef<SPaperExtractSpritesViewport> Viewport = SNew(SPaperExtractSpritesViewport, SourceTexture, ExtractedSprites, ExtractSpriteSettings, this);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(0.0f, 3.0f, 1.0f, 0.0f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				Viewport
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(2.0f)
				.AutoHeight()
				[
					MainPropertyView.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.Padding(2.0f)
				.FillHeight(1.0f)
				[
					DetailsPropertyView.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(16))
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("PaperExtractSpritesExtractButton", "Extract..."))
						.OnClicked(this, &SPaperExtractSpritesDialog::ExtractClicked)
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.Text(LOCTEXT("PaperExtractSpritesCancelButton", "Cancel"))
                        .TextStyle(FAppStyle::Get(), "DialogButtonText")
						.HAlign(HAlign_Center)
                        .OnClicked(this, &SPaperExtractSpritesDialog::CancelClicked)
					]
				]
			]
		]
	];
}

SPaperExtractSpritesDialog::~SPaperExtractSpritesDialog()
{
	if (ExtractSpriteSettings && ExtractSpriteSettings->IsValidLowLevel())
	{
		ExtractSpriteSettings->RemoveFromRoot();
	}

	if (ExtractSpriteGridSettings && ExtractSpriteGridSettings->IsValidLowLevel())
	{
		ExtractSpriteGridSettings->RemoveFromRoot();
	}
}

bool SPaperExtractSpritesDialog::ShowWindow(UTexture2D* SourceTexture)
{
	const FText TitleText = NSLOCTEXT("Paper2D", "Paper2D_ExtractSprites", "Extract sprites");
	// Create the window to pick the class
	TSharedRef<SWindow> ExtractSpritesWindow = SNew(SWindow)
		.Title(TitleText)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(1000.f, 700.f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);
	 
	TSharedRef<SPaperExtractSpritesDialog> PaperExtractSpritesDialog = SNew(SPaperExtractSpritesDialog, SourceTexture);

	ExtractSpritesWindow->SetContent(PaperExtractSpritesDialog);
	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	if (RootWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(ExtractSpritesWindow, RootWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(ExtractSpritesWindow);
	}

	return false;
}

// Extract the sprites based on SpriteExtractSettings->Mode
void SPaperExtractSpritesDialog::PreviewExtractedSprites()
{
	FString NamingTemplate = ExtractSpriteSettings->NamingTemplate;
	if (NamingTemplate.Find(TEXT("{0}")) == INDEX_NONE)
	{
		NamingTemplate.Append(TEXT("_{0}"));
	}
	int32 ExtractedRectIndex = ExtractSpriteSettings->NamingStartIndex;

	const FString DefaultSuffix = TEXT("Sprite");
	if (ExtractSpriteSettings->SpriteExtractMode == ESpriteExtractMode::Auto)
	{
		ExtractedSprites.Empty();

		// First extract the rects from the texture
		TArray<FIntRect> ExtractedRects;
		UPaperSprite::ExtractRectsFromTexture(SourceTexture, /*out*/ ExtractedRects);

		// Sort the rectangles by approximate row
		struct FRectangleSortHelper
		{
			FRectangleSortHelper(TArray<FIntRect>& InOutSprites)
			{
				// Sort by Y, then by X (top left corner), descending order (so we can use it as a stack from the top row down)
				TArray<FIntRect> SpritesLeft = InOutSprites;
				SpritesLeft.Sort([](const FIntRect& A, const FIntRect& B) { return (A.Min.Y == B.Min.Y) ? (A.Min.X > B.Min.X) : (A.Min.Y > B.Min.Y); });
				InOutSprites.Reset();

				// Start pulling sprites out, the first one in each row will dominate remaining ones and cause them to get labeled
				TArray<FIntRect> DominatedSprites;
				DominatedSprites.Empty(SpritesLeft.Num());
				while (SpritesLeft.Num())
				{
					FIntRect DominatingSprite = SpritesLeft.Pop(EAllowShrinking::No);
					DominatedSprites.Add(DominatingSprite);

					// Find the sprites that are dominated (intersect the infinite horizontal band described by the dominating sprite)
					for (int32 Index = 0; Index < SpritesLeft.Num();)
					{
						const FIntRect& CurElement = SpritesLeft[Index];
						if ((CurElement.Min.Y <= DominatingSprite.Max.Y) && (CurElement.Max.Y >= DominatingSprite.Min.Y))
						{
							DominatedSprites.Add(CurElement);
							SpritesLeft.RemoveAt(Index, /*Count=*/ 1, EAllowShrinking::No);
						}
						else
						{
							++Index;
						}
					}

					// Sort the sprites in the band by X and add them to the result
					DominatedSprites.Sort([](const FIntRect& A, const FIntRect& B) { return (A.Min.X < B.Min.X); });
					InOutSprites.Append(DominatedSprites);
					DominatedSprites.Reset();
				}
			}
		};
		FRectangleSortHelper RectSorter(ExtractedRects);

		for (FIntRect Rect : ExtractedRects)
		{
			FPaperExtractedSprite* Sprite = new(ExtractedSprites)FPaperExtractedSprite();
			Sprite->Rect = Rect;
			Sprite->Name = NamingTemplate;
			Sprite->Name.ReplaceInline(TEXT("{0}"), *FString::Printf(TEXT("%d"), ExtractedRectIndex));
			
			ExtractedRectIndex++;
		}
	}
	else
	{
		// Calculate rects
		ExtractedSprites.Empty();
		if (SourceTexture != nullptr)
		{
			const FIntPoint TextureSize(SourceTexture->GetImportedSize());
			const int32 TextureWidth = TextureSize.X;
			const int32 TextureHeight = TextureSize.Y;

			const int32 EffectiveCellWidth = FMath::Clamp(ExtractSpriteGridSettings->CellWidth, 1, TextureWidth);
			const int32 EffectiveCellHeight = FMath::Clamp(ExtractSpriteGridSettings->CellHeight, 1, TextureHeight);

			int NumExtractedCellsY = 0;
			for (int32 Y = ExtractSpriteGridSettings->MarginY; Y + EffectiveCellHeight <= TextureHeight; Y += EffectiveCellHeight + ExtractSpriteGridSettings->SpacingY)
			{
				int NumExtractedCellsX = 0;
				for (int32 X = ExtractSpriteGridSettings->MarginX; X + EffectiveCellWidth <= TextureWidth; X += EffectiveCellWidth + ExtractSpriteGridSettings->SpacingX)
				{
					FPaperExtractedSprite* Sprite = new(ExtractedSprites)FPaperExtractedSprite();
					Sprite->Rect = FIntRect(X, Y, X + EffectiveCellWidth, Y + EffectiveCellHeight);
					
					Sprite->Name = NamingTemplate;
					Sprite->Name.ReplaceInline(TEXT("{0}"), *FString::Printf(TEXT("%d"), ExtractedRectIndex));

					ExtractedRectIndex++;

					++NumExtractedCellsX;
					if (ExtractSpriteGridSettings->NumCellsX > 0 && NumExtractedCellsX >= ExtractSpriteGridSettings->NumCellsX)
					{
						break;
					}
				}

				++NumExtractedCellsY;
				if (ExtractSpriteGridSettings->NumCellsY > 0 && NumExtractedCellsY >= ExtractSpriteGridSettings->NumCellsY)
				{
					break;
				}
			}
		}
	}
}

void SPaperExtractSpritesDialog::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		const FName PropertyName(PropertyChangedEvent.Property->GetFName());

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperExtractSpritesSettings, NamingTemplate) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UPaperExtractSpritesSettings, NamingStartIndex))
		{
			bDirtyExtractedSprites = true;
		}
		else if (PropertyName != GET_MEMBER_NAME_CHECKED(UPaperExtractSpritesSettings, OutlineColor) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UPaperExtractSpritesSettings, BackgroundColor) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UPaperExtractSpritesSettings, ViewportTextureTint))
		{
			PreviewExtractedSprites();
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UPaperExtractSpritesSettings, SpriteExtractMode))
		{
			SetDetailsViewForActiveMode();
		}
	}
}

FReply SPaperExtractSpritesDialog::ExtractClicked()
{
	if (bDirtyExtractedSprites)
	{
		bDirtyExtractedSprites = false;
		PreviewExtractedSprites();
	}

	CreateExtractedSprites();

	CloseContainingWindow();
	return FReply::Handled();
}

FReply SPaperExtractSpritesDialog::CancelClicked()
{
	CloseContainingWindow();
	return FReply::Handled();
}

void SPaperExtractSpritesDialog::CloseContainingWindow()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

void SPaperExtractSpritesDialog::CreateExtractedSprites()
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	TArray<UObject*> ObjectsToSync;

	// Create the factory used to generate the sprite
	UPaperSpriteFactory* SpriteFactory = NewObject<UPaperSpriteFactory>();
	SpriteFactory->InitialTexture = SourceTexture;

	// Create the sprite
	FString Name;
	FString PackageName;

	FScopedSlowTask Feedback(ExtractedSprites.Num(), NSLOCTEXT("Paper2D", "Paper2D_ExtractSpritesFromTexture", "Extracting Sprites From Texture"));
	Feedback.MakeDialog(true);

	for (const FPaperExtractedSprite& ExtractedSprite : ExtractedSprites)
	{
		Feedback.EnterProgressFrame(1, NSLOCTEXT("Paper2D", "Paper2D_ExtractSpritesFromTexture", "Extracting Sprites From Texture"));

		SpriteFactory->bUseSourceRegion = true;
		const FIntRect& ExtractedRect = ExtractedSprite.Rect;
		SpriteFactory->InitialSourceUV = ExtractedRect.Min;
		SpriteFactory->InitialSourceDimension = FIntPoint(ExtractedRect.Width(), ExtractedRect.Height());

		// Get a unique name for the sprite
		// Extracted sprite name is a name, we insert a _ as we're appending this to the texture name
		// Opens up doors to renaming the sprites in the editor, and still ending up with TextureName_UserSpriteName
		FString Suffix = TEXT("_");
		Suffix.Append(ExtractedSprite.Name);

		AssetToolsModule.Get().CreateUniqueAssetName(SourceTexture->GetOutermost()->GetName(), Suffix, /*out*/ PackageName, /*out*/ Name);
		const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

		if (UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, PackagePath, UPaperSprite::StaticClass(), SpriteFactory))
		{
			ObjectsToSync.Add(NewAsset);
		}

		if (GWarn->ReceivedUserCancel())
		{
			break;
		}
	}

	if (ObjectsToSync.Num() > 0)
	{
		ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
	}
}

void SPaperExtractSpritesDialog::SetDetailsViewForActiveMode()
{
	if (ExtractSpriteSettings->SpriteExtractMode == ESpriteExtractMode::Grid)
	{
		DetailsPropertyView->SetObject(ExtractSpriteGridSettings);
	}
	else
	{
		DetailsPropertyView->SetObject(nullptr);
	}
}

#undef LOCTEXT_NAMESPACE

