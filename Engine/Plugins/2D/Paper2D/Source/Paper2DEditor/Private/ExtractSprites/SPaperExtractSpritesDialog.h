// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPaperEditorViewport.h"

class UTexture2D;
struct FPropertyChangedEvent;

class FPaperExtractSpritesViewportClient;
struct Rect;

struct FPaperExtractedSprite
{
	FString Name;
	FIntRect Rect;
};

//////////////////////////////////////////////////////////////////////////
// SPaperExtractSpritesDialog

class SPaperExtractSpritesDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPaperExtractSpritesDialog) {}
	SLATE_END_ARGS()

	// Constructs this widget with InArgs
	void Construct(const FArguments& InArgs, UTexture2D* Texture);

	virtual ~SPaperExtractSpritesDialog();

	// Show the dialog, returns true if successfully extracted sprites
	static bool ShowWindow(UTexture2D* SourceTexture);

private:
	// Handler for Extract button
	FReply ExtractClicked();
	
	// Handler for Cancel button
	FReply CancelClicked();
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);
	void CloseContainingWindow();

	// Calculate a list of extracted sprite rects
	void PreviewExtractedSprites();

	// Actually create extracted sprites
	void CreateExtractedSprites();

	// Sets the details panel appropriately for the currently selected panel
	void SetDetailsViewForActiveMode();

private:
	// Source texture to extract from
	UTexture2D* SourceTexture;
	bool bDirtyExtractedSprites;

	class UPaperExtractSpritesSettings* ExtractSpriteSettings;
	class UPaperExtractSpriteGridSettings* ExtractSpriteGridSettings;

	TSharedPtr<class IDetailsView> MainPropertyView;
	TSharedPtr<class IDetailsView> DetailsPropertyView;
	TArray<FPaperExtractedSprite> ExtractedSprites;
};


//////////////////////////////////////////////////////////////////////////
// SPaperExtractSpritesViewport

class SPaperExtractSpritesViewport : public SPaperEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SPaperExtractSpritesViewport) {}
	SLATE_END_ARGS()

	~SPaperExtractSpritesViewport();
	void Construct(const FArguments& InArgs, UTexture2D* Texture, const TArray<FPaperExtractedSprite>& ExtractedSprites, const class UPaperExtractSpritesSettings* Settings, class SPaperExtractSpritesDialog* InDialog);

protected:
	// SPaperEditorViewport interface
	virtual FText GetTitleText() const override;
	// End of SPaperEditorViewport interface

private:
	TWeakObjectPtr<class UTexture2D> TexturePtr;
	TSharedPtr<class FPaperExtractSpritesViewportClient> TypedViewportClient;
	class SPaperExtractSpritesDialog* Dialog;
};
