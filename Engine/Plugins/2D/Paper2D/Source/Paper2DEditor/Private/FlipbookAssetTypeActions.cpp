// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookAssetTypeActions.h"

#include "FlipbookEditor/FlipbookEditor.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

//////////////////////////////////////////////////////////////////////////
// FFlipbookAssetTypeActions

FFlipbookAssetTypeActions::FFlipbookAssetTypeActions(EAssetTypeCategories::Type InAssetCategory)
	: MyAssetCategory(InAssetCategory)
{
}

FText FFlipbookAssetTypeActions::GetName() const
{
	return LOCTEXT("FFlipbookAssetTypeActionsName", "Paper Flipbook");
}

FColor FFlipbookAssetTypeActions::GetTypeColor() const
{
	return FColor(129, 196, 115);
}

UClass* FFlipbookAssetTypeActions::GetSupportedClass() const
{
	return UPaperFlipbook::StaticClass();
}

void FFlipbookAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	const EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UPaperFlipbook* Flipbook = Cast<UPaperFlipbook>(*ObjIt))
		{
			TSharedRef<FFlipbookEditor> NewFlipbookEditor(new FFlipbookEditor());
			NewFlipbookEditor->InitFlipbookEditor(Mode, EditWithinLevelEditor, Flipbook);
		}
	}
}

uint32 FFlipbookAssetTypeActions::GetCategories()
{
	return MyAssetCategory;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
