// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookEditor/FlipbookComponentDetailsCustomization.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "FlipbookEditor"

//////////////////////////////////////////////////////////////////////////
// FFlipbookComponentDetailsCustomization

TSharedRef<IDetailCustomization> FFlipbookComponentDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FFlipbookComponentDetailsCustomization);
}

void FFlipbookComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Create a category so this is displayed early in the properties
	DetailBuilder.EditCategory("Sprite", FText::GetEmpty(), ECategoryPriority::Important);
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
