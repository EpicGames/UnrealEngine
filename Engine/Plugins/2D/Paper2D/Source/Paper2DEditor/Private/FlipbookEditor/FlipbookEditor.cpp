// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookEditor/FlipbookEditor.h"
#include "Framework/Commands/UICommandList.h"
#include "Modules/ModuleManager.h"
#include "SSingleObjectDetailsPanel.h"


#include "SEditorViewport.h"
#include "ScopedTransaction.h"
#include "PaperFlipbookComponent.h"
#include "FlipbookEditor/FlipbookEditorViewportClient.h"
#include "FlipbookEditor/FlipbookEditorCommands.h"
#include "Paper2DEditorModule.h"
#include "FlipbookEditor/SFlipbookEditorViewportToolbar.h"
#include "SScrubControlPanel.h"
#include "FlipbookEditor/SFlipbookTimeline.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Commands/GenericCommands.h"

#define LOCTEXT_NAMESPACE "FlipbookEditor"

//////////////////////////////////////////////////////////////////////////

const FName FlipbookEditorAppName = FName(TEXT("FlipbookEditorApp"));

//////////////////////////////////////////////////////////////////////////

struct FFlipbookEditorTabs
{
	// Tab identifiers
	static const FName DetailsID;
	static const FName ViewportID;
};

//////////////////////////////////////////////////////////////////////////

const FName FFlipbookEditorTabs::DetailsID(TEXT("Details"));
const FName FFlipbookEditorTabs::ViewportID(TEXT("Viewport"));

//////////////////////////////////////////////////////////////////////////
// SFlipbookEditorViewport

class SFlipbookEditorViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SFlipbookEditorViewport)
		: _FlipbookBeingEdited((UPaperFlipbook*)nullptr)
	{}

		SLATE_ATTRIBUTE( UPaperFlipbook*, FlipbookBeingEdited )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SEditorViewport interface
	virtual void BindCommands() override;
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual EVisibility GetTransformToolbarVisibility() const override;
	virtual void OnFocusViewportToSelection() override;
	// End of SEditorViewport interface

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	UPaperFlipbookComponent* GetPreviewComponent() const
	{
		return EditorViewportClient->GetPreviewComponent();
	}

private:
	TAttribute<UPaperFlipbook*> FlipbookBeingEdited;

	// Viewport client
	TSharedPtr<FFlipbookEditorViewportClient> EditorViewportClient;
};

void SFlipbookEditorViewport::Construct(const FArguments& InArgs)
{
	FlipbookBeingEdited = InArgs._FlipbookBeingEdited;

	SEditorViewport::Construct(SEditorViewport::FArguments());
}

void SFlipbookEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FFlipbookEditorCommands& Commands = FFlipbookEditorCommands::Get();

	TSharedRef<FFlipbookEditorViewportClient> EditorViewportClientRef = EditorViewportClient.ToSharedRef();

	CommandList->MapAction(
		Commands.SetShowGrid,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FEditorViewportClient::SetShowGrid ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FEditorViewportClient::IsSetShowGridChecked ) );

	CommandList->MapAction(
		Commands.SetShowBounds,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FEditorViewportClient::ToggleShowBounds ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FEditorViewportClient::IsSetShowBoundsChecked ) );

	CommandList->MapAction(
		Commands.SetShowCollision,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FEditorViewportClient::SetShowCollision ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FEditorViewportClient::IsSetShowCollisionChecked ) );
 
	CommandList->MapAction(
		Commands.SetShowPivot,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FFlipbookEditorViewportClient::ToggleShowPivot ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FFlipbookEditorViewportClient::IsShowPivotChecked ) );

	CommandList->MapAction(
		Commands.SetShowSockets,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FFlipbookEditorViewportClient::ToggleShowSockets ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FFlipbookEditorViewportClient::IsShowSocketsChecked ) );
}

TSharedRef<FEditorViewportClient> SFlipbookEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable(new FFlipbookEditorViewportClient(FlipbookBeingEdited));

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SFlipbookEditorViewport::BuildViewportToolbar()
{
	return SNew(SFlipbookEditorViewportToolbar, SharedThis(this));
}

EVisibility SFlipbookEditorViewport::GetTransformToolbarVisibility() const
{
	return EVisibility::Visible;
}

void SFlipbookEditorViewport::OnFocusViewportToSelection()
{
	EditorViewportClient->RequestFocusOnSelection(/*bInstant=*/ false);
}

TSharedRef<class SEditorViewport> SFlipbookEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SFlipbookEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SFlipbookEditorViewport::OnFloatingButtonClicked()
{
}

/////////////////////////////////////////////////////
// SFlipbookPropertiesTabBody

class SFlipbookPropertiesTabBody : public SSingleObjectDetailsPanel
{
public:
	SLATE_BEGIN_ARGS(SFlipbookPropertiesTabBody) {}
	SLATE_END_ARGS()

private:
	// Pointer back to owning sprite editor instance (the keeper of state)
	TWeakPtr<class FFlipbookEditor> FlipbookEditorPtr;
public:
	void Construct(const FArguments& InArgs, TSharedPtr<FFlipbookEditor> InFlipbookEditor)
	{
		FlipbookEditorPtr = InFlipbookEditor;

		SSingleObjectDetailsPanel::Construct(SSingleObjectDetailsPanel::FArguments().HostCommandList(InFlipbookEditor->GetToolkitCommands()).HostTabManager(InFlipbookEditor->GetTabManager()), /*bAutomaticallyObserveViaGetObjectToObserve=*/ true, /*bAllowSearch=*/ true);
	}

	// SSingleObjectDetailsPanel interface
	virtual UObject* GetObjectToObserve() const override
	{
		return FlipbookEditorPtr.Pin()->GetFlipbookBeingEdited();
	}

	virtual TSharedRef<SWidget> PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget) override
	{
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				PropertyEditorWidget
			];
	}
	// End of SSingleObjectDetailsPanel interface
};

//////////////////////////////////////////////////////////////////////////
// FFlipbookEditor

FFlipbookEditor::FFlipbookEditor()
	: FlipbookBeingEdited(nullptr)
{
}

TSharedRef<SDockTab> FFlipbookEditor::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	ViewInputMin = 0.0f;
	ViewInputMax = GetTotalSequenceLength();
	LastObservedSequenceLength = ViewInputMax;

	//@TODO: PAPER2D: Implement OnBeginSliderMovement/OnEndSliderMovement so that refreshing works, animation stops, etc...

	TSharedRef<SWidget> ScrubControl = SNew(SScrubControlPanel)
		.IsEnabled(true)
		.Value(this, &FFlipbookEditor::GetPlaybackPosition)
		.NumOfKeys(this, &FFlipbookEditor::GetTotalFrameCountPlusOne)
		.SequenceLength(this, &FFlipbookEditor::GetTotalSequenceLength)
		.OnValueChanged(this, &FFlipbookEditor::SetPlaybackPosition)
//		.OnBeginSliderMovement(this, &SAnimationScrubPanel::OnBeginSliderMovement)
//		.OnEndSliderMovement(this, &SAnimationScrubPanel::OnEndSliderMovement)
		.OnClickedForwardPlay(this, &FFlipbookEditor::OnClick_Forward)
		.OnClickedForwardStep(this, &FFlipbookEditor::OnClick_Forward_Step)
		.OnClickedForwardEnd(this, &FFlipbookEditor::OnClick_Forward_End)
		.OnClickedBackwardPlay(this, &FFlipbookEditor::OnClick_Backward)
		.OnClickedBackwardStep(this, &FFlipbookEditor::OnClick_Backward_Step)
		.OnClickedBackwardEnd(this, &FFlipbookEditor::OnClick_Backward_End)
		.OnClickedToggleLoop(this, &FFlipbookEditor::OnClick_ToggleLoop)
		.OnGetLooping(this, &FFlipbookEditor::IsLooping)
		.OnGetPlaybackMode(this, &FFlipbookEditor::GetPlaybackMode)
		.ViewInputMin(this, &FFlipbookEditor::GetViewRangeMin)
		.ViewInputMax(this, &FFlipbookEditor::GetViewRangeMax)
		.OnSetInputViewRange(this, &FFlipbookEditor::SetViewRange)
		.bAllowZoom(true)
		.IsRealtimeStreamingMode(false);

	return SNew(SDockTab)
		.Label(LOCTEXT("ViewportTab_Title", "Viewport"))
		[
			SNew(SVerticalBox)
			
			+SVerticalBox::Slot()
			[
				ViewportPtr.ToSharedRef()
			]

			+SVerticalBox::Slot()
			.Padding(0, 8, 0, 0)
			.AutoHeight()
			[
				SNew(SFlipbookTimeline, GetToolkitCommands())
				.FlipbookBeingEdited(this, &FFlipbookEditor::GetFlipbookBeingEdited)
				.OnSelectionChanged(this, &FFlipbookEditor::SetSelection)
				.PlayTime(this, &FFlipbookEditor::GetPlaybackPosition)
			]

			+SVerticalBox::Slot()
			.Padding(0, 8, 0, 0)
			.AutoHeight()
			[
				ScrubControl
			]
		];
}

TSharedRef<SDockTab> FFlipbookEditor::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<FFlipbookEditor> FlipbookEditorPtr = SharedThis(this);

	// Spawn the tab
	return SNew(SDockTab)
		.Label(LOCTEXT("DetailsTab_Title", "Details"))
		[
			SNew(SFlipbookPropertiesTabBody, FlipbookEditorPtr)
		];
}

void FFlipbookEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_FlipbookEditor", "Flipbook Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(FFlipbookEditorTabs::ViewportID, FOnSpawnTab::CreateSP(this, &FFlipbookEditor::SpawnTab_Viewport))
		.SetDisplayName( LOCTEXT("ViewportTab", "Viewport") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(FFlipbookEditorTabs::DetailsID, FOnSpawnTab::CreateSP(this, &FFlipbookEditor::SpawnTab_Details))
		.SetDisplayName( LOCTEXT("DetailsTabLabel", "Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FFlipbookEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FFlipbookEditorTabs::ViewportID);
	InTabManager->UnregisterTabSpawner(FFlipbookEditorTabs::DetailsID);
}

void FFlipbookEditor::InitFlipbookEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UPaperFlipbook* InitFlipbook)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseOtherEditors(InitFlipbook, this);
	FlipbookBeingEdited = InitFlipbook;
	CurrentSelectedKeyframe = INDEX_NONE;

	FFlipbookEditorCommands::Register();

	BindCommands();

	ViewportPtr = SNew(SFlipbookEditorViewport)
		.FlipbookBeingEdited(this, &FFlipbookEditor::GetFlipbookBeingEdited);
	
	// Default layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_FlipbookEditor_Layout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell(true)
					->AddTab(FFlipbookEditorTabs::ViewportID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->AddTab(FFlipbookEditorTabs::DetailsID, ETabState::OpenedTab)
				)
			)
		);

	// Initialize the asset editor and spawn nothing (dummy layout)
	InitAssetEditor(Mode, InitToolkitHost, FlipbookEditorAppName, StandaloneDefaultLayout, /*bCreateDefaultStandaloneMenu=*/ true, /*bCreateDefaultToolbar=*/ true, InitFlipbook);

	// Extend things
	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

UPaperFlipbookComponent* FFlipbookEditor::GetPreviewComponent() const
{
	UPaperFlipbookComponent* PreviewComponent = ViewportPtr->GetPreviewComponent();
	check(PreviewComponent);
	return PreviewComponent;
}

void FFlipbookEditor::BindCommands()
{
	const FFlipbookEditorCommands& Commands = FFlipbookEditorCommands::Get();

	const TSharedRef<FUICommandList>& UICommandList = GetToolkitCommands();

	UICommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FFlipbookEditor::DeleteSelection),
		FCanExecuteAction::CreateSP(this, &FFlipbookEditor::HasValidSelection));

	UICommandList->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FFlipbookEditor::DuplicateSelection),
		FCanExecuteAction::CreateSP(this, &FFlipbookEditor::HasValidSelection));

	UICommandList->MapAction(Commands.AddNewFrame,
		FExecuteAction::CreateSP(this, &FFlipbookEditor::AddNewKeyFrameAtEnd),
		FCanExecuteAction());
	UICommandList->MapAction(Commands.AddNewFrameBefore,
		FExecuteAction::CreateSP(this, &FFlipbookEditor::AddNewKeyFrameBefore),
		FCanExecuteAction());
	UICommandList->MapAction(Commands.AddNewFrameAfter,
		FExecuteAction::CreateSP(this, &FFlipbookEditor::AddNewKeyFrameAfter),
		FCanExecuteAction());

	UICommandList->MapAction(Commands.AddKeyFrame,
		FExecuteAction::CreateSP(this, &FFlipbookEditor::AddKeyFrameAtCurrentTime));
}

FName FFlipbookEditor::GetToolkitFName() const
{
	return FName("FlipbookEditor");
}

FText FFlipbookEditor::GetBaseToolkitName() const
{
	return LOCTEXT("FlipbookEditorAppLabel", "Flipbook Editor");
}

FText FFlipbookEditor::GetToolkitName() const
{
	return FText::FromString(FlipbookBeingEdited->GetName());
}

FText FFlipbookEditor::GetToolkitToolTipText() const
{
	return FAssetEditorToolkit::GetToolTipTextForObject(FlipbookBeingEdited);
}

FString FFlipbookEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("FlipbookEditor");
}

FString FFlipbookEditor::GetDocumentationLink() const
{
	return TEXT("AnimatingObjects/Paper2D/Flipbooks");
}

FLinearColor FFlipbookEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

void FFlipbookEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(FlipbookBeingEdited);
}

void FFlipbookEditor::ExtendMenu()
{
}

void FFlipbookEditor::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("Command");
			{
				ToolbarBuilder.AddToolBarButton(FFlipbookEditorCommands::Get().AddKeyFrame);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		/*ViewportPtr->GetCommandList()*/ GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic( &Local::FillToolbar )
		);

	AddToolbarExtender(ToolbarExtender);

 	IPaper2DEditorModule* Paper2DEditorModule = &FModuleManager::LoadModuleChecked<IPaper2DEditorModule>("Paper2DEditor");
 	AddToolbarExtender(Paper2DEditorModule->GetFlipbookEditorToolBarExtensibilityManager()->GetAllExtenders());
}

void FFlipbookEditor::DeleteSelection()
{
	if (FlipbookBeingEdited->IsValidKeyFrameIndex(CurrentSelectedKeyframe))
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteKeyframeTransaction", "Delete Keyframe"));
		FlipbookBeingEdited->Modify();

		FScopedFlipbookMutator EditLock(FlipbookBeingEdited);
		EditLock.KeyFrames.RemoveAt(CurrentSelectedKeyframe);

		CurrentSelectedKeyframe = INDEX_NONE;
	}
}

void FFlipbookEditor::DuplicateSelection()
{
	if (FlipbookBeingEdited->IsValidKeyFrameIndex(CurrentSelectedKeyframe))
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicateKeyframeTransaction", "Duplicate Keyframe"));
		FlipbookBeingEdited->Modify();

		FScopedFlipbookMutator EditLock(FlipbookBeingEdited);

		FPaperFlipbookKeyFrame NewFrame = EditLock.KeyFrames[CurrentSelectedKeyframe];
		EditLock.KeyFrames.Insert(NewFrame, CurrentSelectedKeyframe);

		CurrentSelectedKeyframe = INDEX_NONE;
	}
}

void FFlipbookEditor::AddKeyFrameAtCurrentTime()
{
	const FScopedTransaction Transaction(LOCTEXT("InsertKeyFrameTransaction", "Insert Key Frame"));
	FlipbookBeingEdited->Modify();

	const float CurrentTime = GetPlaybackPosition();
	const int32 KeyFrameIndex = FlipbookBeingEdited->GetKeyFrameIndexAtTime(CurrentTime);
	const int32 ClampedIndex = FMath::Clamp<int32>(KeyFrameIndex, 0, FlipbookBeingEdited->GetNumFrames());

	FScopedFlipbookMutator EditLock(FlipbookBeingEdited);
	FPaperFlipbookKeyFrame NewFrame;
	EditLock.KeyFrames.Insert(NewFrame, ClampedIndex);
}

void FFlipbookEditor::AddNewKeyFrameAtEnd()
{
	const FScopedTransaction Transaction(LOCTEXT("AddKeyFrameTransaction", "Add Key Frame"));
	FlipbookBeingEdited->Modify();

	FScopedFlipbookMutator EditLock(FlipbookBeingEdited);

	FPaperFlipbookKeyFrame& NewFrame = *new (EditLock.KeyFrames) FPaperFlipbookKeyFrame();
}

void FFlipbookEditor::AddNewKeyFrameBefore()
{
	if (FlipbookBeingEdited->IsValidKeyFrameIndex(CurrentSelectedKeyframe))
	{
		const FScopedTransaction Transaction(LOCTEXT("InsertKeyFrameBeforeTransaction", "Insert Key Frame Before"));
		FlipbookBeingEdited->Modify();

		FScopedFlipbookMutator EditLock(FlipbookBeingEdited);

		FPaperFlipbookKeyFrame NewFrame;
		EditLock.KeyFrames.Insert(NewFrame, CurrentSelectedKeyframe);

		CurrentSelectedKeyframe = INDEX_NONE;
	}
}

void FFlipbookEditor::AddNewKeyFrameAfter()
{
	if (FlipbookBeingEdited->IsValidKeyFrameIndex(CurrentSelectedKeyframe))
	{
		const FScopedTransaction Transaction(LOCTEXT("InsertKeyFrameAfterTransaction", "Insert Key Frame After"));
		FlipbookBeingEdited->Modify();

		FScopedFlipbookMutator EditLock(FlipbookBeingEdited);

		FPaperFlipbookKeyFrame NewFrame;
		EditLock.KeyFrames.Insert(NewFrame, CurrentSelectedKeyframe + 1);

		CurrentSelectedKeyframe = INDEX_NONE;
	}
}


void FFlipbookEditor::SetSelection(int32 NewSelection)
{
	CurrentSelectedKeyframe = NewSelection;
}

bool FFlipbookEditor::HasValidSelection() const
{
	return FlipbookBeingEdited->IsValidKeyFrameIndex(CurrentSelectedKeyframe);
}

FReply FFlipbookEditor::OnClick_Forward()
{
	UPaperFlipbookComponent* PreviewComponent = GetPreviewComponent();

	const bool bIsReverse = PreviewComponent->IsReversing();
	const bool bIsPlaying = PreviewComponent->IsPlaying();
		
	if (bIsReverse && bIsPlaying)
	{
		// Play forwards instead of backwards
		PreviewComponent->Play();
	}
	else if (bIsPlaying)
	{
		// Was already playing forwards, so pause
		PreviewComponent->Stop();
	}
	else
	{
		// Was paused, start playing
		PreviewComponent->Play();
	}

	return FReply::Handled();
}

FReply FFlipbookEditor::OnClick_Forward_Step()
{
	GetPreviewComponent()->Stop();
	SetCurrentFrame(GetCurrentFrame() + 1);
	return FReply::Handled();
}

FReply FFlipbookEditor::OnClick_Forward_End()
{
	UPaperFlipbookComponent* PreviewComponent = GetPreviewComponent();
	PreviewComponent->Stop();
	PreviewComponent->SetPlaybackPosition(PreviewComponent->GetFlipbookLength(), /*bFireEvents=*/ false);
	return FReply::Handled();
}

FReply FFlipbookEditor::OnClick_Backward()
{
	UPaperFlipbookComponent* PreviewComponent = GetPreviewComponent();

	const bool bIsReverse = PreviewComponent->IsReversing();
	const bool bIsPlaying = PreviewComponent->IsPlaying();

	if (bIsReverse && bIsPlaying)
	{
		// Was already playing backwards, so pause
		PreviewComponent->Stop();
	}
	else if (bIsPlaying)
	{
		// Play backwards instead of forwards
		PreviewComponent->Reverse();
	}
	else
	{
		// Was paused, start reversing
		PreviewComponent->Reverse();
	}

	return FReply::Handled();
}

FReply FFlipbookEditor::OnClick_Backward_Step()
{
	GetPreviewComponent()->Stop();
	SetCurrentFrame(GetCurrentFrame() - 1);
	return FReply::Handled();
}

FReply FFlipbookEditor::OnClick_Backward_End()
{
	UPaperFlipbookComponent* PreviewComponent = GetPreviewComponent();
	PreviewComponent->Stop();
	PreviewComponent->SetPlaybackPosition(0.0f, /*bFireEvents=*/ false);
	return FReply::Handled();
}

FReply FFlipbookEditor::OnClick_ToggleLoop()
{
	UPaperFlipbookComponent* PreviewComponent = GetPreviewComponent();
	PreviewComponent->SetLooping(!PreviewComponent->IsLooping());
	return FReply::Handled();
}

EPlaybackMode::Type FFlipbookEditor::GetPlaybackMode() const
{
	UPaperFlipbookComponent* PreviewComponent = GetPreviewComponent();
	if (PreviewComponent->IsPlaying())
	{
		return PreviewComponent->IsReversing() ? EPlaybackMode::PlayingReverse : EPlaybackMode::PlayingForward;
	}
	else
	{
		return EPlaybackMode::Stopped;
	}
}

uint32 FFlipbookEditor::GetTotalFrameCount() const
{
	return FlipbookBeingEdited->GetNumFrames();
}

uint32 FFlipbookEditor::GetTotalFrameCountPlusOne() const
{
	return FlipbookBeingEdited->GetNumFrames() + 1;
}

float FFlipbookEditor::GetTotalSequenceLength() const
{
	return FlipbookBeingEdited->GetTotalDuration();
}

float FFlipbookEditor::GetPlaybackPosition() const
{
	return GetPreviewComponent()->GetPlaybackPosition();
}

void FFlipbookEditor::SetPlaybackPosition(float NewTime)
{
	NewTime = FMath::Clamp<float>(NewTime, 0.0f, GetTotalSequenceLength());

	GetPreviewComponent()->SetPlaybackPosition(NewTime, /*bFireEvents=*/ false);
}

bool FFlipbookEditor::IsLooping() const
{
	return GetPreviewComponent()->IsLooping();
}

float FFlipbookEditor::GetViewRangeMin() const
{
	return ViewInputMin;
}

float FFlipbookEditor::GetViewRangeMax() const
{
	// See if the flipbook changed length, and if so reframe the scrub bar to include the full length
	//@TODO: This is a pretty odd place to put it, but there's no callback for a modified timeline at the moment, so...
	const float SequenceLength = GetTotalSequenceLength();
	if (SequenceLength != LastObservedSequenceLength)
	{
		LastObservedSequenceLength = SequenceLength;
		ViewInputMin = 0.0f;
		ViewInputMax = SequenceLength;
	}

	return ViewInputMax;
}

void FFlipbookEditor::SetViewRange(float NewMin, float NewMax)
{
	ViewInputMin = FMath::Max<float>(NewMin, 0.0f);
	ViewInputMax = FMath::Min<float>(NewMax, GetTotalSequenceLength());
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
