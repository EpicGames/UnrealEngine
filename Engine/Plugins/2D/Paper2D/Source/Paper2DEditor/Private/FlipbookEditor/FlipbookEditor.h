// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"
#include "PaperFlipbook.h"
#include "ITransportControl.h"

class FSpawnTabArgs;

class SFlipbookEditorViewport;
class UPaperFlipbookComponent;

//////////////////////////////////////////////////////////////////////////
// FFlipbookEditor

class FFlipbookEditor : public FAssetEditorToolkit, public FGCObject
{
public:
	FFlipbookEditor();

	// IToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	// End of IToolkit interface

	// FAssetEditorToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FString GetDocumentationLink() const override;
	// End of FAssetEditorToolkit

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FFlipbookEditor");
	}
	// End of FSerializableObject interface

public:
	void InitFlipbookEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UPaperFlipbook* InitSprite);

	UPaperFlipbook* GetFlipbookBeingEdited() const { return FlipbookBeingEdited; }

	UPaperFlipbookComponent* GetPreviewComponent() const;
protected:
	TObjectPtr<UPaperFlipbook> FlipbookBeingEdited;

	TSharedPtr<SFlipbookEditorViewport> ViewportPtr;

	// Selection set
	int32 CurrentSelectedKeyframe;

	// Range of times currently being viewed
	mutable float ViewInputMin;
	mutable float ViewInputMax;
	mutable float LastObservedSequenceLength;

protected:
	float GetFramesPerSecond() const
	{
		return FlipbookBeingEdited->GetFramesPerSecond();
	}

	int32 GetCurrentFrame() const
	{
		const int32 TotalLengthInFrames = GetTotalFrameCount();

		if (TotalLengthInFrames == 0)
		{
			return INDEX_NONE;
		}
		else
		{
			return FMath::Clamp<int32>((int32)(GetPlaybackPosition() * GetFramesPerSecond()), 0, TotalLengthInFrames);
		}
	}

	void SetCurrentFrame(int32 NewIndex)
	{
		const int32 TotalLengthInFrames = GetTotalFrameCount();
		if (TotalLengthInFrames > 0)
		{
			int32 ClampedIndex = FMath::Clamp<int32>(NewIndex, 0, TotalLengthInFrames);
			SetPlaybackPosition(ClampedIndex / GetFramesPerSecond());
		}
		else
		{
			SetPlaybackPosition(0.0f);
		}
	}

protected:
	void BindCommands();
	void ExtendMenu();
	void ExtendToolbar();

	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);

	void DeleteSelection();
	void DuplicateSelection();
	void SetSelection(int32 NewSelection);
	bool HasValidSelection() const;

	void AddKeyFrameAtCurrentTime();
	void AddNewKeyFrameAtEnd();
	void AddNewKeyFrameBefore();
	void AddNewKeyFrameAfter();

	FReply OnClick_Forward();
	FReply OnClick_Forward_Step();
	FReply OnClick_Forward_End();
	FReply OnClick_Backward();
	FReply OnClick_Backward_Step();
	FReply OnClick_Backward_End();
	FReply OnClick_ToggleLoop();

	uint32 GetTotalFrameCount() const;
	uint32 GetTotalFrameCountPlusOne() const;
	float GetTotalSequenceLength() const;
	float GetPlaybackPosition() const;
	void SetPlaybackPosition(float NewTime);
	bool IsLooping() const;
	EPlaybackMode::Type GetPlaybackMode() const;

	float GetViewRangeMin() const;
	float GetViewRangeMax() const;
	void SetViewRange(float NewMin, float NewMax);
};
