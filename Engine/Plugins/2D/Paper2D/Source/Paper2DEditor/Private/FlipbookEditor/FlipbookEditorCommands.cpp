// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookEditor/FlipbookEditorCommands.h"

//////////////////////////////////////////////////////////////////////////
// FFlipbookEditorCommands

#define LOCTEXT_NAMESPACE "FlipbookEditorCommands"

void FFlipbookEditorCommands::RegisterCommands()
{
	UI_COMMAND(AddKeyFrame, "Add Key Frame", "Inserts a new key frame at the current time", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(SetShowGrid, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowBounds, "Bounds", "Toggles display of the bounds of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowCollision, "Collision", "Toggles display of the simplified collision mesh of the static mesh, if one has been assigned.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::C, EModifierKey::Alt));

	UI_COMMAND(SetShowPivot, "Show Pivot", "Display the pivot location of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowSockets, "Sockets", "Displays the flipbook sockets.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(AddNewFrame, "Add Key Frame", "Adds a new key frame to the flipbook.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewFrameBefore, "Insert Key Frame Before", "Adds a new key frame to the flipbook before the selection.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewFrameAfter, "Insert Key Frame After", "Adds a new key frame to the flipbook after the selection.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(PickNewSpriteFrame, "Pick New Sprite", "Picks a new sprite for this key frame.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EditSpriteFrame, "Edit Sprite", "Opens the sprite for this key frame in the Sprite Editor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowInContentBrowser, "Show in Content Browser", "Shows the sprite for this key frame in the Content Browser.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
