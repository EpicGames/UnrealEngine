// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "PaperStyle.h"
#include "Styling/ISlateStyle.h"

class FFlipbookEditorCommands : public TCommands<FFlipbookEditorCommands>
{
public:
	FFlipbookEditorCommands()
		: TCommands<FFlipbookEditorCommands>(
			TEXT("FlipbookEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "FlipbookEditor", "Flipbook Editor"), // Localized context name for displaying
			NAME_None, // Parent
			FPaperStyle::Get()->GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:
	TSharedPtr<FUICommandInfo> AddKeyFrame;

	TSharedPtr<FUICommandInfo> SetShowGrid;
	TSharedPtr<FUICommandInfo> SetShowBounds;
	TSharedPtr<FUICommandInfo> SetShowCollision;

	// View Menu Commands
	TSharedPtr<FUICommandInfo> SetShowPivot;
	TSharedPtr<FUICommandInfo> SetShowSockets;

	// Timeline commands
	TSharedPtr<FUICommandInfo> AddNewFrame;
	TSharedPtr<FUICommandInfo> AddNewFrameBefore;
	TSharedPtr<FUICommandInfo> AddNewFrameAfter;

	// Asset commands
	TSharedPtr<FUICommandInfo> PickNewSpriteFrame;
	TSharedPtr<FUICommandInfo> EditSpriteFrame;
	TSharedPtr<FUICommandInfo> ShowInContentBrowser;
};
