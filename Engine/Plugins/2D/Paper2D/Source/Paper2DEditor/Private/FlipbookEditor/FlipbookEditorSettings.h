// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FlipbookEditorSettings.generated.h"

// Settings for the Paper2D sprite editor
UCLASS(config=EditorPerProjectUserSettings)
class UFlipbookEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UFlipbookEditorSettings();

	/** Background color in the flipbook editor */
	UPROPERTY(config, EditAnywhere, Category=Background, meta=(HideAlphaChannel))
	FColor BackgroundColor;

	/** Should the grid be shown by default when the editor is opened? */
	UPROPERTY(config, EditAnywhere, Category=Background)
	bool bShowGridByDefault;
};
