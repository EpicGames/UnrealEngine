// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookEditor/FlipbookEditorViewportClient.h"
#include "PaperFlipbook.h"
#include "PaperFlipbookComponent.h"
#include "CanvasItem.h"
#include "Engine/CollisionProfile.h"
#include "Utils.h"

#include "CanvasTypes.h"
#include "PaperEditorShared/SocketEditing.h"
#include "FlipbookEditor/FlipbookEditorSettings.h"

#define LOCTEXT_NAMESPACE "FlipbookEditor"

//////////////////////////////////////////////////////////////////////////
// FFlipbookEditorViewportClient

FFlipbookEditorViewportClient::FFlipbookEditorViewportClient(const TAttribute<UPaperFlipbook*>& InFlipbookBeingEdited)
{
	FlipbookBeingEdited = InFlipbookBeingEdited;
	FlipbookBeingEditedLastFrame = FlipbookBeingEdited.Get();
	PreviewScene = &OwnedPreviewScene;

	SetRealtime(true);

	// Create a render component for the sprite being edited
	AnimatedRenderComponent = NewObject<UPaperFlipbookComponent>();
	AnimatedRenderComponent->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	AnimatedRenderComponent->SetFlipbook(FlipbookBeingEdited.Get());
	AnimatedRenderComponent->UpdateBounds();
	PreviewScene->AddComponent(AnimatedRenderComponent.Get(), FTransform::Identity);

	bShowPivot = false;
	bShowSockets = true;
	DrawHelper.bDrawGrid = GetDefault<UFlipbookEditorSettings>()->bShowGridByDefault;

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetCompositeEditorPrimitives(true);
}

void FFlipbookEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);

	const bool bIsHitTesting = Canvas.IsHitTesting();
	if (!bIsHitTesting)
	{
		Canvas.SetHitProxy(nullptr);
	}

	int32 YPos = 42;

	static const FText FlipbookHelpStr = LOCTEXT("FlipbookEditHelp", "Flipbook editor\n\nAdd keys using the toolbar or by drag-dropping sprite assets\nChange the timeline scale using Ctrl+MouseWheel\nEdit keys using the handles and right-click menu\nReorder keys by dragging and dropping");

	// Display tool help
	{
		FCanvasTextItem TextItem(FVector2D(6, YPos), FlipbookHelpStr, GEngine->GetSmallFont(), FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Draw(&Canvas);
		YPos += 36;
	}

	if (bShowSockets)
	{
		FSocketEditingHelper::DrawSocketNames(nullptr, AnimatedRenderComponent.Get(), InViewport, View, Canvas);
	}
}

void FFlipbookEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);

	if (bShowPivot && AnimatedRenderComponent.IsValid())
	{
		FUnrealEdUtils::DrawWidget(View, PDI, AnimatedRenderComponent->GetComponentTransform().ToMatrixWithScale(), 0, 0, EAxisList::Screen, EWidgetMovementMode::WMM_Translate);
	}

	if (bShowSockets)
	{
		FSocketEditingHelper::DrawSockets(nullptr, AnimatedRenderComponent.Get(), View, PDI);
	}
}

FBox FFlipbookEditorViewportClient::GetDesiredFocusBounds() const
{
	return AnimatedRenderComponent->Bounds.GetBox();
}

void FFlipbookEditorViewportClient::Tick(float DeltaSeconds)
{
	if (AnimatedRenderComponent.IsValid())
	{
		UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
		if (Flipbook != FlipbookBeingEditedLastFrame.Get())
		{
			AnimatedRenderComponent->SetFlipbook(Flipbook);
			AnimatedRenderComponent->UpdateBounds();
			FlipbookBeingEditedLastFrame = Flipbook;
		}
	}

	FPaperEditorViewportClient::Tick(DeltaSeconds);

	OwnedPreviewScene.GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

bool FFlipbookEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = false;

	// Pass keys to standard controls, if we didn't consume input
	return (bHandled) ? true : FEditorViewportClient::InputKey(EventArgs);
}

FLinearColor FFlipbookEditorViewportClient::GetBackgroundColor() const
{
	return GetDefault<UFlipbookEditorSettings>()->BackgroundColor;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
