// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreviewScene.h"
#include "PaperEditorViewportClient.h"

class FCanvas;
class UPaperFlipbookComponent;

//////////////////////////////////////////////////////////////////////////
// FFlipbookEditorViewportClient

class FFlipbookEditorViewportClient : public FPaperEditorViewportClient
{
public:
	/** Constructor */
	FFlipbookEditorViewportClient(const TAttribute<class UPaperFlipbook*>& InFlipbookBeingEdited);

	// FViewportClient interface
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual void Tick(float DeltaSeconds) override;
	// End of FViewportClient interface

	// FEditorViewportClient interface
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual FLinearColor GetBackgroundColor() const override;
	// End of FEditorViewportClient interface

	void ToggleShowPivot() { bShowPivot = !bShowPivot; Invalidate(); }
	bool IsShowPivotChecked() const { return bShowPivot; }

	void ToggleShowSockets() { bShowSockets = !bShowSockets; Invalidate(); }
	bool IsShowSocketsChecked() const { return bShowSockets; }

	UPaperFlipbookComponent* GetPreviewComponent() const
	{
		return AnimatedRenderComponent.Get();
	}

protected:
	// FPaperEditorViewportClient interface
	virtual FBox GetDesiredFocusBounds() const override;
	// End of FPaperEditorViewportClient interface

private:

	// The preview scene
	FPreviewScene OwnedPreviewScene;

	// The flipbook being displayed in this client
	TAttribute<class UPaperFlipbook*> FlipbookBeingEdited;

	// A cached pointer to the flipbook that was being edited last frame. Used for invalidation reasons.
	TWeakObjectPtr<class UPaperFlipbook> FlipbookBeingEditedLastFrame;

	// Render component for the sprite being edited
	TWeakObjectPtr<UPaperFlipbookComponent> AnimatedRenderComponent;

	// Should we show the sprite pivot?
	bool bShowPivot;

	// Should we show sockets?
	bool bShowSockets;
};
