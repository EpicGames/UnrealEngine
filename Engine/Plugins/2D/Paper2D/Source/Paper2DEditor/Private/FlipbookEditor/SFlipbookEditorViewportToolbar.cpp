// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookEditor/SFlipbookEditorViewportToolbar.h"
#include "Framework/Commands/UICommandList.h"
#include "SEditorViewport.h"
#include "FlipbookEditor/FlipbookEditorCommands.h"

#define LOCTEXT_NAMESPACE "SFlipbookEditorViewportToolbar"

///////////////////////////////////////////////////////////
// SFlipbookEditorViewportToolbar

void SFlipbookEditorViewportToolbar::Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider)
{
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InInfoProvider);
}

TSharedRef<SWidget> SFlipbookEditorViewportToolbar::GenerateShowMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();

	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		ShowMenuBuilder.AddMenuEntry(FFlipbookEditorCommands::Get().SetShowSockets);
		ShowMenuBuilder.AddMenuEntry(FFlipbookEditorCommands::Get().SetShowPivot);

		ShowMenuBuilder.AddMenuSeparator();

		ShowMenuBuilder.AddMenuEntry(FFlipbookEditorCommands::Get().SetShowGrid);
		ShowMenuBuilder.AddMenuEntry(FFlipbookEditorCommands::Get().SetShowBounds);

		ShowMenuBuilder.AddMenuSeparator();

		ShowMenuBuilder.AddMenuEntry(FFlipbookEditorCommands::Get().SetShowCollision);
	}

	return ShowMenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
