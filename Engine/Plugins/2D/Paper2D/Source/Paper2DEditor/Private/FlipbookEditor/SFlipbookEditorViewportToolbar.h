// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCommonEditorViewportToolbarBase.h"

// In-viewport toolbar widget used in the flipbook editor
class SFlipbookEditorViewportToolbar : public SCommonEditorViewportToolbarBase
{
public:
	SLATE_BEGIN_ARGS(SFlipbookEditorViewportToolbar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider);

	// SCommonEditorViewportToolbarBase interface
	virtual TSharedRef<SWidget> GenerateShowMenu() const override;
	// End of SCommonEditorViewportToolbarBase
};
