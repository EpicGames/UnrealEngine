// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookEditor/SFlipbookTimeline.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/SOverlay.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "FlipbookEditor/FlipbookEditorCommands.h"
#include "PaperSprite.h"
#include "FlipbookEditor/STimelineTrack.h"

#define LOCTEXT_NAMESPACE "FlipbookEditor"

//////////////////////////////////////////////////////////////////////////
// Inline widgets

#include "FlipbookEditor/STimelineHeader.h"

//////////////////////////////////////////////////////////////////////////
// SFlipbookTimeline

void SFlipbookTimeline::Construct(const FArguments& InArgs, TSharedPtr<FUICommandList> InCommandList)
{
	FlipbookBeingEdited = InArgs._FlipbookBeingEdited;
	PlayTime = InArgs._PlayTime;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	CommandList = InCommandList;

	SlateUnitsPerFrame = 120.0f;

	BackgroundPerFrameSlices = SNew(SHorizontalBox);

	TimelineHeader = SNew(STimelineHeader)
		.SlateUnitsPerFrame(this, &SFlipbookTimeline::GetSlateUnitsPerFrame)
		.FlipbookBeingEdited(FlipbookBeingEdited)
		.PlayTime(PlayTime);

	TimelineTrack = SNew(SFlipbookTimelineTrack, CommandList)
		.SlateUnitsPerFrame(this, &SFlipbookTimeline::GetSlateUnitsPerFrame)
		.FlipbookBeingEdited(FlipbookBeingEdited)
		.OnSelectionChanged(OnSelectionChanged);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SScrollBox)
			.Orientation(Orient_Horizontal)
			.ScrollBarAlwaysVisible(true)
			.OnUserScrolled(this, &SFlipbookTimeline::AnimationScrollBar_OnUserScrolled)
			+SScrollBox::Slot()
			[
				SNew(SOverlay)

				// Per-frame background
				+SOverlay::Slot()
				.VAlign(VAlign_Fill)
				[
					BackgroundPerFrameSlices.ToSharedRef()
				]

				// Flipbook header and track
				+SOverlay::Slot()
				[
					SNew(SVerticalBox)
		
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0,0,0,2)
					[
						TimelineHeader.ToSharedRef()
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.HeightOverride(FFlipbookUIConstants::FrameHeight)
						[
							TimelineTrack.ToSharedRef()
						]
					]
				]

				// Empty flipbook instructions
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Visibility(this, &SFlipbookTimeline::NoFramesWarningVisibility)
					.Text(LOCTEXT("EmptyTimelineInstruction", "Right-click here or drop in sprites to add key frames"))
				]
			]
		]
	];

	UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
	NumKeyFramesFromLastRebuild = (Flipbook != nullptr) ? Flipbook->GetNumKeyFrames() : 0;
	NumFramesFromLastRebuild = (Flipbook != nullptr) ? Flipbook->GetNumFrames() : 0;
	RebuildPerFrameBG();
}

void SFlipbookTimeline::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SCompoundWidget::OnDragEnter(MyGeometry, DragDropEvent);

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
	}
	else if (Operation->IsOfType<FFlipbookKeyFrameDragDropOp>())
	{
		const auto& FrameDragDropOp = StaticCastSharedPtr<FFlipbookKeyFrameDragDropOp>(Operation);
		FrameDragDropOp->SetCanDropHere(true);
	}
}

void SFlipbookTimeline::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SCompoundWidget::OnDragLeave(DragDropEvent);

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
	}
	else if (Operation->IsOfType<FFlipbookKeyFrameDragDropOp>())
	{
		const auto& FrameDragDropOp = StaticCastSharedPtr<FFlipbookKeyFrameDragDropOp>(Operation);
		FrameDragDropOp->SetCanDropHere(false);
	}
}

FReply SFlipbookTimeline::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bWasDropHandled = false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		const auto& AssetDragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
		OnAssetsDropped(*AssetDragDropOp);
		bWasDropHandled = true;
	}
	else if (Operation->IsOfType<FFlipbookKeyFrameDragDropOp>())
	{
		const auto& FrameDragDropOp = StaticCastSharedPtr<FFlipbookKeyFrameDragDropOp>(Operation);
		if (UPaperFlipbook* ThisFlipbook = FlipbookBeingEdited.Get())
		{
			FrameDragDropOp->AppendToFlipbook(ThisFlipbook);
			bWasDropHandled = true;
		}
	}

	return bWasDropHandled ? FReply::Handled() : FReply::Unhandled();
}

void SFlipbookTimeline::OnAssetsDropped(const class FAssetDragDropOp& DragDropOp)
{
	//@TODO: Support inserting in addition to dropping at the end
	TArray<FPaperFlipbookKeyFrame> NewFrames;
	for (const FAssetData& AssetData : DragDropOp.GetAssets())
	{
		if (UObject* Object = AssetData.GetAsset())
		{
			if (UPaperSprite* SpriteAsset = Cast<UPaperSprite>(Object))
			{
				// Insert this sprite as a keyframe
				FPaperFlipbookKeyFrame& NewFrame = *new (NewFrames) FPaperFlipbookKeyFrame();
				NewFrame.Sprite = SpriteAsset;
			}
			else if (UPaperFlipbook* FlipbookAsset = Cast<UPaperFlipbook>(Object))
			{
				// Insert all of the keyframes from the other flipbook into this one
				for (int32 KeyIndex = 0; KeyIndex < FlipbookAsset->GetNumKeyFrames(); ++KeyIndex)
				{
					const FPaperFlipbookKeyFrame& OtherFlipbookFrame = FlipbookAsset->GetKeyFrameChecked(KeyIndex);
					FPaperFlipbookKeyFrame& NewFrame = *new (NewFrames) FPaperFlipbookKeyFrame();
					NewFrame = OtherFlipbookFrame;
				}
			}
		}
	}

	UPaperFlipbook* ThisFlipbook = FlipbookBeingEdited.Get();
	if (NewFrames.Num() && (ThisFlipbook != nullptr))
	{
		const FScopedTransaction Transaction(LOCTEXT("DroppedAssetOntoTimelineTransaction", "Insert assets as frames"));
		ThisFlipbook->Modify();

		FScopedFlipbookMutator EditLock(ThisFlipbook);
		EditLock.KeyFrames.Append(NewFrames);
	}
}

int32 SFlipbookTimeline::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	const float CurrentTimeSecs = PlayTime.Get();
	UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
	const float TotalTimeSecs = (Flipbook != nullptr) ? Flipbook->GetTotalDuration() : 0.0f;
	const int32 TotalNumFrames = (Flipbook != nullptr) ? Flipbook->GetNumFrames() : 0;

	const float SlateTotalDistance = SlateUnitsPerFrame * TotalNumFrames;
	const float CurrentTimeXPos = (((CurrentTimeSecs / TotalTimeSecs) * SlateTotalDistance) - AnimationScrollBarPosition) + FMath::Clamp((AllottedGeometry.GetLocalSize().X + AnimationScrollBarPosition) - SlateTotalDistance, 0.0f, AnimationScrollBarPosition);

	// Draw a line for the current scrub cursor
	++LayerId;
	TArray<FVector2D> LinePoints;
	LinePoints.Add(FVector2D(CurrentTimeXPos, 0.f));
	LinePoints.Add(FVector2D(CurrentTimeXPos, AllottedGeometry.GetLocalSize().Y));

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::None,
		FLinearColor::Red
		);

	return LayerId;
}

FReply SFlipbookTimeline::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsControlDown())
	{
		const float DirectionScale = 0.08f;
		const float MinFrameSize = 16.0f;
		const float Direction = MouseEvent.GetWheelDelta();
		const float NewUnitsPerFrame = FMath::Max(MinFrameSize, SlateUnitsPerFrame * (1.0f + Direction * DirectionScale));
		SlateUnitsPerFrame = NewUnitsPerFrame;
		
		CheckForRebuild(/*bRebuildAll=*/ true);

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

TSharedRef<SWidget> SFlipbookTimeline::GenerateContextMenu()
{
	FMenuBuilder MenuBuilder(true, CommandList);
	MenuBuilder.BeginSection("KeyframeActions", LOCTEXT("KeyframeActionsSectionHeader", "Keyframe Actions"));

	// 		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
	// 		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
	// 		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
	MenuBuilder.AddMenuEntry(FFlipbookEditorCommands::Get().AddNewFrame);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FReply SFlipbookTimeline::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		TSharedRef<SWidget> MenuContents = GenerateContextMenu();
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContents, MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

void SFlipbookTimeline::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	CheckForRebuild();
}

void SFlipbookTimeline::CheckForRebuild(bool bRebuildAll)
{
	UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();

	const int32 NewNumKeyframes = (Flipbook != nullptr) ? Flipbook->GetNumKeyFrames() : 0;
	if ((NewNumKeyframes != NumKeyFramesFromLastRebuild) || bRebuildAll)
	{
		NumKeyFramesFromLastRebuild = NewNumKeyframes;
		TimelineTrack->Rebuild();
	}

	const int32 NewNumFrames = (Flipbook != nullptr) ? Flipbook->GetNumFrames() : 0;
	if ((NewNumFrames != NumFramesFromLastRebuild) || bRebuildAll)
	{
		NumFramesFromLastRebuild = NewNumFrames;
		TimelineHeader->Rebuild();
		RebuildPerFrameBG();
	}
}

EVisibility SFlipbookTimeline::NoFramesWarningVisibility() const
{
	UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
	const int32 TotalNumFrames = (Flipbook != nullptr) ? Flipbook->GetNumFrames() : 0;
	return (TotalNumFrames == 0) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SFlipbookTimeline::RebuildPerFrameBG()
{
	const FLinearColor BackgroundColors[2] = { FLinearColor(1.0f, 1.0f, 1.0f, 0.05f), FLinearColor(0.0f, 0.0f, 0.0f, 0.05f) };

	BackgroundPerFrameSlices->ClearChildren();
	for (int32 FrameIndex = 0; FrameIndex < NumFramesFromLastRebuild; ++FrameIndex)
	{
		const FLinearColor& BackgroundColorForFrameIndex = BackgroundColors[FrameIndex & 1];

		BackgroundPerFrameSlices->AddSlot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(SlateUnitsPerFrame)
			[
				SNew(SColorBlock)
				.Color(BackgroundColorForFrameIndex)
			]
		];
	}
}

void SFlipbookTimeline::AnimationScrollBar_OnUserScrolled(float ScrollOffset)
{
	AnimationScrollBarPosition = ScrollOffset;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
