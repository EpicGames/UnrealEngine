// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UPaperFlipbook;

class FPaintArgs;
class FSlateWindowElementList;
class FUICommandList;
class SFlipbookTimelineTrack;
class SHorizontalBox;
class STimelineHeader;

// Called when the selection changes
DECLARE_DELEGATE_OneParam(FOnFlipbookKeyframeSelectionChanged, int32);

class SFlipbookTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFlipbookTimeline)
		: _FlipbookBeingEdited(nullptr)
		, _PlayTime(0)
	{}

		SLATE_ATTRIBUTE(class UPaperFlipbook*, FlipbookBeingEdited)
		SLATE_ATTRIBUTE(float, PlayTime)
		SLATE_EVENT(FOnFlipbookKeyframeSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FUICommandList> InCommandList);

	// SWidget interface
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

private:
	void OnAssetsDropped(const class FAssetDragDropOp& DragDropOp);
	void RebuildPerFrameBG();

	TSharedRef<SWidget> GenerateContextMenu();
	EVisibility NoFramesWarningVisibility() const;
	void CheckForRebuild(bool bRebuildAll = false);

	float GetSlateUnitsPerFrame() const { return SlateUnitsPerFrame; }

	void AnimationScrollBar_OnUserScrolled(float ScrollOffset);
private:
	TSharedPtr<SHorizontalBox> BackgroundPerFrameSlices;
	TSharedPtr<class STimelineHeader> TimelineHeader;
	TSharedPtr<class SFlipbookTimelineTrack> TimelineTrack;

	TAttribute<class UPaperFlipbook*> FlipbookBeingEdited;
	TAttribute<float> PlayTime;
	TSharedPtr<FUICommandList> CommandList;
	FOnFlipbookKeyframeSelectionChanged OnSelectionChanged;
	float SlateUnitsPerFrame;
	float AnimationScrollBarPosition;

	// Observer values (used to detect if the widget needs to be rebuilt)
	int32 NumFramesFromLastRebuild;
	int32 NumKeyFramesFromLastRebuild;
};
