// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperFlipbook.h"
#include "Framework/Application/SlateApplication.h"
#include "PaperStyle.h"
#include "Widgets/Images/SImage.h"

//////////////////////////////////////////////////////////////////////////
// SFlipbookTrackHandle

// This is the grab handle at the end of a frame region, which can be dragged to change the duration
class SFlipbookTrackHandle : public SImage
{
public:
	SLATE_BEGIN_ARGS(SFlipbookTrackHandle)
		: _SlateUnitsPerFrame(1)
		, _FlipbookBeingEdited(nullptr)
		, _KeyFrameIdx(INDEX_NONE)
	{}

		SLATE_ATTRIBUTE( float, SlateUnitsPerFrame )
		SLATE_ATTRIBUTE( class UPaperFlipbook*, FlipbookBeingEdited )
		SLATE_ARGUMENT( int32, KeyFrameIdx )
		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SlateUnitsPerFrame = InArgs._SlateUnitsPerFrame;
		FlipbookBeingEdited = InArgs._FlipbookBeingEdited;
		KeyFrameIdx = InArgs._KeyFrameIdx;

		DistanceDragged = 0;
		bDragging = false;
		StartingFrameRun = INDEX_NONE;

		SImage::Construct(
			SImage::FArguments()
			.Image(FPaperStyle::Get()->GetBrush("FlipbookEditor.RegionGrabHandle")));
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			DistanceDragged = 0;
			StartingFrameRun = INDEX_NONE;
			return FReply::Handled().CaptureMouse(SharedThis(this)).UseHighPrecisionMouseMovement(SharedThis(this));
		}
		else
		{

			return FReply::Unhandled();
		}
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && this->HasMouseCapture())
		{
			if (bDragging && (StartingFrameRun != INDEX_NONE))
			{
				UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
				if (Flipbook && Flipbook->IsValidKeyFrameIndex(KeyFrameIdx))
				{
					const FPaperFlipbookKeyFrame& KeyFrame = Flipbook->GetKeyFrameChecked(KeyFrameIdx);

					if (KeyFrame.FrameRun != StartingFrameRun)
					{
						Flipbook->MarkPackageDirty();
						Flipbook->PostEditChange();
					}
				}
			}

			bDragging = false;

			FIntPoint NewMousePos(
				(MyGeometry.AbsolutePosition.X + MyGeometry.GetLocalSize().X / 2) * MyGeometry.Scale,
				(MyGeometry.AbsolutePosition.Y + MyGeometry.GetLocalSize().Y / 2) * MyGeometry.Scale
				);

			return FReply::Handled().ReleaseMouseCapture().SetMousePos(NewMousePos);

		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (this->HasMouseCapture())
		{
			UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
			if (Flipbook && Flipbook->IsValidKeyFrameIndex(KeyFrameIdx))
			{
				DistanceDragged += MouseEvent.GetCursorDelta().X;

				if (!bDragging)
				{
					if (FMath::Abs(DistanceDragged) > FSlateApplication::Get().GetDragTriggerDistance())
					{
						const FPaperFlipbookKeyFrame& KeyFrame = Flipbook->GetKeyFrameChecked(KeyFrameIdx);
						StartingFrameRun = KeyFrame.FrameRun;
						bDragging = true;
					}
				}
				else
				{
					float LocalSlateUnitsPerFrame = SlateUnitsPerFrame.Get();
					if (LocalSlateUnitsPerFrame != 0)
					{
						FScopedFlipbookMutator EditLock(Flipbook);
						FPaperFlipbookKeyFrame& KeyFrame = EditLock.KeyFrames[KeyFrameIdx];
						KeyFrame.FrameRun = StartingFrameRun + (DistanceDragged / LocalSlateUnitsPerFrame);
						KeyFrame.FrameRun = FMath::Max<int32>(1, KeyFrame.FrameRun);
					}
				}
			}

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override
	{
		return FCursorReply::Cursor(bDragging ? EMouseCursor::None : EMouseCursor::ResizeLeftRight);
	}

private:
	float DistanceDragged;
	int32 StartingFrameRun;
	bool bDragging;

	TAttribute<float> SlateUnitsPerFrame;
	TAttribute<class UPaperFlipbook*> FlipbookBeingEdited;
	int32 KeyFrameIdx;
};
