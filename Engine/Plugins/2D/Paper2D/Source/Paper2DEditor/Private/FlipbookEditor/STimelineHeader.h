// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "PaperFlipbook.h"

//////////////////////////////////////////////////////////////////////////
// STimelineHeader

// This is the bar above the timeline which (will someday) shows the frame ticks and current time
class STimelineHeader : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STimelineHeader)
		: _SlateUnitsPerFrame(1)
		, _FlipbookBeingEdited((class UPaperFlipbook*)nullptr)
		, _PlayTime(0)
	{}

		SLATE_ATTRIBUTE(float, SlateUnitsPerFrame)
		SLATE_ATTRIBUTE(class UPaperFlipbook*, FlipbookBeingEdited)
		SLATE_ATTRIBUTE(float, PlayTime)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SlateUnitsPerFrame = InArgs._SlateUnitsPerFrame;
		FlipbookBeingEdited = InArgs._FlipbookBeingEdited;
		PlayTime = InArgs._PlayTime;

		NumFramesFromLastRebuild = 0;

		ChildSlot
		[
			SAssignNew(MainBoxPtr, SHorizontalBox)
		];

		Rebuild();
	}

	void Rebuild()
	{
		MainBoxPtr->ClearChildren();

		UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
		const float LocalSlateUnitsPerFrame = SlateUnitsPerFrame.Get();
		if ((Flipbook != nullptr) && (LocalSlateUnitsPerFrame > 0))
		{
			const int32 NumFrames = Flipbook->GetNumFrames();
			for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
			{
				MainBoxPtr->AddSlot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(LocalSlateUnitsPerFrame)
						.HAlign(HAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::AsNumber(FrameIndex))
						]
					];
			}

			NumFramesFromLastRebuild = NumFrames;
		}
		else
		{
			NumFramesFromLastRebuild = 0;
		}
	}

private:
	TAttribute<float> SlateUnitsPerFrame;
	TAttribute<class UPaperFlipbook*> FlipbookBeingEdited;
	TAttribute<float> PlayTime;

	TSharedPtr<SHorizontalBox> MainBoxPtr;

	int32 NumFramesFromLastRebuild;
};
