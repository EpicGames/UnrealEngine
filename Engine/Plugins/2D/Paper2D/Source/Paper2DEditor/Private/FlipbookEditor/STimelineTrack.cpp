// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlipbookEditor/STimelineTrack.h"
#include "Layout/WidgetPath.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/GenericCommands.h"
#include "FlipbookEditor/FlipbookEditorCommands.h"
#include "FlipbookEditor/SFlipbookTrackHandle.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor.h"
#include "PropertyCustomizationHelpers.h"
#include "PaperSprite.h"

#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/ImportSubsystem.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "FlipbookEditor"

//////////////////////////////////////////////////////////////////////////
// SFlipbookKeyframeWidget

TSharedRef<SWidget> SFlipbookKeyframeWidget::GenerateContextMenu()
{
	const FFlipbookEditorCommands& Commands = FFlipbookEditorCommands::Get();

	OnSelectionChanged.ExecuteIfBound(FrameIndex);

	FMenuBuilder MenuBuilder(true, CommandList);
	{
		FNumberFormattingOptions NoCommas;
		NoCommas.UseGrouping = false;
		
		const FText KeyframeSectionTitle = FText::Format(LOCTEXT("KeyframeXActionsSectionHeader", "Keyframe #{0} Actions"), FText::AsNumber(FrameIndex, &NoCommas));
		MenuBuilder.BeginSection("KeyframeActions", KeyframeSectionTitle);

		// 		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		// 		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		// 		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);

		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddMenuEntry(Commands.AddNewFrameBefore);
		MenuBuilder.AddMenuEntry(Commands.AddNewFrameAfter);

		MenuBuilder.EndSection();
	}

	{
		TAttribute<FText> CurrentAssetTitle = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SFlipbookKeyframeWidget::GetKeyframeAssetName));
		MenuBuilder.BeginSection("KeyframeAssetActions", CurrentAssetTitle);

		MenuBuilder.AddMenuEntry(Commands.ShowInContentBrowser);
		MenuBuilder.AddMenuEntry(Commands.EditSpriteFrame);

		MenuBuilder.AddSubMenu(
			Commands.PickNewSpriteFrame->GetLabel(),
			Commands.PickNewSpriteFrame->GetDescription(),
			FNewMenuDelegate::CreateSP(this, &SFlipbookKeyframeWidget::OpenSpritePickerMenu));

		MenuBuilder.EndSection();
	}
	return MenuBuilder.MakeWidget();
}

void SFlipbookKeyframeWidget::Construct(const FArguments& InArgs, int32 InFrameIndex, TSharedPtr<FUICommandList> InCommandList)
{
	FrameIndex = InFrameIndex;

	const FFlipbookEditorCommands& Commands = FFlipbookEditorCommands::Get();
	CommandList = MakeShareable(new FUICommandList);
	CommandList->Append(InCommandList.ToSharedRef());
	CommandList->MapAction(Commands.ShowInContentBrowser, FExecuteAction::CreateSP(this, &SFlipbookKeyframeWidget::ShowInContentBrowser));
	CommandList->MapAction(Commands.EditSpriteFrame, FExecuteAction::CreateSP(this, &SFlipbookKeyframeWidget::EditKeyFrame));

	SlateUnitsPerFrame = InArgs._SlateUnitsPerFrame;
	FlipbookBeingEdited = InArgs._FlipbookBeingEdited;
	OnSelectionChanged = InArgs._OnSelectionChanged;

	// Color each region based on whether a sprite has been set or not for it
	const auto BorderColorDelegate = [](TAttribute<UPaperFlipbook*> ThisFlipbookPtr, int32 TestIndex) -> FSlateColor
	{
		UPaperFlipbook* FlipbookPtr = ThisFlipbookPtr.Get();
		const bool bFrameValid = (FlipbookPtr != nullptr) && (FlipbookPtr->GetSpriteAtFrame(TestIndex) != nullptr);
		return bFrameValid ? FLinearColor::White : FLinearColor::Black;
	};

	ChildSlot
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			SNew(SBox)
			.Padding(FFlipbookUIConstants::FramePadding)
			.WidthOverride(this, &SFlipbookKeyframeWidget::GetFrameWidth)
			[
				SNew(SBorder)
				.BorderImage(FPaperStyle::Get()->GetBrush("FlipbookEditor.RegionBody"))
				.BorderBackgroundColor_Static(BorderColorDelegate, FlipbookBeingEdited, FrameIndex)
				.OnMouseButtonUp(this, &SFlipbookKeyframeWidget::KeyframeOnMouseButtonUp)
				.ToolTipText(this, &SFlipbookKeyframeWidget::GetKeyframeTooltip)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FLinearColor::Black)
					.Text(this, &SFlipbookKeyframeWidget::GetKeyframeText)
				]
			]
		]
		+SOverlay::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.WidthOverride(FFlipbookUIConstants::HandleWidth)
			[
				SNew(SFlipbookTrackHandle)
				.SlateUnitsPerFrame(SlateUnitsPerFrame)
				.FlipbookBeingEdited(FlipbookBeingEdited)
				.KeyFrameIdx(FrameIndex)
			]
		]
	];
}

FReply SFlipbookKeyframeWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return FReply::Unhandled();
}

FReply SFlipbookKeyframeWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
		if ((Flipbook != nullptr) && Flipbook->IsValidKeyFrameIndex(FrameIndex))
		{
			TSharedRef<FFlipbookKeyFrameDragDropOp> Operation = FFlipbookKeyFrameDragDropOp::New(
				GetFrameWidth().Get(), Flipbook, FrameIndex);

			return FReply::Handled().BeginDragDrop(Operation);
		}
	}

	return FReply::Unhandled();
}

FReply SFlipbookKeyframeWidget::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bWasDropHandled = false;

	UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
	if ((Flipbook != nullptr) && Flipbook->IsValidKeyFrameIndex(FrameIndex))
	{

		TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
		if (!Operation.IsValid())
		{
		}
		else if (Operation->IsOfType<FAssetDragDropOp>())
		{
			const auto& AssetDragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
			//@TODO: Handle asset inserts

			// 			OnAssetsDropped(*AssetDragDropOp);
			// 			bWasDropHandled = true;
		}
		else if (Operation->IsOfType<FFlipbookKeyFrameDragDropOp>())
		{
			const auto& FrameDragDropOp = StaticCastSharedPtr<FFlipbookKeyFrameDragDropOp>(Operation);
			FrameDragDropOp->InsertInFlipbook(Flipbook, FrameIndex);
			bWasDropHandled = true;
		}
	}

	return bWasDropHandled ? FReply::Handled() : FReply::Unhandled();
}

FReply SFlipbookKeyframeWidget::KeyframeOnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		TSharedRef<SWidget> MenuContents = GenerateContextMenu();
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContents, MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

// Can return null
const FPaperFlipbookKeyFrame* SFlipbookKeyframeWidget::GetKeyFrameData() const
{
	UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get();
	if ((Flipbook != nullptr) && Flipbook->IsValidKeyFrameIndex(FrameIndex))
	{
		return &(Flipbook->GetKeyFrameChecked(FrameIndex));
	}

	return nullptr;
}

FText SFlipbookKeyframeWidget::GetKeyframeAssetName() const
{
	if (const FPaperFlipbookKeyFrame* KeyFrame = GetKeyFrameData())
	{
		const FText SpriteLine = (KeyFrame->Sprite != nullptr) ? FText::FromString(KeyFrame->Sprite->GetName()) : LOCTEXT("NoSprite", "(none)");
		return FText::Format(LOCTEXT("KeyFrameAssetName", "Current Asset: {0}"), SpriteLine);
	}
	else
	{
		return LOCTEXT("KeyFrameAssetName_None", "Current Asset: (none)");
	}
}

FText SFlipbookKeyframeWidget::GetKeyframeText() const
{
	if (const FPaperFlipbookKeyFrame* KeyFrame = GetKeyFrameData())
	{
		if (KeyFrame->Sprite != nullptr)
		{
			return FText::AsCultureInvariant(KeyFrame->Sprite->GetName());
		}
	}

	return FText::GetEmpty();
}

FText SFlipbookKeyframeWidget::GetKeyframeTooltip() const
{
	if (const FPaperFlipbookKeyFrame* KeyFrame = GetKeyFrameData())
	{
		const FText SpriteLine = (KeyFrame->Sprite != nullptr) ? FText::FromString(KeyFrame->Sprite->GetName()) : LOCTEXT("NoSprite", "(none)");

		const FText FramesText = (KeyFrame->FrameRun == 1) ? LOCTEXT("SingularFrames", "frame") : LOCTEXT("PluralFrames", "frames");
		
		return FText::Format(LOCTEXT("KeyFrameTooltip", "Sprite: {0}\nIndex: {1}\nDuration: {2} {3}"),
			SpriteLine,
			FText::AsNumber(FrameIndex),
			FText::AsNumber(KeyFrame->FrameRun),
			FramesText);
	}
	else
	{
		return LOCTEXT("KeyFrameTooltip_Invalid", "Invalid key frame index");
	}
}

FOptionalSize SFlipbookKeyframeWidget::GetFrameWidth() const
{
	if (const FPaperFlipbookKeyFrame* KeyFrame = GetKeyFrameData())
	{
		return FMath::Max<float>(0, KeyFrame->FrameRun * SlateUnitsPerFrame.Get());
	}
	else
	{
		return 1;
	}
}

void SFlipbookKeyframeWidget::OpenSpritePickerMenu(FMenuBuilder& MenuBuilder)
{
	const bool bAllowClear = true;

	TArray<const UClass*> AllowedClasses;
	AllowedClasses.Add(UPaperSprite::StaticClass());

	FAssetData CurrentAssetData;
	if (const FPaperFlipbookKeyFrame* KeyFrame = GetKeyFrameData())
	{
		CurrentAssetData = FAssetData(KeyFrame->Sprite);
	}

	TSharedRef<SWidget> AssetPickerWidget = PropertyCustomizationHelpers::MakeAssetPickerWithMenu(CurrentAssetData,
		bAllowClear,
		AllowedClasses,
		PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses),
		FOnShouldFilterAsset(),
		FOnAssetSelected::CreateSP(this, &SFlipbookKeyframeWidget::OnAssetSelected),
		FSimpleDelegate::CreateSP(this, &SFlipbookKeyframeWidget::CloseMenu));

	MenuBuilder.AddWidget(AssetPickerWidget, FText::GetEmpty(), /*bNoIndent=*/ true);
}

void SFlipbookKeyframeWidget::CloseMenu()
{
	FSlateApplication::Get().DismissAllMenus();
}

void SFlipbookKeyframeWidget::OnAssetSelected(const FAssetData& AssetData)
{
	if (UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get())
	{
		FScopedFlipbookMutator EditLock(Flipbook);

		if (EditLock.KeyFrames.IsValidIndex(FrameIndex))
		{
			UPaperSprite* NewSprite = Cast<UPaperSprite>(AssetData.GetAsset());

			EditLock.KeyFrames[FrameIndex].Sprite = NewSprite;
		}
	}
}

void SFlipbookKeyframeWidget::ShowInContentBrowser()
{
	if (const FPaperFlipbookKeyFrame* KeyFrame = GetKeyFrameData())
	{
		if (KeyFrame->Sprite != nullptr)
		{
			TArray<UObject*> ObjectsToSync;
			ObjectsToSync.Add(KeyFrame->Sprite);
			GEditor->SyncBrowserToObjects(ObjectsToSync);
		}
	}
}

void SFlipbookKeyframeWidget::EditKeyFrame()
{
	if (const FPaperFlipbookKeyFrame* KeyFrame = GetKeyFrameData())
	{
		if (KeyFrame->Sprite != nullptr)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(KeyFrame->Sprite);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FFlipbookKeyFrameDragDropOp

TSharedPtr<SWidget> FFlipbookKeyFrameDragDropOp::GetDefaultDecorator() const
{
	const FLinearColor BorderColor = (KeyFrameData.Sprite != nullptr) ? FLinearColor::White : FLinearColor::Black;

	return SNew(SBox)
		.WidthOverride(WidgetWidth - FFlipbookUIConstants::FramePadding.GetTotalSpaceAlong<Orient_Horizontal>())
		.HeightOverride(FFlipbookUIConstants::FrameHeight - FFlipbookUIConstants::FramePadding.GetTotalSpaceAlong<Orient_Vertical>())
		[
			SNew(SBorder)
			.BorderImage(FPaperStyle::Get()->GetBrush("FlipbookEditor.RegionBody"))
			.BorderBackgroundColor(BorderColor)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::Black)
				.Text(BodyText)
			]
		];
}

void FFlipbookKeyFrameDragDropOp::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition());
	}
}

void FFlipbookKeyFrameDragDropOp::Construct()
{
	MouseCursor = EMouseCursor::GrabHandClosed;

	if (UPaperFlipbook* Flipbook = SourceFlipbook.Get())
	{
		if (UPaperSprite* Sprite = Flipbook->GetSpriteAtFrame(SourceFrameIndex))
		{
			BodyText = FText::AsCultureInvariant(Sprite->GetName());
		}
	}

	FDragDropOperation::Construct();
}

void FFlipbookKeyFrameDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if (!bDropWasHandled)
	{
		// Add us back to our source, the drop fizzled
		InsertInFlipbook(SourceFlipbook.Get(), SourceFrameIndex);
		Transaction.Cancel();
	}
}

void FFlipbookKeyFrameDragDropOp::AppendToFlipbook(UPaperFlipbook* DestinationFlipbook)
{
	DestinationFlipbook->Modify();
	FScopedFlipbookMutator EditLock(DestinationFlipbook);
	EditLock.KeyFrames.Add(KeyFrameData);
}

void FFlipbookKeyFrameDragDropOp::InsertInFlipbook(UPaperFlipbook* DestinationFlipbook, int32 Index)
{
	DestinationFlipbook->Modify();
	FScopedFlipbookMutator EditLock(DestinationFlipbook);
	EditLock.KeyFrames.Insert(KeyFrameData, Index);
}

TSharedRef<FFlipbookKeyFrameDragDropOp> FFlipbookKeyFrameDragDropOp::New(int32 InWidth, UPaperFlipbook* InFlipbook, int32 InFrameIndex)
{
	// Create the drag-drop op containing the key
	TSharedRef<FFlipbookKeyFrameDragDropOp> Operation = MakeShareable(new FFlipbookKeyFrameDragDropOp);
	Operation->KeyFrameData = InFlipbook->GetKeyFrameChecked(InFrameIndex);
	Operation->SourceFrameIndex = InFrameIndex;
	Operation->SourceFlipbook = InFlipbook;
	Operation->WidgetWidth = InWidth;
	Operation->Construct();

	// Remove the key from the flipbook
	{
		InFlipbook->Modify();
		FScopedFlipbookMutator EditLock(InFlipbook);
		EditLock.KeyFrames.RemoveAt(InFrameIndex);
	}

	return Operation;
}

FFlipbookKeyFrameDragDropOp::FFlipbookKeyFrameDragDropOp()
	: Transaction(LOCTEXT("MovedFramesInTimeline", "Reorder key frames"))
{

}

//////////////////////////////////////////////////////////////////////////
// SFlipbookTimelineTrack

void SFlipbookTimelineTrack::Construct(const FArguments& InArgs, TSharedPtr<FUICommandList> InCommandList)
{
	CommandList = InCommandList;
	SlateUnitsPerFrame = InArgs._SlateUnitsPerFrame;
	FlipbookBeingEdited = InArgs._FlipbookBeingEdited;
	OnSelectionChanged = InArgs._OnSelectionChanged;

	ChildSlot
	[
		SAssignNew(MainBoxPtr, SHorizontalBox)
	];

	Rebuild();
}

void SFlipbookTimelineTrack::Rebuild()
{
	MainBoxPtr->ClearChildren();

	// Create the sections for each keyframe
	if (UPaperFlipbook* Flipbook = FlipbookBeingEdited.Get())
	{
		for (int32 KeyFrameIdx = 0; KeyFrameIdx < Flipbook->GetNumKeyFrames(); ++KeyFrameIdx)
		{
			MainBoxPtr->AddSlot()
			.AutoWidth()
			[
				SNew(SFlipbookKeyframeWidget, KeyFrameIdx, CommandList)
				.SlateUnitsPerFrame(this->SlateUnitsPerFrame)
				.FlipbookBeingEdited(this->FlipbookBeingEdited)
				.OnSelectionChanged(this->OnSelectionChanged)
			];
		}
	}
}

#undef LOCTEXT_NAMESPACE
