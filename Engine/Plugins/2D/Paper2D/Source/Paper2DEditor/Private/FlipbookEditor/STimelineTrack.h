// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "FlipbookEditor/SFlipbookTimeline.h"
#include "PaperFlipbook.h"
#include "ScopedTransaction.h"

struct FOptionalSize;

class FUICommandList;
class SHorizontalBox;

namespace FFlipbookUIConstants
{
	const float HandleWidth = 12.0f;
	const float FrameHeight = 48;
	const float HeightBeforeFrames = 16;
	const FMargin FramePadding(0.0f, 7.0f, 0.0f, 7.0f);
};

//////////////////////////////////////////////////////////////////////////
// FFlipbookKeyFrameDragDropOp

class FFlipbookKeyFrameDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FFlipbookKeyFrameDragDropOp, FDragDropOperation)

	float WidgetWidth;
	FPaperFlipbookKeyFrame KeyFrameData;
	int32 SourceFrameIndex;
	FText BodyText;
	TWeakObjectPtr<UPaperFlipbook> SourceFlipbook;
	FScopedTransaction Transaction;

	// FDragDropOperation interface
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
	virtual void Construct() override;
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	// End of FDragDropOperation interface

	void AppendToFlipbook(UPaperFlipbook* DestinationFlipbook);

	void InsertInFlipbook(UPaperFlipbook* DestinationFlipbook, int32 Index);

	void SetCanDropHere(bool bCanDropHere)
	{
		MouseCursor = bCanDropHere ? EMouseCursor::TextEditBeam : EMouseCursor::SlashedCircle;
	}

	static TSharedRef<FFlipbookKeyFrameDragDropOp> New(int32 InWidth, UPaperFlipbook* InFlipbook, int32 InFrameIndex);

protected:
	FFlipbookKeyFrameDragDropOp();
};

//////////////////////////////////////////////////////////////////////////
// SFlipbookKeyframeWidget

class SFlipbookKeyframeWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFlipbookKeyframeWidget)
		: _SlateUnitsPerFrame(1)
		, _FlipbookBeingEdited(nullptr)
	{}

		SLATE_ATTRIBUTE( float, SlateUnitsPerFrame )
		SLATE_ATTRIBUTE( class UPaperFlipbook*, FlipbookBeingEdited )
		SLATE_EVENT( FOnFlipbookKeyframeSelectionChanged, OnSelectionChanged )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, int32 InFrameIndex, TSharedPtr<FUICommandList> InCommandList);

	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

protected:
	FReply KeyframeOnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	FText GetKeyframeAssetName() const;
	FText GetKeyframeText() const;
	FText GetKeyframeTooltip() const;

	TSharedRef<SWidget> GenerateContextMenu();

	FOptionalSize GetFrameWidth() const;

	void OpenSpritePickerMenu(class FMenuBuilder& MenuBuilder);
	void OnAssetSelected(const struct FAssetData& AssetData);
	void CloseMenu();
	void ShowInContentBrowser();
	void EditKeyFrame();

	// Can return null
	const struct FPaperFlipbookKeyFrame* GetKeyFrameData() const;

protected:
	int32 FrameIndex;
	TAttribute<float> SlateUnitsPerFrame;
	TAttribute<class UPaperFlipbook*> FlipbookBeingEdited;
	FOnFlipbookKeyframeSelectionChanged OnSelectionChanged;
	TSharedPtr<FUICommandList> CommandList;
};

//////////////////////////////////////////////////////////////////////////
// SFlipbookTimelineTrack

class SFlipbookTimelineTrack : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFlipbookTimelineTrack)
		: _SlateUnitsPerFrame(1)
		, _FlipbookBeingEdited(nullptr)
	{}

		SLATE_ATTRIBUTE( float, SlateUnitsPerFrame )
		SLATE_ATTRIBUTE( class UPaperFlipbook*, FlipbookBeingEdited )
		SLATE_EVENT( FOnFlipbookKeyframeSelectionChanged, OnSelectionChanged )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FUICommandList> InCommandList);

	void Rebuild();

private:
	TAttribute<float> SlateUnitsPerFrame;
	TAttribute< class UPaperFlipbook* > FlipbookBeingEdited;

	TSharedPtr<SHorizontalBox> MainBoxPtr;

	float HandleWidth;

	FOnFlipbookKeyframeSelectionChanged OnSelectionChanged;
	TSharedPtr<FUICommandList> CommandList;
};
