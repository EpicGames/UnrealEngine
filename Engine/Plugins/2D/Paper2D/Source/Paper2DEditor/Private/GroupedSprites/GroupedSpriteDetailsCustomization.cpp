// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroupedSprites/GroupedSpriteDetailsCustomization.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Editor.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"

#include "PaperGroupedSpriteComponent.h"
#include "GroupedSprites/PaperGroupedSpriteUtilities.h"
#include "ScopedTransaction.h"

#include "Engine/RendererSettings.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////
// FGroupedSpriteComponentDetailsCustomization

TSharedRef<IDetailCustomization> FGroupedSpriteComponentDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FGroupedSpriteComponentDetailsCustomization());
}

FGroupedSpriteComponentDetailsCustomization::FGroupedSpriteComponentDetailsCustomization()
{
}

void FGroupedSpriteComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Create a category so this is displayed early in the properties
	IDetailCategoryBuilder& SpriteCategory = DetailBuilder.EditCategory("Sprite", FText::GetEmpty(), ECategoryPriority::Important);

	ObjectsBeingCustomized.Empty();
	DetailBuilder.GetObjectsBeingCustomized(/*out*/ ObjectsBeingCustomized);


	TSharedRef<SWrapBox> ButtonBox = SNew(SWrapBox).UseAllottedSize(true);

	const float MinButtonSize = 100.0f;
	const FMargin ButtonPadding(0.0f, 2.0f, 2.0f, 0.0f);

	// Split button
	ButtonBox->AddSlot()
	.Padding(ButtonPadding)
	[
		SNew(SBox)
		.MinDesiredWidth(MinButtonSize)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("SplitSprites", "Split Sprites"))
			.ToolTipText(LOCTEXT("SplitSprites_Tooltip", "Splits all sprite instances into separate sprite actors or components"))
			.OnClicked(this, &FGroupedSpriteComponentDetailsCustomization::SplitSprites)
		]
	];

	// Sort button
	ButtonBox->AddSlot()
	.Padding(ButtonPadding)
	[
		SNew(SBox)
		.MinDesiredWidth(MinButtonSize)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("SortSprites", "Sort Sprites"))
			.ToolTipText(LOCTEXT("SortSprites_Tooltip", "Sorts all sprite instances according to the Translucency Sort Axis in the Rendering project settings"))
			.OnClicked(this, &FGroupedSpriteComponentDetailsCustomization::SortSprites)
		]
	];

	// Add the action buttons
	FDetailWidgetRow& GroupActionsRow = SpriteCategory.AddCustomRow(LOCTEXT("GroupActionsSearchText", "Split Sort"))
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				ButtonBox
			]
		];
}

FReply FGroupedSpriteComponentDetailsCustomization::SplitSprites()
{
	TArray<UObject*> StrongObjects;
	CopyFromWeakArray(StrongObjects, ObjectsBeingCustomized);

	FPaperGroupedSpriteUtilities::SplitSprites(StrongObjects);

	return FReply::Handled();
}

FReply FGroupedSpriteComponentDetailsCustomization::SortSprites()
{
	TArray<UObject*> StrongObjects;
	CopyFromWeakArray(StrongObjects, ObjectsBeingCustomized);

	TArray<UActorComponent*> ComponentList;
	TArray<AActor*> IgnoredList;
	FPaperGroupedSpriteUtilities::BuildHarvestList(StrongObjects, UPaperGroupedSpriteComponent::StaticClass(), /*out*/ ComponentList, /*out*/ IgnoredList);

	const FVector SortAxis = GetDefault<URendererSettings>()->TranslucentSortAxis;

	const FScopedTransaction Transaction(LOCTEXT("SortSpritesTransaction", "Sort instances in group"));
	for (UActorComponent* Component : ComponentList)
	{
		UPaperGroupedSpriteComponent* GroupedComponent = CastChecked<UPaperGroupedSpriteComponent>(Component);
		GroupedComponent->SortInstancesAlongAxis(SortAxis);
	}

	GEditor->RedrawLevelEditingViewports(true);

	return FReply::Handled();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
