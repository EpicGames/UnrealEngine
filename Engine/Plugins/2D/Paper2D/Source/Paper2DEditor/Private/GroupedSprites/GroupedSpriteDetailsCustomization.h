// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReply;

class IDetailLayoutBuilder;

//////////////////////////////////////////////////////////////////////////
// FGroupedSpriteComponentDetailsCustomization

class FGroupedSpriteComponentDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

protected:
	FGroupedSpriteComponentDetailsCustomization();

	FReply SplitSprites();
	FReply SortSprites();

protected:
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
};
