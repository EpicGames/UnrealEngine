// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroupedSprites/PaperGroupedSpriteUtilities.h"
#include "Paper2DEditorLog.h"
#include "Kismet/GameplayStatics.h"
#include "Editor.h"
#include "PaperGroupedSpriteActor.h"
#include "PaperSpriteActor.h"
#include "PaperSprite.h"
#include "PaperSpriteComponent.h"
#include "ScopedTransaction.h"
#include "Layers/LayersSubsystem.h"
#include "ComponentReregisterContext.h"
#include "PaperGroupedSpriteComponent.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////
// FPaperGroupedSpriteUtilities

void FPaperGroupedSpriteUtilities::BuildHarvestList(const TArray<UObject*>& ObjectsToConsider, TSubclassOf<UActorComponent> HarvestClassType, TArray<UActorComponent*>& OutComponentsToHarvest, TArray<AActor*>& OutActorsToDelete)
{
	// Determine the components to harvest
	for (UObject* Object : ObjectsToConsider)
	{
		if (Object == nullptr)
		{
			continue;
		}

		if (AActor* SelectedActor = Cast<AActor>(Object))
		{
			bool bHadHarvestableComponents = false;
			for (UActorComponent* Component : SelectedActor->GetComponents())
			{
				if (Component && Component->IsA(HarvestClassType) && !Component->IsEditorOnly())
				{
					bHadHarvestableComponents = true;
					OutComponentsToHarvest.Add(Component);
				}
			}

			if (bHadHarvestableComponents)
			{
				OutActorsToDelete.Add(SelectedActor);
			}
		}
		else if (UActorComponent* SelectedComponent = Cast<UActorComponent>(Object))
		{
			if (SelectedComponent->IsA(HarvestClassType) && !SelectedComponent->IsEditorOnly())
			{
				OutComponentsToHarvest.Add(SelectedComponent);
				if (AActor* SelectedComponentOwner = SelectedComponent->GetOwner())
				{
					OutActorsToDelete.AddUnique(SelectedComponentOwner);
				}
			}
		}
	}
}

FBox FPaperGroupedSpriteUtilities::ComputeBoundsForComponents(const TArray<UActorComponent*>& ComponentList)
{
	FBox BoundsOfList(ForceInit);

	for (const UActorComponent* Component : ComponentList)
	{
		if (const USceneComponent* SceneComponent = Cast<const USceneComponent>(Component))
		{
			BoundsOfList += SceneComponent->Bounds.GetBox();
		}
	}

	return BoundsOfList;
}

void FPaperGroupedSpriteUtilities::SplitSprites(const TArray<UObject*>& InObjectList)
{
	TArray<UActorComponent*> ComponentsToHarvest;
	TSubclassOf<UActorComponent> HarvestClassType = UPaperSpriteComponent::StaticClass();
	TArray<AActor*> ActorsToDelete;
	TArray<AActor*> ActorsCreated;

	FPaperGroupedSpriteUtilities::BuildHarvestList(InObjectList, UPaperGroupedSpriteComponent::StaticClass(), /*out*/ ComponentsToHarvest, /*out*/ ActorsToDelete);

	if (ComponentsToHarvest.Num() > 0)
	{
		if (UWorld* World = ComponentsToHarvest[0]->GetWorld())
		{
			const FScopedTransaction Transaction(LOCTEXT("SplitSpritesTransation", "Split sprite instances"));

			// Create an instance from each item of each batch component that we're harvesting
			for (UActorComponent* SourceComponent : ComponentsToHarvest)
			{
				UPaperGroupedSpriteComponent* SourceBatchComponent = CastChecked<UPaperGroupedSpriteComponent>(SourceComponent);

				for (const FSpriteInstanceData& InstanceData : SourceBatchComponent->GetPerInstanceSpriteData())
				{
					if (InstanceData.SourceSprite != nullptr)
					{
						const FTransform InstanceTransform(FTransform(InstanceData.Transform) * SourceBatchComponent->GetComponentTransform());

						FActorSpawnParameters SpawnParams;
						SpawnParams.bDeferConstruction = true;
						if (APaperSpriteActor* SpawnedActor = World->SpawnActor<APaperSpriteActor>(SpawnParams))
						{
							UPaperSpriteComponent* SpawnedSpriteComponent = SpawnedActor->GetRenderComponent();

							{
								FComponentReregisterContext ReregisterContext(SpawnedSpriteComponent);

								SpawnedSpriteComponent->Modify();
								SpawnedSpriteComponent->SetSpriteColor(InstanceData.VertexColor.ReinterpretAsLinear());
								SpawnedSpriteComponent->SetSprite(InstanceData.SourceSprite);

								// Apply the material override if there is one
								UMaterialInterface* InstanceMaterial = SourceBatchComponent->GetMaterial(InstanceData.MaterialIndex);
								if (InstanceMaterial != InstanceData.SourceSprite->GetMaterial(0))
								{
									SpawnedSpriteComponent->SetMaterial(0, InstanceMaterial);
								}
							}

							UGameplayStatics::FinishSpawningActor(SpawnedActor, InstanceTransform);
							ActorsCreated.Add(SpawnedActor);

							// Give it a good name
							FActorLabelUtilities::SetActorLabelUnique(SpawnedActor, InstanceData.SourceSprite->GetName());
						}
					}
				}
			}

			// Delete the existing actor instances
			ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
			for (AActor* ActorToDelete : ActorsToDelete)
			{
				// Remove from active selection in editor
				GEditor->SelectActor(ActorToDelete, /*bSelected=*/ false, /*bNotify=*/ false);
				Layers->DisassociateActorFromLayers(ActorToDelete);
				World->EditorDestroyActor(ActorToDelete, /*bShouldModifyLevel=*/ true);
			}

			// Select the new replacements instance
			GEditor->SelectNone(/*bNoteSelectionChange=*/ false, false, false);
			for (AActor* SpawnedActor : ActorsCreated)
			{
				GEditor->SelectActor(SpawnedActor, /*bSelected=*/ true, /*bNotify=*/ true);
			}
			GEditor->NoteSelectionChange();
		}
		else
		{
			// We're in the BP editor and don't currently support splitting here!
			UE_LOGF(LogPaper2DEditor, Warning, "Splitting sprites in the Blueprint editor is not currently supported");
		}
	}

	GEditor->RedrawLevelEditingViewports(true);
}

void FPaperGroupedSpriteUtilities::MergeSprites(const TArray<UObject*>& InObjectList)
{
	TArray<UActorComponent*> ComponentsToHarvest;
	TSubclassOf<UActorComponent> HarvestClassType = UPaperSpriteComponent::StaticClass();
	TArray<AActor*> ActorsToDelete;

	FPaperGroupedSpriteUtilities::BuildHarvestList(InObjectList, UPaperSpriteComponent::StaticClass(), /*out*/ ComponentsToHarvest, /*out*/ ActorsToDelete);

	if (ComponentsToHarvest.Num() > 0)
	{
		if (UWorld* World = ComponentsToHarvest[0]->GetWorld())
		{
			const FBox ComponentBounds = ComputeBoundsForComponents(ComponentsToHarvest);
			const FTransform MergedWorldTM(ComponentBounds.GetCenter());

			FActorSpawnParameters SpawnParams;
			SpawnParams.bDeferConstruction = true;

			const FScopedTransaction Transaction(LOCTEXT("MergeSpritesTransaction", "Merge sprite instances"));

			if (APaperGroupedSpriteActor* SpawnedActor = World->SpawnActor<APaperGroupedSpriteActor>(SpawnParams))
			{
				SpawnedActor->Modify();
				// Create a merged sprite component
				{
					UPaperGroupedSpriteComponent* MergedSpriteComponent = SpawnedActor->GetRenderComponent();
					MergedSpriteComponent->Modify();
					FComponentReregisterContext ReregisterContext(MergedSpriteComponent);

					// Create an instance from each sprite component that we're harvesting
					for (UActorComponent* SourceComponent : ComponentsToHarvest)
					{
						UPaperSpriteComponent* SourceSpriteComponent = CastChecked<UPaperSpriteComponent>(SourceComponent);

						UPaperSprite* Sprite = SourceSpriteComponent->GetSprite();
						const FLinearColor SpriteColor = SourceSpriteComponent->GetSpriteColor();
						const FTransform RelativeSpriteTransform = SourceSpriteComponent->GetComponentTransform().GetRelativeTransform(MergedWorldTM);

						UMaterialInterface* OverrideMaterial = SourceSpriteComponent->GetMaterial(0);
						if ((Sprite == nullptr) || (OverrideMaterial == Sprite->GetMaterial(0)))
						{
							OverrideMaterial = nullptr;
						}

						MergedSpriteComponent->AddInstanceWithMaterial(RelativeSpriteTransform, Sprite, OverrideMaterial, /*bWorldSpace=*/ false, SpriteColor);
					}
				}

				// Finalize the new component
				UGameplayStatics::FinishSpawningActor(SpawnedActor, MergedWorldTM);

				// Delete the existing actor instances
				ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
				for (AActor* ActorToDelete : ActorsToDelete)
				{
					// Remove from active selection in editor
					GEditor->SelectActor(ActorToDelete, /*bSelected=*/ false, /*bNotify=*/ false);
					Layers->DisassociateActorFromLayers(ActorToDelete);
					World->EditorDestroyActor(ActorToDelete, /*bShouldModifyLevel=*/ true);
				}

				// Select the new replacement instance
				GEditor->SelectActor(SpawnedActor, /*bSelected=*/ true, /*bNotify=*/ true);
			}
			else
			{
				// We're in the BP editor and don't currently support merging here!
				UE_LOGF(LogPaper2DEditor, Warning, "Merging sprites in the Blueprint editor is not currently supported");
			}
		}
	}

	GEditor->RedrawLevelEditingViewports(true);
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
