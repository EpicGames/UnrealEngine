// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"

class UActorComponent;
template <class TClass> class TSubclassOf;

class AActor;

class FPaperGroupedSpriteUtilities
{
public:
	static void BuildHarvestList(const TArray<UObject*>& ObjectsToConsider, TSubclassOf<UActorComponent> HarvestClassType, TArray<UActorComponent*>& OutComponentsToHarvest, TArray<AActor*>& OutActorsToDelete);

	// Computes the enclosing bounding box of the specified components (using their individual bounds)
	static FBox ComputeBoundsForComponents(const TArray<UActorComponent*>& ComponentList);

	static void SplitSprites(const TArray<UObject*>& InObjectList);

	static void MergeSprites(const TArray<UObject*>& InObjectList);
};
