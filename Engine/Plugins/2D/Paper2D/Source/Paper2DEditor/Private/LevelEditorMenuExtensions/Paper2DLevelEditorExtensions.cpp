// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorMenuExtensions/Paper2DLevelEditorExtensions.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"

#include "PaperSpriteComponent.h"
#include "PaperGroupedSpriteComponent.h"
#include "GroupedSprites/PaperGroupedSpriteUtilities.h"
#include "Selection.h"

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////

FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors LevelEditorMenuExtenderDelegate;
FDelegateHandle LevelEditorExtenderDelegateHandle;

//////////////////////////////////////////////////////////////////////////
// FPaperLevelEditorMenuExtensions_Impl

class FPaperLevelEditorMenuExtensions_Impl
{
public:
	static void MergeSprites()
	{
		// Create an array of actors to consider
		TArray<UObject*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects(AActor::StaticClass(), /*out*/ SelectedActors);

		// Merge them
		FPaperGroupedSpriteUtilities::MergeSprites(SelectedActors);
	}

	static void SplitSprites()
	{
		// Create an array of actors to consider
		TArray<UObject*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects(AActor::StaticClass(), /*out*/ SelectedActors);

		// Split them
		FPaperGroupedSpriteUtilities::SplitSprites(SelectedActors);
	}

	static void CreateSpriteActionsMenuEntries(FMenuBuilder& MenuBuilder, bool bCanMerge, bool bCanSplit)
	{
		MenuBuilder.BeginSection("Paper2D", LOCTEXT("Paper2DLevelEditorHeading", "Paper2D"));

		if (bCanMerge)
		{
			FUIAction Action_MergeSprites(FExecuteAction::CreateStatic(&FPaperLevelEditorMenuExtensions_Impl::MergeSprites));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("MenuExtensionMergeSprites", "Merge Sprites"),
				LOCTEXT("MenuExtensionMergeSprites_Tooltip", "Replaces all of the selected actors that contain a sprite component with a single grouped sprite component"),
				FSlateIcon(),
				Action_MergeSprites,
				NAME_None,
				EUserInterfaceActionType::Button);
		}

		if (bCanSplit)
		{
			FUIAction Action_SplitSprites(FExecuteAction::CreateStatic(&FPaperLevelEditorMenuExtensions_Impl::SplitSprites));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("MenuExtensionSplitSprites", "Split Sprites"),
				LOCTEXT("MenuExtensionSplitSprites_Tooltip", "Replaces all of the selected actors that contain a grouped sprite component with many individual sprite components, one per element"),
				FSlateIcon(),
				Action_SplitSprites,
				NAME_None,
				EUserInterfaceActionType::Button);
		}

		MenuBuilder.EndSection();
	}

	static TSharedRef<FExtender> OnExtendLevelEditorMenu(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors)
	{
		TSharedRef<FExtender> Extender(new FExtender());

		// Run thru the actors to determine if any meet our criteria
		bool bCanMergeSprites = false;
		bool bCanSplitSprites = false;

		for (AActor* Actor : SelectedActors)
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (Component)
				{
					if (Component->IsA(UPaperSpriteComponent::StaticClass()))
					{
						bCanMergeSprites = true;
					}
					else if (Component->IsA(UPaperGroupedSpriteComponent::StaticClass()))
					{
						bCanSplitSprites = true;
					}
				}
			}

		}

		bCanMergeSprites = bCanMergeSprites && (SelectedActors.Num() > 1);

		if (bCanMergeSprites || bCanSplitSprites)
		{
			// Add the sprite actions sub-menu extender
			Extender->AddMenuExtension(
				"ActorTypeTools",
				EExtensionHook::After,
				nullptr,
				FMenuExtensionDelegate::CreateStatic(&FPaperLevelEditorMenuExtensions_Impl::CreateSpriteActionsMenuEntries, /*bCanMerge=*/ bCanMergeSprites, /*bCanSplit=*/ bCanSplitSprites));
		}

		return Extender;
	}
};

//////////////////////////////////////////////////////////////////////////
// FPaperLevelEditorMenuExtensions

void FPaperLevelEditorMenuExtensions::InstallHooks()
{
	LevelEditorMenuExtenderDelegate = FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateStatic(&FPaperLevelEditorMenuExtensions_Impl::OnExtendLevelEditorMenu);

	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	auto& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
	MenuExtenders.Add(LevelEditorMenuExtenderDelegate);
	LevelEditorExtenderDelegateHandle = MenuExtenders.Last().GetHandle();
}

void FPaperLevelEditorMenuExtensions::RemoveHooks()
{
	// Remove level viewport context menu extenders
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetAllLevelViewportContextMenuExtenders().RemoveAll([&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate) {
			return Delegate.GetHandle() == LevelEditorExtenderDelegateHandle;
		});
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
