// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// Integrate Paper2D actions into the level editor context menu
class FPaperLevelEditorMenuExtensions
{
public:
	static void InstallHooks();
	static void RemoveHooks();
};
