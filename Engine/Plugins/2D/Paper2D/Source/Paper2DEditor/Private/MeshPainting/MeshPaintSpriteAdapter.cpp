// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPainting/MeshPaintSpriteAdapter.h"
#include "Paper2DModule.h"
#include "PaperSpriteComponent.h"
#include "PaperSprite.h"
#include "MeshPaintTypes.h"

//////////////////////////////////////////////////////////////////////////
// FMeshPaintSpriteAdapter

bool FMeshPaintSpriteAdapter::Construct(UMeshComponent* InComponent, int32 InMeshLODIndex)
{
	SpriteComponent = CastChecked<UPaperSpriteComponent>(InComponent);
	return SpriteComponent->GetSprite() != nullptr && Initialize();
}

bool FMeshPaintSpriteAdapter::Initialize()
{
	Sprite = SpriteComponent->GetSprite();
	checkSlow(Sprite != nullptr);
	const int32 NumTriangles = Sprite->BakedRenderData.Num() / 3;
	const int32 NumIndices = NumTriangles * 3;
	const int32 NumVertices = Sprite->BakedRenderData.Num();

	const TArray<FVector4>& BakedPoints = Sprite->BakedRenderData;
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		const FVector4& XYUV = BakedPoints[VertexIndex];
		MeshVertices.Add(FVector(PaperAxisX * XYUV.X) + (PaperAxisY * XYUV.Y));
	}

	for (int32 Index = 0; Index < NumIndices; ++Index)
	{
		MeshIndices.Add(Index);
	}

	return MeshVertices.Num() > 0 && MeshIndices.Num() > 0;
}

bool FMeshPaintSpriteAdapter::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const
{
	const FTransform& ComponentToWorld = SpriteComponent->GetComponentTransform();

	// Can we possibly intersect with the sprite?
	const FBoxSphereBounds& Bounds = SpriteComponent->Bounds;
	if (FMath::PointDistToSegment(Bounds.Origin, Start, End) <= Bounds.SphereRadius)
	{
		const FVector LocalStart = ComponentToWorld.InverseTransformPosition(Start);
		const FVector LocalEnd = ComponentToWorld.InverseTransformPosition(End);

		FPlane LocalSpacePlane(FVector::ZeroVector, PaperAxisX, PaperAxisY);

		FVector Intersection;
		if (FMath::SegmentPlaneIntersection(LocalStart, LocalEnd, LocalSpacePlane, /*out*/ Intersection))
		{
			const float LocalX = FVector::DotProduct(Intersection, PaperAxisX);
			const float LocalY = FVector::DotProduct(Intersection, PaperAxisY);
			const FVector LocalPoint(LocalX, LocalY, 0.0f);

			const TArray<FVector4>& BakedPoints = Sprite->BakedRenderData;
			check((BakedPoints.Num() % 3) == 0);

			for (int32 VertexIndex = 0; VertexIndex < BakedPoints.Num(); VertexIndex += 3)
			{
				const FVector& A = BakedPoints[VertexIndex + 0];
				const FVector& B = BakedPoints[VertexIndex + 1];
				const FVector& C = BakedPoints[VertexIndex + 2];
				const FVector Q = FMath::GetBaryCentric2D(LocalPoint, A, B, C);

				if ((Q.X >= 0.0f) && (Q.Y >= 0.0f) && (Q.Z >= 0.0f) && FMath::IsNearlyEqual(Q.X + Q.Y + Q.Z, 1.0f))
				{
					const FVector WorldIntersection = ComponentToWorld.TransformPosition(Intersection);

					const FVector WorldNormalFront = ComponentToWorld.TransformVectorNoScale(PaperAxisZ);
					const FVector WorldNormal = (LocalSpacePlane.PlaneDot(LocalStart) >= 0.0f) ? WorldNormalFront : -WorldNormalFront;

					OutHit.bBlockingHit = true;
					OutHit.Time = (WorldIntersection - Start).Size() / (End - Start).Size();
					OutHit.Location = WorldIntersection;
					OutHit.Normal = WorldNormal;
					OutHit.ImpactPoint = WorldIntersection;
					OutHit.ImpactNormal = WorldNormal;
					OutHit.TraceStart = Start;
					OutHit.TraceEnd = End;
					OutHit.HitObjectHandle = FActorInstanceHandle(SpriteComponent->GetOwner());
					OutHit.Component = SpriteComponent;
					OutHit.FaceIndex = VertexIndex / 3;

					return true;
				}
			}
		}
	}

	return false;
}

TArray<uint32> FMeshPaintSpriteAdapter::SphereIntersectTriangles(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const
{
	//@TODO: MESHPAINT: This isn't very precise..., but since the sprite is planar it shouldn't cause any actual issues with paint UX other than being suboptimal
	const uint32 NumTriangles = Sprite->BakedRenderData.Num() / 3;
	TArray<uint32> OutTriangles;
	OutTriangles.Reserve(OutTriangles.Num() + NumTriangles);
	for (uint32 TriIndex = 0; TriIndex < NumTriangles; ++TriIndex)
	{
		OutTriangles.Add(TriIndex);
	}

	return OutTriangles;
}

void FMeshPaintSpriteAdapter::QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	// Grab the sprite texture first
	int32 ForceIndex = INDEX_NONE;
	if (UTexture2D* SourceTexture = Sprite->GetBakedTexture())
	{
		FPaintableTexture PaintableTexture(SourceTexture, 0);
		ForceIndex = InOutTextureList.AddUnique(PaintableTexture);
	}

	// Grab the additional textures next
	FAdditionalSpriteTextureArray AdditionalTextureList;
	Sprite->GetBakedAdditionalSourceTextures(/*out*/ AdditionalTextureList);
	for (UTexture* AdditionalTexture : AdditionalTextureList)
	{
		if (AdditionalTexture != nullptr)
		{
			FPaintableTexture PaintableTexture(AdditionalTexture, 0);
			InOutTextureList.AddUnique(AdditionalTexture);
		}
	}

	// Now ask the material
	DefaultQueryPaintableTextures(MaterialIndex, SpriteComponent, OutDefaultIndex, InOutTextureList);

	if (ForceIndex != INDEX_NONE)
	{
		OutDefaultIndex = ForceIndex;
	}
}

void FMeshPaintSpriteAdapter::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const
{
	// Apply it to the sprite component
	SpriteComponent->SetTransientTextureOverride(SourceTexture, OverrideTexture);

	// Make sure we swap it out on any textures that aren't part of the sprite as well
	DefaultApplyOrRemoveTextureOverride(SpriteComponent, SourceTexture, OverrideTexture);
}

void FMeshPaintSpriteAdapter::PreEdit()
{
}

void FMeshPaintSpriteAdapter::PostEdit()
{
}

void FMeshPaintSpriteAdapter::GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance /*= true*/) const
{
	// Do nothing
	OutColor = FColor::White;
}

void FMeshPaintSpriteAdapter::SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance /*= true*/)
{
	// Do nothing
}

void FMeshPaintSpriteAdapter::GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const
{
	OutTextureCoordinate.X = Sprite->BakedRenderData[VertexIndex].Z;
	OutTextureCoordinate.Y = Sprite->BakedRenderData[VertexIndex].W;	
}

void FMeshPaintSpriteAdapter::GetVertexPosition(int32 VertexIndex, FVector& OutVertex) const
{
	OutVertex = MeshVertices[VertexIndex];
}

FMatrix FMeshPaintSpriteAdapter::GetComponentToWorldMatrix() const
{
	return SpriteComponent->GetComponentToWorld().ToMatrixWithScale();
}

void FMeshPaintSpriteAdapter::GetInfluencedVertexIndices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TSet<int32> &InfluencedVertices) const
{
	// Get a list of (optionally front-facing) triangles that are within a reasonable distance to the brush
	TArray<uint32> InfluencedTriangles = SphereIntersectTriangles(
		ComponentSpaceSquaredBrushRadius,
		ComponentSpaceBrushPosition,
		ComponentSpaceCameraPosition,
		bOnlyFrontFacing);

	// Make sure we're dealing with triangle lists
	const int32 NumIndexBufferIndices = MeshIndices.Num();
	check(NumIndexBufferIndices % 3 == 0);

	InfluencedVertices.Reserve(InfluencedTriangles.Num());
	for (int32 InfluencedTriangle : InfluencedTriangles)
	{
		InfluencedVertices.Add(MeshIndices[InfluencedTriangle * 3 + 0]);
		InfluencedVertices.Add(MeshIndices[InfluencedTriangle * 3 + 1]);
		InfluencedVertices.Add(MeshIndices[InfluencedTriangle * 3 + 2]);
	}
}

TArray<FVector> FMeshPaintSpriteAdapter::SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const
{
	const TArray<uint32> IntersectedTriangles = SphereIntersectTriangles(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, bOnlyFrontFacing);

	// Get a list of unique vertices indexed by the influenced triangles
	TSet<int32> InfluencedVertices;
	InfluencedVertices.Reserve(IntersectedTriangles.Num());
	for (int32 IntersectedTriangle : IntersectedTriangles)
	{
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 0]);
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 1]);
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 2]);
	}

	TArray<FVector> InRangeVertices;
	InRangeVertices.Empty(MeshVertices.Num());
	for (int32 VertexIndex : InfluencedVertices)
	{
		const FVector& Vertex = MeshVertices[VertexIndex];
		if (FVector::DistSquared(ComponentSpaceBrushPosition, Vertex) <= ComponentSpaceSquaredBrushRadius)
		{
			InRangeVertices.Add(Vertex);
		}
	}

	return InRangeVertices;
}

void FMeshPaintSpriteAdapter::GetInfluencedVertexData(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TArray<TPair<int32, FVector>>& OutData) const
{
	// Get a list of (optionally front-facing) triangles that are within a reasonable distance to the brush
	TArray<uint32> InfluencedTriangles = SphereIntersectTriangles(
		ComponentSpaceSquaredBrushRadius,
		ComponentSpaceBrushPosition,
		ComponentSpaceCameraPosition,
		bOnlyFrontFacing);

	// Make sure we're dealing with triangle lists
	const int32 NumIndexBufferIndices = MeshIndices.Num();
	check(NumIndexBufferIndices % 3 == 0);

	OutData.Reserve(InfluencedTriangles.Num() * 3);
	for(int32 InfluencedTriangle : InfluencedTriangles)
	{
		for(int32 Index = 0; Index < 3; ++Index)
		{
			OutData.AddDefaulted();
			TPair<int32, FVector>& OutPair = OutData.Last();
			OutPair.Key = MeshIndices[InfluencedTriangle * 3 + Index];
			OutPair.Value = MeshVertices[OutPair.Key];
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FMeshPaintSpriteAdapterFactory

TSharedPtr<IMeshPaintGeometryAdapter> FMeshPaintSpriteAdapterFactory::Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const
{
	if (UPaperSpriteComponent* SpriteComponent = Cast<UPaperSpriteComponent>(InComponent))
	{
		if (SpriteComponent->GetSprite() != nullptr)
		{
			TSharedRef<FMeshPaintSpriteAdapter> Result = MakeShareable(new FMeshPaintSpriteAdapter());
			if (Result->Construct(InComponent, InMeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
