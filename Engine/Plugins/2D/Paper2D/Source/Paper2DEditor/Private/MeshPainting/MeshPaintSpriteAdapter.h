// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMeshPaintGeometryAdapter.h"
#include "IMeshPaintGeometryAdapterFactory.h"

class FReferenceCollector;
class UMeshComponent;
class UPaperSpriteComponent;
class UTexture;
class UPaperSprite;

//////////////////////////////////////////////////////////////////////////
// FMeshPaintSpriteAdapter

class FMeshPaintSpriteAdapter : public IMeshPaintGeometryAdapter
{
public:
	static void InitializeAdapterGlobals() {}
	static void AddReferencedObjectsGlobals(FReferenceCollector& Collector) {}
	static void CleanupGlobals() {}

	virtual bool Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) override;
	virtual bool Initialize() override;
	virtual void OnAdded() override {}
	virtual void OnRemoved() override {}
	virtual bool IsValid() const override { return true; }	
	virtual bool SupportsTexturePaint() const override { return true; }
	virtual bool SupportsVertexPaint() const override { return false; }
	virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const override;
	virtual TArray<uint32> SphereIntersectTriangles(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const override;
	virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) override;
	virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {}

	virtual void PreEdit() override;
	virtual void PostEdit() override;
		
	virtual const TArray<FVector>& GetMeshVertices() const override { return MeshVertices; }
	virtual const TArray<uint32>& GetMeshIndices() const override { return MeshIndices; }
	virtual void GetVertexPosition(int32 VertexIndex, FVector& OutVertex) const override;
	virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const override;
	virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const override;
	virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) override;
	virtual FMatrix GetComponentToWorldMatrix() const override;
	
	virtual void GetInfluencedVertexIndices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TSet<int32> &InfluencedVertices) const override;
	virtual void GetInfluencedVertexData(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TArray<TPair<int32, FVector>>& OutData) const override;
	
	virtual TArray<FVector> SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const override;



protected:
	UPaperSpriteComponent* SpriteComponent;
	UPaperSprite* Sprite;
	TArray<FVector> MeshVertices;
	TArray<uint32> MeshIndices;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintSpriteAdapterFactory

class FMeshPaintSpriteAdapterFactory : public IMeshPaintGeometryAdapterFactory
{
public:
	virtual TSharedPtr<IMeshPaintGeometryAdapter> Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) const override;
	virtual void InitializeAdapterGlobals() override { FMeshPaintSpriteAdapter::InitializeAdapterGlobals(); }
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) override { FMeshPaintSpriteAdapter::AddReferencedObjectsGlobals(Collector); }
	virtual void CleanupGlobals() override { FMeshPaintSpriteAdapter::CleanupGlobals(); }
};
