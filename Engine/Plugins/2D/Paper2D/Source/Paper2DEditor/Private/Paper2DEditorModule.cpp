// Copyright Epic Games, Inc. All Rights Reserved.

#include "Paper2DEditorModule.h"
#include "Paper2DEditorLog.h"
#include "EditorModeRegistry.h"

#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Subsystems/ImportSubsystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectIterator.h"

#include "PropertyEditorModule.h"
#include "PaperEditorCommands.h"
#include "PaperEditorShared/SpriteGeometryEditMode.h"
#include "PaperTileSet.h"

#include "ContentBrowserExtensions/ContentBrowserExtensions.h"
#include "LevelEditorMenuExtensions/Paper2DLevelEditorExtensions.h"
#include "PaperImporterSettings.h"

// Sprite support
#include "SpriteAssetTypeActions.h"
#include "PaperSpriteAssetBroker.h"
#include "SpriteEditor/SpriteDetailsCustomization.h"
#include "SpriteEditor/SpriteComponentDetailsCustomization.h"
#include "SpriteEditor/SpriteEditorSettings.h"

// Flipbook support
#include "FlipbookAssetTypeActions.h"
#include "PaperFlipbookAssetBroker.h"
#include "PaperFlipbookThumbnailRenderer.h"
#include "FlipbookEditor/FlipbookComponentDetailsCustomization.h"
#include "FlipbookEditor/FlipbookEditorSettings.h"

// Tile set support
#include "TileSetAssetTypeActions.h"
#include "PaperTileSetThumbnailRenderer.h"
#include "TileSetEditor/TileSetEditorSettings.h"
#include "TileSetEditor/TileSetDetailsCustomization.h"

// Tile map support
#include "TileMapEditing/TileMapAssetTypeActions.h"
#include "TileMapEditing/PaperTileMapAssetBroker.h"
#include "TileMapEditing/EdModeTileMap.h"
#include "TileMapEditing/PaperTileMapDetailsCustomization.h"
#include "TileMapEditing/TileMapEditorSettings.h"

// Atlas support
#include "Atlasing/AtlasAssetTypeActions.h"
#include "Atlasing/PaperAtlasGenerator.h"
#include "PaperSpriteAtlas.h"

// Grouped sprite support
#include "PaperGroupedSpriteComponent.h"
#include "GroupedSprites/GroupedSpriteDetailsCustomization.h"

// Settings
#include "PaperRuntimeSettings.h"
#include "ISettingsModule.h"

// Mesh paint adapters
#include "MeshPaintModule.h"
#include "MeshPainting/MeshPaintSpriteAdapter.h"

DEFINE_LOG_CATEGORY(LogPaper2DEditor);

#define LOCTEXT_NAMESPACE "Paper2DEditor"

//////////////////////////////////////////////////////////////////////////
// FPaper2DEditor

class FPaper2DEditor : public IPaper2DEditorModule
{
public:
	FPaper2DEditor()
		: Paper2DAssetCategoryBit(EAssetTypeCategories::Misc)
	{
	}

	// IPaper2DEditorModule interface
	virtual TSharedPtr<FExtensibilityManager> GetSpriteEditorMenuExtensibilityManager() override { return SpriteEditor_MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetSpriteEditorToolBarExtensibilityManager() override { return SpriteEditor_ToolBarExtensibilityManager; }

	virtual TSharedPtr<FExtensibilityManager> GetFlipbookEditorMenuExtensibilityManager() override { return FlipbookEditor_MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetFlipbookEditorToolBarExtensibilityManager() override { return FlipbookEditor_ToolBarExtensibilityManager; }
	virtual uint32 GetPaper2DAssetCategory() const override { return Paper2DAssetCategoryBit; }
	// End of IPaper2DEditorModule

private:
	TSharedPtr<FExtensibilityManager> SpriteEditor_MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> SpriteEditor_ToolBarExtensibilityManager;

	TSharedPtr<FExtensibilityManager> FlipbookEditor_MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> FlipbookEditor_ToolBarExtensibilityManager;

	/** All created asset type actions.  Cached here so that we can unregister them during shutdown. */
	TArray< TSharedPtr<IAssetTypeActions> > CreatedAssetTypeActions;

	TSharedPtr<IComponentAssetBroker> PaperSpriteBroker;
	TSharedPtr<IComponentAssetBroker> PaperFlipbookBroker;
	TSharedPtr<IComponentAssetBroker> PaperTileMapBroker;

	TSharedPtr<IMeshPaintGeometryAdapterFactory> SpriteMeshPaintAdapterFactory;
	FDelegateHandle OnPropertyChangedDelegateHandle;
	FDelegateHandle OnAssetReimportDelegateHandle;

	EAssetTypeCategories::Type Paper2DAssetCategoryBit;

public:
	virtual void StartupModule() override
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FPaper2DEditor::OnPostEngineInit);
	}

	void OnPostEngineInit()
	{
		SpriteEditor_MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		SpriteEditor_ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		FlipbookEditor_MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		FlipbookEditor_ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		// Register slate style overrides
		FPaperStyle::Initialize();

		// Register commands
		FPaperEditorCommands::Register();

		// Register asset types
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		Paper2DAssetCategoryBit = AssetTools.RegisterAdvancedAssetCategory(FName(TEXT("Paper2D")), LOCTEXT("Paper2DAssetCategory", "Paper2D"));

		RegisterAssetTypeAction(AssetTools, MakeShareable(new FSpriteAssetTypeActions(Paper2DAssetCategoryBit)));
		RegisterAssetTypeAction(AssetTools, MakeShareable(new FFlipbookAssetTypeActions(Paper2DAssetCategoryBit)));
		RegisterAssetTypeAction(AssetTools, MakeShareable(new FTileSetAssetTypeActions(Paper2DAssetCategoryBit)));
		RegisterAssetTypeAction(AssetTools, MakeShareable(new FTileMapAssetTypeActions(Paper2DAssetCategoryBit)));
		RegisterAssetTypeAction(AssetTools, MakeShareable(new FAtlasAssetTypeActions(Paper2DAssetCategoryBit)));

		PaperSpriteBroker = MakeShareable(new FPaperSpriteAssetBroker);
		FComponentAssetBrokerage::RegisterBroker(PaperSpriteBroker, UPaperSpriteComponent::StaticClass(), true, true);

		PaperFlipbookBroker = MakeShareable(new FPaperFlipbookAssetBroker);
		FComponentAssetBrokerage::RegisterBroker(PaperFlipbookBroker, UPaperFlipbookComponent::StaticClass(), true, true);

		PaperTileMapBroker = MakeShareable(new FPaperTileMapAssetBroker);
		FComponentAssetBrokerage::RegisterBroker(PaperTileMapBroker, UPaperTileMapComponent::StaticClass(), true, true);

		// Register the details customizations
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomClassLayout(UPaperTileMapComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPaperTileMapDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UPaperTileMap::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPaperTileMapDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UPaperTileSet::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FTileSetDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UPaperSprite::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FSpriteDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UPaperSpriteComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FSpriteComponentDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UPaperFlipbookComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FFlipbookComponentDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UPaperGroupedSpriteComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGroupedSpriteComponentDetailsCustomization::MakeInstance));

			//@TODO: Struct registration should happen using ::StaticStruct, not by string!!!
			//PropertyModule.RegisterCustomPropertyTypeLayout( "SpritePolygonCollection", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FSpritePolygonCollectionCustomization::MakeInstance ) );

			PropertyModule.NotifyCustomizationModuleChanged();
		}

		// Register to be notified when properties are edited
		OnPropertyChangedDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FPaper2DEditor::OnPropertyChanged);

		// Register to be notified when an asset is reimported
		if (GEditor)
		{
			OnAssetReimportDelegateHandle = GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddRaw(this, &FPaper2DEditor::OnObjectReimported);
		}

		// Register the thumbnail renderers
		UThumbnailManager::Get().RegisterCustomRenderer(UPaperSprite::StaticClass(), UPaperSpriteThumbnailRenderer::StaticClass());
		UThumbnailManager::Get().RegisterCustomRenderer(UPaperTileSet::StaticClass(), UPaperTileSetThumbnailRenderer::StaticClass());
		UThumbnailManager::Get().RegisterCustomRenderer(UPaperFlipbook::StaticClass(), UPaperFlipbookThumbnailRenderer::StaticClass());
		//@TODO: PAPER2D: UThumbnailManager::Get().RegisterCustomRenderer(UPaperTileMap::StaticClass(), UPaperTileMapThumbnailRenderer::StaticClass());

		// Register the editor modes
		FEditorModeRegistry::Get().RegisterMode<FEdModeTileMap>(
			FEdModeTileMap::EM_TileMap,
			LOCTEXT("TileMapEditMode", "Tile Map Editor"),
			FSlateIcon(),
			false);

		FEditorModeRegistry::Get().RegisterMode<FSpriteGeometryEditMode>(
			FSpriteGeometryEditMode::EM_SpriteGeometry,
			LOCTEXT("SpriteGeometryEditMode", "Sprite Geometry Editor"),
			FSlateIcon(),
			false);

		// Integrate Paper2D actions into existing editor context menus
		if (!IsRunningCommandlet())
		{
			FPaperContentBrowserExtensions::InstallHooks();
			FPaperLevelEditorMenuExtensions::InstallHooks();
		}
		// Register with the mesh paint module
		if (IMeshPaintModule* MeshPaintModule = FModuleManager::LoadModulePtr<IMeshPaintModule>("MeshPaint"))
		{
			SpriteMeshPaintAdapterFactory = MakeShareable(new FMeshPaintSpriteAdapterFactory());
			MeshPaintModule->RegisterGeometryAdapterFactory(SpriteMeshPaintAdapterFactory.ToSharedRef());
		}

		RegisterSettings();
	}

	virtual void ShutdownModule() override
	{
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

		SpriteEditor_MenuExtensibilityManager.Reset();
		SpriteEditor_ToolBarExtensibilityManager.Reset();

		FlipbookEditor_MenuExtensibilityManager.Reset();
		FlipbookEditor_ToolBarExtensibilityManager.Reset();

		if (UObjectInitialized())
		{
			UnregisterSettings();

			// Unregister from the mesh paint module
			if (IMeshPaintModule* MeshPaintModule = FModuleManager::GetModulePtr<IMeshPaintModule>("MeshPaint"))
			{
				MeshPaintModule->UnregisterGeometryAdapterFactory(SpriteMeshPaintAdapterFactory.ToSharedRef());
				SpriteMeshPaintAdapterFactory.Reset();
			}

			FPaperLevelEditorMenuExtensions::RemoveHooks();
			FPaperContentBrowserExtensions::RemoveHooks();

			FComponentAssetBrokerage::UnregisterBroker(PaperTileMapBroker);
			FComponentAssetBrokerage::UnregisterBroker(PaperFlipbookBroker);
			FComponentAssetBrokerage::UnregisterBroker(PaperSpriteBroker);

			// Unregister the editor modes
			FEditorModeRegistry::Get().UnregisterMode(FSpriteGeometryEditMode::EM_SpriteGeometry);
			FEditorModeRegistry::Get().UnregisterMode(FEdModeTileMap::EM_TileMap);

			// Unregister the thumbnail renderers
			UThumbnailManager::Get().UnregisterCustomRenderer(UPaperSprite::StaticClass());
			UThumbnailManager::Get().UnregisterCustomRenderer(UPaperTileMap::StaticClass());
			UThumbnailManager::Get().UnregisterCustomRenderer(UPaperTileSet::StaticClass());
			UThumbnailManager::Get().UnregisterCustomRenderer(UPaperFlipbook::StaticClass());

			// Unregister the property modification handler
			FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnPropertyChangedDelegateHandle);

			// Unregister the asset reimport handler
			if (GEditor)
			{
				GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.Remove(OnAssetReimportDelegateHandle);
			}
		}

		// Unregister the details customization
		//@TODO: Unregister them

		// Unregister all the asset types that we registered
		if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
		{
			IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			for (int32 Index = 0; Index < CreatedAssetTypeActions.Num(); ++Index)
			{
				AssetTools.UnregisterAssetTypeActions(CreatedAssetTypeActions[Index].ToSharedRef());
			}
		}
		CreatedAssetTypeActions.Empty();

		// Unregister commands
		FPaperEditorCommands::Unregister();

		// Unregister slate style overrides
		FPaperStyle::Shutdown();
	}
private:
	void RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action)
	{
		AssetTools.RegisterAssetTypeActions(Action);
		CreatedAssetTypeActions.Add(Action);
	}

	// Called when a property on the specified object is modified
	void OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (UPaperSpriteAtlas* Atlas = Cast<UPaperSpriteAtlas>(ObjectBeingModified))
		{
			FPaperAtlasGenerator::HandleAssetChangedEvent(Atlas);
		}
		else if (UPaperRuntimeSettings* Settings = Cast<UPaperRuntimeSettings>(ObjectBeingModified))
		{
			// Handle changes to experimental flags here
		}
	}

	void OnObjectReimported(UObject* InObject)
	{
		if (UTexture2D* Texture = Cast<UTexture2D>(InObject))
		{
			if (GetDefault<UPaperRuntimeSettings>()->bResizeSpriteDataToMatchTextures)
			{
				// Ideally we wouldn't need to load the sprite to keep the atlas up to date but this will work for now.
				IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
				AssetRegistry.WaitForCompletion();
				TArray<FAssetData> AssetsData;
				const bool bSearchSubClasses = true;
				AssetRegistry.GetAssetsByClass(UPaperSprite::StaticClass()->GetClassPathName(), AssetsData, bSearchSubClasses);

				const FString TexturePath = Texture->GetPathName();
				const FName SourceTextureMemberName = UPaperSprite::GetSourceTextureMemberName();
				for (const FAssetData& SpriteAssetData : AssetsData)
				{
					// We can't update a cooked asset
					if (!SpriteAssetData.HasAnyPackageFlags(PKG_Cooked))
					{
						FAssetTagValueRef AssetTagValue = SpriteAssetData.TagsAndValues.FindTag(SourceTextureMemberName);
						if (AssetTagValue.IsSet())
						{
							FAssetData SavedTextureAssetData = AssetRegistry.GetAssetByObjectPath(AssetTagValue.GetValue());
				
							FSoftObjectPath TextureSourcePath;
							if (SavedTextureAssetData.IsRedirector())
							{
								TextureSourcePath = AssetRegistry.GetRedirectedObjectPath(SavedTextureAssetData.GetSoftObjectPath());
							}
							else
							{
								TextureSourcePath = SavedTextureAssetData.GetSoftObjectPath();
							}

							if (TextureSourcePath == TexturePath)
							{
								if (UPaperSprite* PaperSprite = Cast<UPaperSprite>(SpriteAssetData.GetAsset()))
								{
									PaperSprite->OnObjectReimported(Texture);
								}
							}
						}
					}
				}
			}
		}
	}

	void RegisterSettings()
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "Paper2D",
				LOCTEXT("RuntimeSettingsName", "Paper 2D"),
				LOCTEXT("RuntimeSettingsDescription", "Configure the Paper 2D plugin"),
				GetMutableDefault<UPaperRuntimeSettings>());

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "SpriteEditor",
				LOCTEXT("SpriteEditorSettingsName", "Sprite Editor"),
				LOCTEXT("SpriteEditorSettingsDescription", "Configure the look and feel of the Sprite Editor."),
				GetMutableDefault<USpriteEditorSettings>());

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "FlipbookEditor",
				LOCTEXT("FlipbookEditorSettingsName", "Flipbook Editor"),
				LOCTEXT("FlipbookEditorSettingsDescription", "Configure the look and feel of the Flipbook Editor."),
				GetMutableDefault<UFlipbookEditorSettings>());

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "TileMapEditor",
				LOCTEXT("TileMapEditorSettingsName", "Tile Map Editor"),
				LOCTEXT("TileMapEditorSettingsDescription", "Configure the look and feel of the Tile Map Editor."),
				GetMutableDefault<UTileMapEditorSettings>());

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "TileSetEditor",
				LOCTEXT("TileSetEditorSettingsName", "Tile Set Editor"),
				LOCTEXT("TileSetEditorSettingsDescription", "Configure the look and feel of the Tile Set Editor."),
				GetMutableDefault<UTileSetEditorSettings>());

			SettingsModule->RegisterSettings("Project", "Editor", "Paper2DImport",
				LOCTEXT("PaperImporterSettingsName", "Paper2D - Import"),
				LOCTEXT("PaperImporterSettingsDescription", "Configure how assets get imported or converted to sprites."),
				GetMutableDefault<UPaperImporterSettings>());
		}
	}

	void UnregisterSettings()
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Project", "Editor", "Paper2DImport");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "TileSetEditor");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "TileMapEditor");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "FlipbookEditor");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "SpriteEditor");
			SettingsModule->UnregisterSettings("Project", "Plugins", "Paper2D");
		}
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FPaper2DEditor, Paper2DEditor);

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
