// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "PaperStyle.h"
#include "Styling/ISlateStyle.h"

class FPaperEditorCommands : public TCommands<FPaperEditorCommands>
{
public:
	FPaperEditorCommands()
		: TCommands<FPaperEditorCommands>(
			TEXT("PaperEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "PaperEditor", "Paper2D Editor"), // Localized context name for displaying
			NAME_None, // Parent
			FPaperStyle::Get()->GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:

};
