// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/ICursor.h"
#include "HitProxies.h"
#include "Math/Rotator.h"
#include "UnrealWidgetFwd.h"

//////////////////////////////////////////////////////////////////////////
// FSelectedItem

class FSelectedItem
{
protected:
	FSelectedItem(FName InTypeName)
		: TypeName(InTypeName)
	{
	}

protected:
	FName TypeName;
public:
	virtual bool IsA(FName TestType) const
	{
		return TestType == TypeName;
	}

	template<typename SelectedType>
	SelectedType* CastTo(FName TypeID)
	{
		return IsA(TypeID) ? (SelectedType*)this : (SelectedType*)nullptr;
	}

	virtual uint32 GetTypeHash() const
	{
		return 0;
	}

	virtual bool Equals(const FSelectedItem& OtherItem) const
	{
		return false;
	}

	virtual void ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode)
	{
	}

	virtual FVector GetWorldPos() const
	{
		return FVector::ZeroVector;
	}

	virtual EMouseCursor::Type GetMouseCursor() const
	{
		return EMouseCursor::Default;
	}

	// Is this item a background item (one that can be clicked thru/etc... to select something behind it?)
	virtual bool IsBackgroundObject() const
	{
		return false;
	}

	// Can this item be deleted?
	virtual bool CanBeDeleted() const
	{
		return true;
	}

	// Delete this item
	virtual void DeleteThisItem()
	{
	}

	virtual ~FSelectedItem() = default;
};

inline uint32 GetTypeHash(const FSelectedItem& Item)
{
	return Item.GetTypeHash();
}

inline bool operator==(const FSelectedItem& V1, const FSelectedItem& V2)
{
	return V1.Equals(V2);
}


//////////////////////////////////////////////////////////////////////////
// HSpriteSelectableObjectHitProxy

struct HSpriteSelectableObjectHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY(PAPER2DEDITOR_API);

	TSharedPtr<FSelectedItem> Data;

	HSpriteSelectableObjectHitProxy(TSharedPtr<FSelectedItem> InData)
		: HHitProxy(InData->IsBackgroundObject() ? HPP_World : HPP_UI)
		, Data(InData)
	{
	}

	// HHitProxy interface
	virtual bool AlwaysAllowsTranslucentPrimitives() const override { return true; }
	virtual EMouseCursor::Type GetMouseCursor() override { return Data->GetMouseCursor(); }
	// End of HHitProxy interface
};
