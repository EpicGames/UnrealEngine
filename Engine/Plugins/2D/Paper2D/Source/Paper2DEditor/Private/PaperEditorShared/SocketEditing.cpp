// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperEditorShared/SocketEditing.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "CanvasItem.h"
#include "Paper2DModule.h"
#include "PaperSpriteComponent.h"
#include "PaperSprite.h"
#include "PaperEditorShared/SpriteGeometryEditMode.h"
#include "CanvasTypes.h"
#include "UnrealClient.h"

//////////////////////////////////////////////////////////////////////////
// HSpriteSelectableObjectHitProxy

IMPLEMENT_HIT_PROXY(HSpriteSelectableObjectHitProxy, HHitProxy);

//////////////////////////////////////////////////////////////////////////
// FSpriteSelectedSocket

const FName FSpriteSelectedSocket::SocketTypeID(TEXT("Socket"));

FSpriteSelectedSocket::FSpriteSelectedSocket()
	: FSelectedItem(SocketTypeID)
{
}

bool FSpriteSelectedSocket::Equals(const FSelectedItem& OtherItem) const
{
	if (OtherItem.IsA(FSpriteSelectedSocket::SocketTypeID))
	{
		const FSpriteSelectedSocket& S1 = *this;
		const FSpriteSelectedSocket& S2 = *(FSpriteSelectedSocket*)(&OtherItem);

		return (S1.SocketName == S2.SocketName) && (S1.PreviewComponentPtr == S2.PreviewComponentPtr);
	}
	else
	{
		return false;
	}
}

void FSpriteSelectedSocket::ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode)
{
	if (UPrimitiveComponent* PreviewComponent = PreviewComponentPtr.Get())
	{
		UObject* AssociatedAsset = const_cast<UObject*>(PreviewComponent->AdditionalStatObject());
		if (UPaperSprite* Sprite = Cast<UPaperSprite>(AssociatedAsset))
		{
			if (FPaperSpriteSocket* Socket = Sprite->FindSocket(SocketName))
			{
				const bool bDoRotation = (MoveMode == UE::Widget::WM_Rotate) || (MoveMode == UE::Widget::WM_TranslateRotateZ);
				const bool bDoTranslation = (MoveMode == UE::Widget::WM_Translate) || (MoveMode == UE::Widget::WM_TranslateRotateZ);
				const bool bDoScale = MoveMode == UE::Widget::WM_Scale;

				if (bDoTranslation)
				{
					//@TODO: Currently sockets are in unflipped pivot space,
					const FVector Delta3D_UU = (PaperAxisX * Delta.X) + (PaperAxisY * -Delta.Y);
					const FVector Delta3D = Delta3D_UU * Sprite->GetPixelsPerUnrealUnit();
					Socket->LocalTransform.SetLocation(Socket->LocalTransform.GetLocation() + Delta3D);
				}

				if (bDoRotation)
				{
					const FRotator CurrentRot = Socket->LocalTransform.GetRotation().Rotator();
					FRotator SocketWinding;
					FRotator SocketRotRemainder;
					CurrentRot.GetWindingAndRemainder(SocketWinding, SocketRotRemainder);

					const FQuat ActorQ = SocketRotRemainder.Quaternion();
					const FQuat DeltaQ = Rotation.Quaternion();
					const FQuat ResultQ = DeltaQ * ActorQ;
					const FRotator NewSocketRotRem = FRotator( ResultQ );
					FRotator DeltaRot = NewSocketRotRem - SocketRotRemainder;
					DeltaRot.Normalize();

					const FRotator NewRotation(CurrentRot + DeltaRot);
					Socket->LocalTransform.SetRotation(NewRotation.Quaternion());
				}
				
				if (bDoScale)
				{
					const FVector4 LocalSpaceScaleOffset = Socket->LocalTransform.TransformVector(Scale3D);

					Socket->LocalTransform.SetScale3D(Socket->LocalTransform.GetScale3D() + LocalSpaceScaleOffset);
				}
			}
		}
	}
}

FVector FSpriteSelectedSocket::GetWorldPos() const
{
	if (UPrimitiveComponent* PreviewComponent = PreviewComponentPtr.Get())
	{
		return PreviewComponent->GetSocketLocation(SocketName);
	}

	return FVector::ZeroVector;
}

void FSpriteSelectedSocket::DeleteThisItem()
{
	if (UPrimitiveComponent* PreviewComponent = PreviewComponentPtr.Get())
	{
		if (PreviewComponent->DoesSocketExist(SocketName))
		{
			if (UPaperSpriteComponent* SpriteComponent = Cast<UPaperSpriteComponent>(PreviewComponent))
			{
				if (UPaperSprite* SpriteAsset = SpriteComponent->GetSprite())
				{
					SpriteAsset->RemoveSocket(SocketName);
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FSocketEditingHelper

void FSocketEditingHelper::DrawSockets(FSpriteGeometryEditMode* GeometryEditMode, UPrimitiveComponent* PreviewComponent, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (PreviewComponent != nullptr)
	{
		const bool bIsHitTesting = PDI->IsHitTesting();

		const float DiamondSize = 5.0f;
		const FColor UnselectedDiamondColor(255, 128, 128);
		const FColor SelectedDiamondColor(FColor::White);

		TArray<FComponentSocketDescription> SocketList;
		PreviewComponent->QuerySupportedSockets(/*out*/ SocketList);

		for (const FComponentSocketDescription& Socket : SocketList)
		{
			if (bIsHitTesting)
			{
				TSharedPtr<FSpriteSelectedSocket> Data = MakeShareable(new FSpriteSelectedSocket);
				Data->PreviewComponentPtr = PreviewComponent;
				Data->SocketName = Socket.Name;
				PDI->SetHitProxy(new HSpriteSelectableObjectHitProxy(Data));
			}

			const bool bIsSelected = (GeometryEditMode != nullptr) && GeometryEditMode->IsSocketSelected(Socket.Name);
			const FColor& DiamondColor = bIsSelected ? SelectedDiamondColor : UnselectedDiamondColor;

			const FMatrix SocketTM = PreviewComponent->GetSocketTransform(Socket.Name, RTS_World).ToMatrixWithScale();
			DrawWireDiamond(PDI, SocketTM, DiamondSize, DiamondColor, SDPG_Foreground);

			if (bIsHitTesting)
			{
				PDI->SetHitProxy(nullptr);
			}
		}
	}
}

void FSocketEditingHelper::DrawSocketNames(FSpriteGeometryEditMode* GeometryEditMode, UPrimitiveComponent* PreviewComponent, FViewport& Viewport, FSceneView& View, FCanvas& Canvas)
{
	if (PreviewComponent != nullptr)
	{
		const int32 HalfX = Viewport.GetSizeXY().X / 2;
		const int32 HalfY = Viewport.GetSizeXY().Y / 2;

		const FColor UnselectedSocketNameColor(255, 196, 196);
		const FColor SelectedSocketNameColor(FColor::White);

		TArray<FComponentSocketDescription> SocketList;
		PreviewComponent->QuerySupportedSockets(/*out*/ SocketList);

		for (const FComponentSocketDescription& Socket : SocketList)
		{
			const FVector SocketWorldPos = PreviewComponent->GetSocketLocation(Socket.Name);

			const FPlane Proj = View.Project(SocketWorldPos);
			if (Proj.W > 0.f)
			{
				const int32 XPos = HalfX + (HalfX * Proj.X);
				const int32 YPos = HalfY + (HalfY * (-Proj.Y));

				const bool bIsSelected = (GeometryEditMode != nullptr) && GeometryEditMode->IsSocketSelected(Socket.Name);
				const FColor& SocketColor = bIsSelected ? SelectedSocketNameColor : UnselectedSocketNameColor;

				FCanvasTextItem Msg(FVector2D(XPos, YPos), FText::FromString(Socket.Name.ToString()), GEngine->GetMediumFont(), SocketColor);
				Msg.EnableShadow(FLinearColor::Black);
				Canvas.DrawItem(Msg);

// 				//@TODO: Draws the current value of the rotation (probably want to keep this!)
// 				if (bManipulating && StaticMeshEditorPtr.Pin()->GetSelectedSocket() == Socket)
// 				{
// 					// Figure out the text height
// 					FTextSizingParameters Parameters(GEngine->GetSmallFont(), 1.0f, 1.0f);
// 					UCanvas::CanvasStringSize(Parameters, *Socket->SocketName.ToString());
// 					int32 YL = FMath::TruncToInt(Parameters.DrawYL);
// 
// 					DrawAngles(&Canvas, XPos, YPos + YL, 
// 						Widget->GetCurrentAxis(), 
// 						GetWidgetMode(),
// 						Socket->RelativeRotation,
// 						Socket->RelativeLocation);
// 				}
			}
		}
	}
}
