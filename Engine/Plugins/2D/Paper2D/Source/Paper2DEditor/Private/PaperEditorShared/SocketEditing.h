// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "PaperEditorShared/AssetEditorSelectedItem.h"

class FCanvas;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class UPrimitiveComponent;

//////////////////////////////////////////////////////////////////////////
// FSpriteSelectedSocket

class FSpriteSelectedSocket : public FSelectedItem
{
public:
	FName SocketName;
	TWeakObjectPtr<UPrimitiveComponent> PreviewComponentPtr;

	static const FName SocketTypeID;

public:
	FSpriteSelectedSocket();

	// FSelectedItem interface
	virtual bool Equals(const FSelectedItem& OtherItem) const override;
	virtual void ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode) override;
	FVector GetWorldPos() const override;
	virtual void DeleteThisItem() override;
	// End of FSelectedItem interface
};

//////////////////////////////////////////////////////////////////////////
// FSocketEditingHelper

class FSocketEditingHelper
{
public:
	static void DrawSockets(class FSpriteGeometryEditMode* GeometryEditMode, UPrimitiveComponent* PreviewComponent, const FSceneView* View, FPrimitiveDrawInterface* PDI);
	static void DrawSocketNames(class FSpriteGeometryEditMode* GeometryEditMode, UPrimitiveComponent* PreviewComponent, FViewport& Viewport, FSceneView& View, FCanvas& Canvas);
};
