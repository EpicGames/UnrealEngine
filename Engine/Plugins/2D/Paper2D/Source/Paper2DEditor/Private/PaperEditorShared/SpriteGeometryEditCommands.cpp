// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperEditorShared/SpriteGeometryEditCommands.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////
// FSpriteGeometryEditCommands

void FSpriteGeometryEditCommands::RegisterCommands()
{
	// Show toggles
	UI_COMMAND(SetShowNormals, "Normals", "Toggles display of vertex normals in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());

	// Geometry editing commands
	UI_COMMAND(DeleteSelection, "Delete", "Delete the selection.", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete, EModifierKey::None));
	UI_COMMAND(SplitEdge, "Split", "Split edge.", EUserInterfaceActionType::Button, FInputChord(EKeys::Insert, EModifierKey::None));
	UI_COMMAND(AddBoxShape, "Add Box", "Adds a new box shape.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleAddPolygonMode, "Add Polygon", "Adds a new polygon shape.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AddCircleShape, "Add Circle", "Adds a new circle shape.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SnapAllVertices, "Snap to pixel grid", "Snaps all vertices to the pixel grid.", EUserInterfaceActionType::Button, FInputChord());
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
