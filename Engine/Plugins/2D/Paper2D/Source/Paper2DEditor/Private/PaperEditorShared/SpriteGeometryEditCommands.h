// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "PaperStyle.h"
#include "Styling/ISlateStyle.h"

class FSpriteGeometryEditCommands : public TCommands<FSpriteGeometryEditCommands>
{
public:
	FSpriteGeometryEditCommands()
		: TCommands<FSpriteGeometryEditCommands>(
			TEXT("SpriteGeometryEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "SpriteGeometryEditor", "Sprite Geometry Editor"), // Localized context name for displaying
			NAME_None, // Parent
			FPaperStyle::Get()->GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:
	// Show toggles
	TSharedPtr<FUICommandInfo> SetShowNormals;

	// Geometry editing commands
	TSharedPtr<FUICommandInfo> DeleteSelection;
	TSharedPtr<FUICommandInfo> SplitEdge;
	TSharedPtr<FUICommandInfo> AddBoxShape;
	TSharedPtr<FUICommandInfo> AddCircleShape;
	TSharedPtr<FUICommandInfo> ToggleAddPolygonMode;
	TSharedPtr<FUICommandInfo> SnapAllVertices;
};
