// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperEditorShared/SpriteGeometryEditMode.h"
#include "SceneView.h"
#include "EditorViewportClient.h"
#include "Framework/Commands/UICommandList.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EditorModeManager.h"
#include "SpriteEditor/SpriteEditorSelections.h"
#include "PaperEditorShared/SpriteGeometryEditCommands.h"
#include "PhysicsEngine/BodySetup.h"
#include "PaperEditorShared/SocketEditing.h"

#define LOCTEXT_NAMESPACE "PaperGeometryEditing"

//////////////////////////////////////////////////////////////////////////
// FSpriteGeometryEditMode

const FEditorModeID FSpriteGeometryEditMode::EM_SpriteGeometry(TEXT("SpriteGeometryEditMode"));

const FLinearColor FSpriteGeometryEditMode::MarqueeDrawColor(1.0f, 1.0f, 1.0f, 0.5f);

FSpriteGeometryEditMode::FSpriteGeometryEditMode()
	: BoundsForNewShapes(ForceInitToZero)
	, GeometryVertexColor(FLinearColor::White)
	, NegativeGeometryVertexColor(FLinearColor::White)
	, SpriteGeometryHelper(nullptr)
	, bIsMarqueeTracking(false)
	, MarqueeStartPos(ForceInitToZero)
	, MarqueeEndPos(ForceInitToZero)
{
	BoundsForNewShapes.Max = FVector2D(20.0f, 20.0f);

	bDrawPivot = false;
	bDrawGrid = false;
}

void FSpriteGeometryEditMode::Initialize()
{
}

void FSpriteGeometryEditMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	FEdMode::DrawHUD(ViewportClient, Viewport, View, Canvas);

	int32 YPos = 42;
	SpriteGeometryHelper.DrawGeometry_CanvasPass(*Viewport, *View, *Canvas, /*inout*/ YPos, GeometryVertexColor, NegativeGeometryVertexColor);

	if (bIsMarqueeTracking)
	{
		DrawMarquee(*Viewport, *View, *Canvas, MarqueeDrawColor);
	}
}

void FSpriteGeometryEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);

	SpriteGeometryHelper.DrawGeometry(*View, *PDI, GeometryVertexColor, NegativeGeometryVertexColor);
}

bool FSpriteGeometryEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	FViewport* Viewport = InViewportClient->Viewport;

	const bool bIsCtrlKeyDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	const bool bIsShiftKeyDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
	const bool bIsAltKeyDown = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);
	bool bHandled = false;

	const bool bAllowSelectVertex = !(IsEditingGeometry() && SpriteGeometryHelper.IsAddingPolygon()) && !bIsShiftKeyDown;

	const bool bClearSelectionModifier = bIsCtrlKeyDown;
	const bool bDeleteClickedVertex = bIsAltKeyDown;
	const bool bInsertVertexModifier = bIsShiftKeyDown;
	HSpriteSelectableObjectHitProxy* SelectedItemProxy = HitProxyCast<HSpriteSelectableObjectHitProxy>(HitProxy);

	if (bAllowSelectVertex && (SelectedItemProxy != nullptr))
	{
		if (!bClearSelectionModifier)
		{
			SpriteGeometryHelper.ClearSelectionSet();
		}

		if (bDeleteClickedVertex)
		{
			// Delete selection
			if (const FSpriteSelectedVertex* SelectedVertex = SelectedItemProxy->Data->CastTo<const FSpriteSelectedVertex>(FSelectionTypes::Vertex))
			{
				SpriteGeometryHelper.ClearSelectionSet();
				SpriteGeometryHelper.AddPolygonVertexToSelection(SelectedVertex->ShapeIndex, SelectedVertex->VertexIndex);
				SpriteGeometryHelper.DeleteSelectedItems();
			}
			else if (const FSpriteSelectedShape* SelectedShape = SelectedItemProxy->Data->CastTo<const FSpriteSelectedShape>(FSelectionTypes::GeometryShape))
			{
				SpriteGeometryHelper.ClearSelectionSet();
				SpriteGeometryHelper.AddShapeToSelection(SelectedShape->ShapeIndex);
				SpriteGeometryHelper.DeleteSelectedItems();
			}
		}
		else if (Click.GetEvent() == EInputEvent::IE_DoubleClick)
		{
			// Double click to select a polygon
			if (const FSpriteSelectedVertex* SelectedVertex = SelectedItemProxy->Data->CastTo<const FSpriteSelectedVertex>(FSelectionTypes::Vertex))
			{
				SpriteGeometryHelper.ClearSelectionSet();
				SpriteGeometryHelper.AddShapeToSelection(SelectedVertex->ShapeIndex);
			}
		}
		else
		{
			//@TODO: This needs to be generalized!
			if (const FSpriteSelectedEdge* SelectedEdge = SelectedItemProxy->Data->CastTo<const FSpriteSelectedEdge>(FSelectionTypes::Edge))
			{
				// Add the next vertex defined by this edge
				SpriteGeometryHelper.AddPolygonEdgeToSelection(SelectedEdge->ShapeIndex, SelectedEdge->VertexIndex);
			}
			else if (const FSpriteSelectedVertex* SelectedVertex = SelectedItemProxy->Data->CastTo<const FSpriteSelectedVertex>(FSelectionTypes::Vertex))
			{
				SpriteGeometryHelper.AddPolygonVertexToSelection(SelectedVertex->ShapeIndex, SelectedVertex->VertexIndex);
			}
			else if (const FSpriteSelectedShape* SelectedShape = SelectedItemProxy->Data->CastTo<const FSpriteSelectedShape>(FSelectionTypes::GeometryShape))
			{
				SpriteGeometryHelper.AddShapeToSelection(SelectedShape->ShapeIndex);
			}
			else
			{
				SpriteGeometryHelper.SelectItem(SelectedItemProxy->Data);
			}
		}

		bHandled = true;
	}
	// 	else if (HWidgetUtilProxy* PivotProxy = HitProxyCast<HWidgetUtilProxy>(HitProxy))
	// 	{
	// 		//const bool bUserWantsPaint = bIsLeftButtonDown && ( !GetDefault<ULevelEditorViewportSettings>()->bLeftMouseDragMovesCamera ||  bIsCtrlDown );
	// 		//findme
	// 		WidgetAxis = WidgetProxy->Axis;
	// 
	// 			// Calculate the screen-space directions for this drag.
	// 			FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( Viewport, GetScene(), EngineShowFlags ));
	// 			FSceneView* View = CalcSceneView(&ViewFamily);
	// 			WidgetProxy->CalcVectors(View, FViewportClick(View, this, Key, Event, HitX, HitY), LocalManipulateDir, WorldManipulateDir, DragX, DragY);
	// 			bHandled = true;
	// 	}
	else
	{
		if (IsEditingGeometry() && !SpriteGeometryHelper.IsAddingPolygon())
		{
			FSpriteGeometryCollection* Geometry = SpriteGeometryHelper.GetGeometryBeingEdited();

			if (bInsertVertexModifier)
			{
				const FPlane SpritePlane(PaperAxisX, FVector::ZeroVector, PaperAxisY);
				const FVector WorldPoint = FMath::LinePlaneIntersection(Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection(), SpritePlane);
				const FVector2D SpriteSpaceClickPoint = SpriteGeometryHelper.GetEditorContext()->WorldSpaceToTextureSpace(WorldPoint);

				// find a polygon to add vert to
				bool bFoundShapeToAddTo = false;
				for (TSharedPtr<FSelectedItem> SelectedItemPtr : SpriteGeometryHelper.GetSelectionSet())
				{
					if (const FSpriteSelectedVertex* SelectedVertex = SelectedItemPtr->CastTo<const FSpriteSelectedVertex>(FSelectionTypes::Vertex)) //@TODO:  Inflexible?
					{
						SpriteGeometryHelper.AddPointToGeometry(SpriteSpaceClickPoint, SelectedVertex->ShapeIndex);
						bFoundShapeToAddTo = true;
						break;
					}
				}

				if (!bFoundShapeToAddTo)
				{
					SpriteGeometryHelper.AddPointToGeometry(SpriteSpaceClickPoint);
				}
				bHandled = true;
			}
		}
		else if (!IsEditingGeometry())
		{
			// Clicked on the background (missed any proxies), deselect the socket or whatever was selected
			SpriteGeometryHelper.ClearSelectionSet();
		}
	}

	return bHandled ? true : FEdMode::HandleClick(InViewportClient, HitProxy, Click);
}

bool FSpriteGeometryEditMode::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	bool bHandled = false;
	FInputEventState InputState(Viewport, Key, Event);

	// Handle marquee tracking in source region edit mode
	if (IsEditingGeometry())
	{
		if (SpriteGeometryHelper.IsAddingPolygon())
		{
			if (Key == EKeys::LeftMouseButton)
			{
				const int32 HitX = Viewport->GetMouseX();
				const int32 HitY = Viewport->GetMouseY();

				// Calculate the texture space position of the mouse click
				FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(Viewport, ViewportClient->GetScene(), ViewportClient->EngineShowFlags));
				FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
				const FVector WorldPoint = View->PixelToWorld(HitX, HitY, 0);
				const FVector2D TexturePoint = SpriteGeometryHelper.GetEditorContext()->WorldSpaceToTextureSpace(WorldPoint);

				// Add or close the polygon (depending on where the click happened and how)
				const bool bMakeSubtractiveIfAllowed = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
				SpriteGeometryHelper.HandleAddPolygonClick(TexturePoint, bMakeSubtractiveIfAllowed, *View, Event);
			}
			else if ((Key == EKeys::BackSpace) && (Event == IE_Pressed))
			{
				SpriteGeometryHelper.DeleteLastVertexFromAddPolygonMode();
			}			
			else if (Key == EKeys::Enter)
			{
				SpriteGeometryHelper.ResetAddPolygonMode();
			}
			else if (Key == EKeys::Escape)
			{
				SpriteGeometryHelper.AbandonAddPolygonMode();
			}
		}
		else
		{
 			if (ProcessMarquee(Viewport, Key, Event, true))
 			{
 				const bool bAddingToSelection = InputState.IsShiftButtonPressed(); //@TODO: control button moves widget? Hopefully make this more consistent when that is changed
 				SelectVerticesInMarquee(ViewportClient, Viewport, bAddingToSelection);
 			}
		}
	}

	//@TODO: Support select-and-drag in a single operation (may involve InputAxis and StartTracking)

	// Pass keys to standard controls, if we didn't consume input
	return bHandled ? true : FEdMode::InputKey(ViewportClient, Viewport, Key, Event);
}

void FSpriteGeometryEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if (bIsMarqueeTracking)
	{
		const int32 HitX = ViewportClient->Viewport->GetMouseX();
		const int32 HitY = ViewportClient->Viewport->GetMouseY();
		MarqueeEndPos = FVector2D(HitX, HitY);
	}

	FEdMode::Tick(ViewportClient, DeltaTime);
}

bool FSpriteGeometryEditMode::ShouldDrawWidget() const
{
	return SpriteGeometryHelper.HasAnySelectedItems();
}

FVector FSpriteGeometryEditMode::GetWidgetLocation() const
{
	FVector SummedPos(ForceInitToZero);

	if (SpriteGeometryHelper.HasAnySelectedItems())
	{
		// Find the center of the selection set
		for (TSharedPtr<FSelectedItem> SelectedItem : SpriteGeometryHelper.GetSelectionSet())
		{
			SummedPos += SelectedItem->GetWorldPos();
		}
		return (SummedPos / SpriteGeometryHelper.GetSelectionSet().Num());
	}

	return SummedPos;
}

bool FSpriteGeometryEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	bool bHandled = false;

	const bool bManipulating = GetModeManager()->IsTracking();
	EAxisList::Type CurrentAxis = GetCurrentWidgetAxis();
	
	if (bManipulating && (CurrentAxis != EAxisList::None))
	{
		bHandled = true;

		const UE::Widget::EWidgetMode MoveMode = GetModeManager()->GetWidgetMode();

		// Negate Y because vertices are in source texture space, not world space
		const FVector2D Drag2D(FVector::DotProduct(InDrag, PaperAxisX), -FVector::DotProduct(InDrag, PaperAxisY));

		// Apply the delta to all of the selected objects
		for (TSharedPtr<FSelectedItem> SelectedItem : SpriteGeometryHelper.GetSelectionSet())
		{
			SelectedItem->ApplyDelta(Drag2D, InRot, InScale, MoveMode);
		}

		if (SpriteGeometryHelper.HasAnySelectedItems())
		{
			SpriteGeometryHelper.GetEditorContext()->MarkTransactionAsDirty();
		}
	}

	return bHandled ? true : FEdMode::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
}

void FSpriteGeometryEditMode::SetEditorContext(class ISpriteSelectionContext* InNewEditorContext)
{
	SpriteGeometryHelper.SetEditorContext(InNewEditorContext);
}

void FSpriteGeometryEditMode::SetNewGeometryPreferredBounds(FBox2D& NewDesiredBounds)
{
	BoundsForNewShapes = NewDesiredBounds;
}

void FSpriteGeometryEditMode::SetGeometryColors(const FLinearColor& NewVertexColor, const FLinearColor& NewNegativeVertexColor)
{
	GeometryVertexColor = NewVertexColor;
	NegativeGeometryVertexColor = NewNegativeVertexColor;
}

void FSpriteGeometryEditMode::SetGeometryBeingEdited(FSpriteGeometryCollection* NewGeometryBeingEdited, bool bInAllowCircles, bool bInAllowSubtractivePolygons)
{
	if (SpriteGeometryHelper.GetGeometryBeingEdited() != NewGeometryBeingEdited)
	{
		SpriteGeometryHelper.SetGeometryBeingEdited(NewGeometryBeingEdited, bInAllowCircles, bInAllowSubtractivePolygons);
	}
	bIsMarqueeTracking = false;
}

void FSpriteGeometryEditMode::BindCommands(TSharedPtr<FUICommandList> CommandList)
{
	const FSpriteGeometryEditCommands& Commands = FSpriteGeometryEditCommands::Get();

	// Show toggles
	CommandList->MapAction(
		Commands.SetShowNormals,
		FExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::ToggleShowNormals),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::IsShowNormalsEnabled));

	// Geometry editing commands
	CommandList->MapAction(
		Commands.DeleteSelection,
		FExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::DeleteSelectedItems),
		FCanExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanDeleteSelection));
	CommandList->MapAction(
		Commands.AddBoxShape,
		FExecuteAction::CreateSP(this, &FSpriteGeometryEditMode::AddBoxShape),
		FCanExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanAddBoxShape),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanAddBoxShape));
	CommandList->MapAction(
		Commands.ToggleAddPolygonMode,
		FExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::ToggleAddPolygonMode),
		FCanExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanAddPolygon),
		FIsActionChecked::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::IsAddingPolygon),
		FIsActionButtonVisible::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanAddPolygon));
	CommandList->MapAction(
		Commands.AddCircleShape,
		FExecuteAction::CreateSP(this, &FSpriteGeometryEditMode::AddCircleShape),
		FCanExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanAddCircleShape),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanAddCircleShape));
	CommandList->MapAction(
		Commands.SnapAllVertices,
		FExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::SnapAllVerticesToPixelGrid),
		FCanExecuteAction::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanSnapVerticesToPixelGrid),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(&SpriteGeometryHelper, &FSpriteGeometryEditingHelper::CanSnapVerticesToPixelGrid));
}


void FSpriteGeometryEditMode::AddBoxShape()
{
	SpriteGeometryHelper.AddNewBoxShape(BoundsForNewShapes.GetCenter(), BoundsForNewShapes.GetSize());
}

void FSpriteGeometryEditMode::AddCircleShape()
{
	const float SmallerBoundingAxisSize = BoundsForNewShapes.GetSize().GetMin();
	const float CircleRadius = SmallerBoundingAxisSize * 0.5f;

	SpriteGeometryHelper.AddNewCircleShape(BoundsForNewShapes.GetCenter(), CircleRadius);
}

bool FSpriteGeometryEditMode::IsEditingGeometry() const
{
	return SpriteGeometryHelper.IsEditingGeometry();
}

void FSpriteGeometryEditMode::SelectVerticesInMarquee(FEditorViewportClient* ViewportClient, FViewport* Viewport, bool bAddToSelection)
{
	if (!bAddToSelection)
	{
		SpriteGeometryHelper.ClearSelectionSet();
	}

	{
		// Calculate world space positions
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(Viewport, ViewportClient->GetScene(), ViewportClient->EngineShowFlags));
		FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
		const FVector StartPos = View->PixelToWorld(MarqueeStartPos.X, MarqueeStartPos.Y, 0);
		const FVector EndPos = View->PixelToWorld(MarqueeEndPos.X, MarqueeEndPos.Y, 0);

		// Convert to source texture space to work out the pixels dragged
		FVector2D TextureSpaceStartPos = SpriteGeometryHelper.GetEditorContext()->WorldSpaceToTextureSpace(StartPos);
		FVector2D TextureSpaceEndPos = SpriteGeometryHelper.GetEditorContext()->WorldSpaceToTextureSpace(EndPos);

		if (TextureSpaceStartPos.X > TextureSpaceEndPos.X)
		{
			Swap(TextureSpaceStartPos.X, TextureSpaceEndPos.X);
		}
		if (TextureSpaceStartPos.Y > TextureSpaceEndPos.Y)
		{
			Swap(TextureSpaceStartPos.Y, TextureSpaceEndPos.Y);
		}

		const FBox2D QueryBounds(TextureSpaceStartPos, TextureSpaceEndPos);

		// Check geometry
		if (FSpriteGeometryCollection* Geometry = SpriteGeometryHelper.GetGeometryBeingEdited())
		{
			for (int32 ShapeIndex = 0; ShapeIndex < Geometry->Shapes.Num(); ++ShapeIndex)
			{
				const FSpriteGeometryShape& Shape = Geometry->Shapes[ShapeIndex];

				bool bSelectWholeShape = false;

				if ((Shape.ShapeType == ESpriteShapeType::Circle) || (Shape.ShapeType == ESpriteShapeType::Box))
				{
					// First see if we are fully contained
					const FBox2D ShapeBoxBounds(Shape.BoxPosition - Shape.BoxSize * 0.5f, Shape.BoxPosition + Shape.BoxSize * 0.5f);
					if (QueryBounds.IsInside(ShapeBoxBounds))
					{
						bSelectWholeShape = true;
					}
				}

				//@TODO: Try intersecting with the circle if it wasn't entirely enclosed

				if (bSelectWholeShape)
				{
					SpriteGeometryHelper.AddShapeToSelection(ShapeIndex);
				}
				else
				{
					// Try to select some subset of the vertices
					for (int32 VertexIndex = 0; VertexIndex < Shape.Vertices.Num(); ++VertexIndex)
					{
						const FVector2D TextureSpaceVertex = Shape.ConvertShapeSpaceToTextureSpace(Shape.Vertices[VertexIndex]);
						if (QueryBounds.IsInside(TextureSpaceVertex))
						{
							SpriteGeometryHelper.AddPolygonVertexToSelection(ShapeIndex, VertexIndex);
						}
					}
				}
			}
		}

		//@TODO: Check other items (sockets/etc...)
	}
}

bool FSpriteGeometryEditMode::ProcessMarquee(FViewport* Viewport, FKey Key, EInputEvent Event, bool bMarqueeStartModifierPressed)
{
	bool bMarqueeReady = false;

	if (Key == EKeys::LeftMouseButton)
	{
		int32 HitX = Viewport->GetMouseX();
		int32 HitY = Viewport->GetMouseY();
		if (Event == IE_Pressed && bMarqueeStartModifierPressed)
		{
			HHitProxy* HitResult = Viewport->GetHitProxy(HitX, HitY);

			if ((HitResult == nullptr) || (HitResult->Priority == HPP_World))
			{
				bIsMarqueeTracking = true;
				MarqueeStartPos = FVector2D(HitX, HitY);
				MarqueeEndPos = MarqueeStartPos;
			}
		}
		else if (bIsMarqueeTracking && (Event == IE_Released))
		{
			MarqueeEndPos = FVector2D(HitX, HitY);
			bIsMarqueeTracking = false;
			bMarqueeReady = true;
		}
	}
	else if (bIsMarqueeTracking && (Key == EKeys::Escape))
	{
		// Cancel marquee selection
		bIsMarqueeTracking = false;
	}

	return bMarqueeReady;
}

bool FSpriteGeometryEditMode::IsSocketSelected(FName SocketName) const
{
	for (TSharedPtr<FSelectedItem> SelectedItemPtr : SpriteGeometryHelper.GetSelectionSet())
	{
		if (const FSpriteSelectedSocket* SelectedSocket = SelectedItemPtr->CastTo<const FSpriteSelectedSocket>(FSpriteSelectedSocket::SocketTypeID))
		{
			if (SelectedSocket->SocketName == SocketName)
			{
				return true;
			}
		}
	}

	return false;
}

void FSpriteGeometryEditMode::DrawMarquee(FViewport& InViewport, const FSceneView& View, FCanvas& Canvas, const FLinearColor& Color)
{
	FVector2D MarqueeDelta = MarqueeEndPos - MarqueeStartPos;
	FVector2D MarqueeVertices[4];
	MarqueeVertices[0] = FVector2D(MarqueeStartPos.X, MarqueeStartPos.Y);
	MarqueeVertices[1] = MarqueeVertices[0] + FVector2D(MarqueeDelta.X, 0);
	MarqueeVertices[2] = MarqueeVertices[0] + FVector2D(MarqueeDelta.X, MarqueeDelta.Y);
	MarqueeVertices[3] = MarqueeVertices[0] + FVector2D(0, MarqueeDelta.Y);
	for (int32 MarqueeVertexIndex = 0; MarqueeVertexIndex < 4; ++MarqueeVertexIndex)
	{
		const int32 NextVertexIndex = (MarqueeVertexIndex + 1) % 4;
		FCanvasLineItem MarqueeLine(MarqueeVertices[MarqueeVertexIndex], MarqueeVertices[NextVertexIndex]);
		MarqueeLine.SetColor(Color);
		Canvas.DrawItem(MarqueeLine);
	}
}


void FSpriteGeometryEditMode::DrawGeometryStats(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, const FSpriteGeometryCollection& Geometry, bool bIsRenderGeometry, int32& YPos)
{
	// Draw the type of geometry we're displaying stats for
	const FText GeometryName = bIsRenderGeometry ? LOCTEXT("RenderGeometry", "Render Geometry (source)") : LOCTEXT("CollisionGeometry", "Collision Geometry (source)");

	FCanvasTextItem TextItem(FVector2D(6, YPos), GeometryName, GEngine->GetSmallFont(), FLinearColor::White);
	TextItem.EnableShadow(FLinearColor::Black);

	TextItem.Draw(&Canvas);
	TextItem.Position += FVector2D(6.0f, 18.0f);

	// Draw the number of shapes
	TextItem.Text = FText::Format(LOCTEXT("PolygonCount", "Shapes: {0}"), FText::AsNumber(Geometry.Shapes.Num()));
	TextItem.Draw(&Canvas);
	TextItem.Position.Y += 18.0f;

	// Draw the number of vertices
	int32 NumVerts = 0;
	for (int32 PolyIndex = 0; PolyIndex < Geometry.Shapes.Num(); ++PolyIndex)
	{
		NumVerts += Geometry.Shapes[PolyIndex].Vertices.Num();
	}

	TextItem.Text = FText::Format(LOCTEXT("VerticesCount", "Verts: {0}"), FText::AsNumber(NumVerts));
	TextItem.Draw(&Canvas);
	TextItem.Position.Y += 18.0f;

	YPos = (int32)TextItem.Position.Y;
}

void FSpriteGeometryEditMode::DrawCollisionStats(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, class UBodySetup* BodySetup, int32& YPos)
{
	FCanvasTextItem TextItem(FVector2D(6, YPos), LOCTEXT("CollisionGeomBaked", "Collision Geometry (baked)"), GEngine->GetSmallFont(), FLinearColor::White);
	TextItem.EnableShadow(FLinearColor::Black);

	TextItem.Draw(&Canvas);
	TextItem.Position += FVector2D(6.0f, 18.0f);

	// Collect stats
	const FKAggregateGeom& AggGeom3D = BodySetup->AggGeom;

	int32 NumSpheres = AggGeom3D.SphereElems.Num();
	int32 NumBoxes = AggGeom3D.BoxElems.Num();
	int32 NumCapsules = AggGeom3D.SphylElems.Num();
	int32 NumConvexElems = AggGeom3D.ConvexElems.Num();
	int32 NumConvexVerts = 0;
	bool bIs2D = false;

	for (const FKConvexElem& ConvexElement : AggGeom3D.ConvexElems)
	{
		NumConvexVerts += ConvexElement.VertexData.Num();
	}

	if (NumSpheres > 0)
	{
		static const FText SpherePrompt = LOCTEXT("SphereCount", "Spheres: {0}");
		static const FText CirclePrompt = LOCTEXT("CircleCount", "Circles: {0}");

		TextItem.Text = FText::Format(bIs2D ? CirclePrompt : SpherePrompt, FText::AsNumber(NumSpheres));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	if (NumBoxes > 0)
	{
		static const FText BoxPrompt = LOCTEXT("BoxCount", "Boxes: {0}");
		TextItem.Text = FText::Format(BoxPrompt, FText::AsNumber(NumBoxes));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	if (NumCapsules > 0)
	{
		static const FText CapsulePrompt = LOCTEXT("CapsuleCount", "Capsules: {0}");
		TextItem.Text = FText::Format(CapsulePrompt, FText::AsNumber(NumCapsules));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	if (NumConvexElems > 0)
	{
		static const FText ConvexPrompt = LOCTEXT("ConvexCount", "Convex Shapes: {0} ({1} verts)");
		TextItem.Text = FText::Format(ConvexPrompt, FText::AsNumber(NumConvexElems), FText::AsNumber(NumConvexVerts));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	if ((NumConvexElems + NumCapsules + NumBoxes + NumSpheres) == 0)
	{
		static const FText NoShapesPrompt = LOCTEXT("NoCollisionDataWarning", "Warning: Collision is enabled but there are no shapes");
		TextItem.Text = NoShapesPrompt;
		TextItem.SetColor(FLinearColor::Yellow);
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	YPos = (int32)TextItem.Position.Y;
}

bool FSpriteGeometryEditMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	switch(CheckMode)
	{
	case UE::Widget::WM_None:
	case UE::Widget::WM_Translate:
		return true;
	default:
		return false;
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
