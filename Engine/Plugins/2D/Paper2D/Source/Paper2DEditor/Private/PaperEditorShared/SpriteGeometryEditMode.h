// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdMode.h"
#include "PaperEditorShared/SpriteGeometryEditing.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FUICommandList;
class FViewport;
struct FSpriteGeometryCollection;
struct FViewportClick;

//////////////////////////////////////////////////////////////////////////
// FSpriteGeometryEditMode

class FSpriteGeometryEditMode : public FEdMode
{
public:
	static const FEditorModeID EM_SpriteGeometry;
	static const FLinearColor MarqueeDrawColor;
public:

	FSpriteGeometryEditMode();

	// FEdMode interface
	virtual void Initialize() override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual bool ShouldDrawWidget() const override;
	// End of FEdMode interface

	// Changes the editor interface to point to the hosting editor; this is basically required
	void SetEditorContext(class ISpriteSelectionContext* InNewEditorContext);

	// Sets the default bounds for newly created boxes/circles/etc...
	void SetNewGeometryPreferredBounds(FBox2D& NewDesiredBounds);

	// Sets the draw color for geometry
	void SetGeometryColors(const FLinearColor& NewVertexColor, const FLinearColor& NewNegativeVertexColor);

	// Changes the geometry being edited (clears the selection set in the process)
	void SetGeometryBeingEdited(FSpriteGeometryCollection* NewGeometryBeingEdited, bool bInAllowCircles, bool bInAllowSubtractivePolygons);

	void BindCommands(TSharedPtr<FUICommandList> InCommandList);

	FVector2D GetMarqueeStartPos() const { return MarqueeStartPos; }
	FVector2D GetMarqueeEndPos() const { return MarqueeEndPos; }
	bool ProcessMarquee(FViewport* Viewport, FKey Key, EInputEvent Event, bool bMarqueeStartModifierPressed);

	// Returns true if the specified socket is selected
	bool IsSocketSelected(FName SocketName) const;

	static void DrawGeometryStats(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, const FSpriteGeometryCollection& Geometry, bool bIsRenderGeometry, int32& YPos);
	static void DrawCollisionStats(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, class UBodySetup* BodySetup, int32& YPos);

	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
protected:
	FBox2D BoundsForNewShapes;
	FLinearColor GeometryVertexColor;
	FLinearColor NegativeGeometryVertexColor;

	// Sprite geometry editing/rendering helper
	FSpriteGeometryEditingHelper SpriteGeometryHelper;

	// Marquee tracking
	bool bIsMarqueeTracking;
	FVector2D MarqueeStartPos;
	FVector2D MarqueeEndPos;

protected:
	void AddBoxShape();
	void AddCircleShape();

	bool IsEditingGeometry() const;

	void SelectVerticesInMarquee(FEditorViewportClient* ViewportClient, FViewport* Viewport, bool bAddToSelection);

	void DrawMarquee(FViewport& InViewport, const FSceneView& View, FCanvas& Canvas, const FLinearColor& Color);
};
