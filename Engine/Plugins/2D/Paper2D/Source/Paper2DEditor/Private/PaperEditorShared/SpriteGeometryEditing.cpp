// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperEditorShared/SpriteGeometryEditing.h"
#include "GlobalRenderResources.h"
#include "Materials/Material.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "SpriteEditor/SpriteEditorSelections.h"
#include "GeomTools.h"
#include "DynamicMeshBuilder.h"
#include "UnrealClient.h"

#define LOCTEXT_NAMESPACE "PaperGeometryEditing"

//////////////////////////////////////////////////////////////////////////

namespace SpriteEditingConstantsEX
{
	// The length and color to draw a normal tick in
	const float GeometryNormalLength = 15.0f;
	const FLinearColor GeometryNormalColor(0.0f, 1.0f, 0.0f, 0.5f);
	
	// Geometry edit rendering stuff
	const int32 CircleShapeNumSides = 64;
	const float GeometryVertexSize = 8.0f;
	const float GeometryBorderLineThickness = 2.0f;
	const FLinearColor GeometrySelectedColor(FLinearColor::White);

	// Add polygon mode
	const float AddPolygonVertexWeldScreenSpaceDistance = 6.0f;
}

//////////////////////////////////////////////////////////////////////////
// FSpriteSelectionHelper

void FSpriteSelectionHelper::ClearSelectionSet()
{
	SelectedItemSet.Empty();
}

void FSpriteSelectionHelper::SelectItem(TSharedPtr<FSelectedItem> NewItem)
{
	SelectedItemSet.Add(NewItem);
}

bool FSpriteSelectionHelper::CanDeleteSelection() const
{
	return SelectedItemSet.Num() > 0;
}

//////////////////////////////////////////////////////////////////////////
// FSpriteSelectedShape

FSpriteSelectedShape::FSpriteSelectedShape(ISpriteSelectionContext* InEditorContext, FSpriteGeometryCollection& InGeometry, int32 InShapeIndex, bool bInIsBackground)
	: FSelectedItem(FSelectionTypes::GeometryShape)
	, EditorContext(InEditorContext)
	, Geometry(InGeometry)
	, ShapeIndex(InShapeIndex)
	, bIsBackground(bInIsBackground)
{
}

uint32 FSpriteSelectedShape::GetTypeHash() const
{
	return (ShapeIndex * 311);
}

EMouseCursor::Type FSpriteSelectedShape::GetMouseCursor() const
{
	return EMouseCursor::GrabHand;
}

bool FSpriteSelectedShape::Equals(const FSelectedItem& OtherItem) const
{
	if (OtherItem.IsA(FSelectionTypes::Vertex))
	{
		const FSpriteSelectedShape& V1 = *this;
		const FSpriteSelectedShape& V2 = *(FSpriteSelectedShape*)(&OtherItem);

		return (V1.ShapeIndex == V2.ShapeIndex) && (&(V1.Geometry) == &(V2.Geometry));
	}
	else
	{
		return false;
	}
}

bool FSpriteSelectedShape::IsBackgroundObject() const
{
	return bIsBackground;
}

void FSpriteSelectedShape::ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode)
{
	if (Geometry.Shapes.IsValidIndex(ShapeIndex))
	{
		FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];

		const bool bDoRotation = (MoveMode == UE::Widget::WM_Rotate) || (MoveMode == UE::Widget::WM_TranslateRotateZ);
		const bool bDoTranslation = (MoveMode == UE::Widget::WM_Translate) || (MoveMode == UE::Widget::WM_TranslateRotateZ);
		const bool bDoScale = MoveMode == UE::Widget::WM_Scale;

		if (bDoTranslation)
		{
			const FVector WorldSpaceDelta = (PaperAxisX * Delta.X) + (PaperAxisY * Delta.Y);
			const FVector2D TextureSpaceDelta = EditorContext->SelectedItemConvertWorldSpaceDeltaToLocalSpace(WorldSpaceDelta);

			Shape.BoxPosition += TextureSpaceDelta;

			Geometry.GeometryType = ESpritePolygonMode::FullyCustom;
		}

		if (bDoScale)
		{
			const float ScaleDeltaX = FVector::DotProduct(Scale3D, PaperAxisX);
			const float ScaleDeltaY = FVector::DotProduct(Scale3D, PaperAxisY);

			const FVector2D OldSize = Shape.BoxSize;
			const FVector2D NewSize(OldSize.X + ScaleDeltaX, OldSize.Y + ScaleDeltaY);

			if (!FMath::IsNearlyZero(NewSize.X, KINDA_SMALL_NUMBER) && !FMath::IsNearlyZero(NewSize.Y, KINDA_SMALL_NUMBER))
			{
				const FVector2D ScaleFactor(NewSize.X / OldSize.X, NewSize.Y / OldSize.Y);
				Shape.BoxSize = NewSize;

				// Now apply it to the verts
				for (FVector2D& Vertex : Shape.Vertices)
				{
					Vertex.X *= ScaleFactor.X;
					Vertex.Y *= ScaleFactor.Y;
				}

				Geometry.GeometryType = ESpritePolygonMode::FullyCustom;
			}
		}

		if (bDoRotation)
		{
			//@TODO: This stuff should probably be wrapped up into a utility method (also used for socket editing)
			const FRotator CurrentRot(Shape.Rotation, 0.0f, 0.0f);
			FRotator SocketWinding;
			FRotator SocketRotRemainder;
			CurrentRot.GetWindingAndRemainder(SocketWinding, SocketRotRemainder);

			const FQuat ActorQ = SocketRotRemainder.Quaternion();
			const FQuat DeltaQ = Rotation.Quaternion();
			const FQuat ResultQ = DeltaQ * ActorQ;
			const FRotator NewSocketRotRem = FRotator(ResultQ);
			FRotator DeltaRot = NewSocketRotRem - SocketRotRemainder;
			DeltaRot.Normalize();

			const FRotator NewRotation(CurrentRot + DeltaRot);

			Shape.Rotation = NewRotation.Pitch;
			Geometry.GeometryType = ESpritePolygonMode::FullyCustom;
		}
	}
}

FVector FSpriteSelectedShape::GetWorldPos() const
{
	FVector Result = FVector::ZeroVector;

	if (Geometry.Shapes.IsValidIndex(ShapeIndex))
	{
		const FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];

		switch (Shape.ShapeType)
		{
		case ESpriteShapeType::Box:
		case ESpriteShapeType::Circle:
			Result = EditorContext->TextureSpaceToWorldSpace(Shape.BoxPosition);
			break;
		case ESpriteShapeType::Polygon:
			// Average the vertex positions
			//@TODO: Eventually this will just be BoxPosition as well once the vertex positions are relative
			Result = EditorContext->TextureSpaceToWorldSpace(Shape.GetPolygonCentroid());
			break;
		default:
			check(false);
		}
	}

	return Result;
}

//////////////////////////////////////////////////////////////////////////
// FSpriteGeometryEditingHelper

FSpriteGeometryEditingHelper::FSpriteGeometryEditingHelper(ISpriteSelectionContext* InEditorContext)
	: EditorContext(InEditorContext)
	, GeometryBeingEdited(nullptr)
	, bIsAddingPolygon(false)
	, AddingPolygonIndex(INDEX_NONE)
	, bShowNormals(true)
	, bAllowSubtractivePolygons(false)
	, bAllowCircles(true)
{
	WidgetVertexColorMaterial = (UMaterial*)StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/WidgetVertexColorMaterial.WidgetVertexColorMaterial"), NULL, LOAD_None, NULL);
}

void FSpriteGeometryEditingHelper::ClearSelectionSet()
{
	FSpriteSelectionHelper::ClearSelectionSet();
	SelectedIDSet.Empty();

	if (bIsAddingPolygon)
	{
		ResetAddPolygonMode();
	}

	EditorContext->InvalidateViewportAndHitProxies();
}

void FSpriteGeometryEditingHelper::DeleteSelectedItems()
{
	// Determine which vertices or entire shapes should be deleted
	TSet<FShapeVertexPair> CompositeIndicesSet;
	TSet<int32> ShapesToDeleteSet;

	if (IsEditingGeometry())
	{
		FSpriteGeometryCollection& Geometry = GetGeometryChecked();

		for (TSharedPtr<FSelectedItem> SelectionIt : GetSelectionSet())
		{
			if (const FSpriteSelectedVertex* SelectedVertex = SelectionIt->CastTo<const FSpriteSelectedVertex>(FSelectionTypes::Vertex))
			{
				CompositeIndicesSet.Add(FShapeVertexPair(SelectedVertex->ShapeIndex, SelectedVertex->VertexIndex));

				if (SelectedVertex->IsA(FSelectionTypes::Edge)) // add the "next" point for the edge
				{
					const int32 NextIndex = (SelectedVertex->VertexIndex + 1) % Geometry.Shapes[SelectedVertex->ShapeIndex].Vertices.Num();
					CompositeIndicesSet.Add(FShapeVertexPair(SelectedVertex->ShapeIndex, NextIndex));
				}
			}
			else if (const FSpriteSelectedShape* SelectedShape = SelectionIt->CastTo<const FSpriteSelectedShape>(FSelectionTypes::GeometryShape))
			{
				ShapesToDeleteSet.Add(SelectedShape->ShapeIndex);
			}
		}
	}

	// See if anything else can be deleted
	bool bCanDeleteNonGeometry = false;
	for (const TSharedPtr<FSelectedItem>& SelectedItem : GetSelectionSet())
	{
		if (SelectedItem->CanBeDeleted())
		{
			bCanDeleteNonGeometry = true;
			break;
		}
	}

	// Now delete the stuff that was selected in the correct order so that indices aren't messed up
	const bool bDeletingGeometry = (CompositeIndicesSet.Num() > 0) || (ShapesToDeleteSet.Num() > 0);
	if (bDeletingGeometry || bCanDeleteNonGeometry)
	{
		EditorContext->BeginTransaction(LOCTEXT("DeleteSelectionTransaction", "Delete Selection"));
		EditorContext->MarkTransactionAsDirty();

		if (bDeletingGeometry)
		{
			FSpriteGeometryCollection& Geometry = GetGeometryChecked();

			// Delete the selected vertices first, as they may cause entire shapes to need to be deleted (sort so we delete from the back first)
			TArray<FShapeVertexPair> CompositeIndices = CompositeIndicesSet.Array();
			CompositeIndices.Sort([](const FShapeVertexPair& A, const FShapeVertexPair& B) { return (A.VertexIndex > B.VertexIndex); });
			for (const FShapeVertexPair& Composite : CompositeIndices)
			{
				const int32 ShapeIndex = Composite.ShapeIndex;
				const int32 VertexIndex = Composite.VertexIndex;
				if (DeleteVertexInPolygonInternal(Geometry, ShapeIndex, VertexIndex))
				{
					ShapesToDeleteSet.Add(ShapeIndex);
				}
			}

			// Delete the selected shapes (plus any shapes that became empty due to selected vertices)
			if (ShapesToDeleteSet.Num() > 0)
			{
				// Sort so we delete from the back first
				TArray<int32> ShapesToDeleteIndicies = ShapesToDeleteSet.Array();
				ShapesToDeleteIndicies.Sort([](const int32& A, const int32& B) { return (A > B); });
				for (const int32 ShapeToDeleteIndex : ShapesToDeleteIndicies)
				{
					Geometry.Shapes.RemoveAt(ShapeToDeleteIndex);
				}
			}

			Geometry.GeometryType = ESpritePolygonMode::FullyCustom;
		}

		// Delete everything else
		if (bCanDeleteNonGeometry)
		{
			for (TSharedPtr<FSelectedItem> SelectedItem : GetSelectionSet())
			{
				if (SelectedItem->CanBeDeleted())
				{
					SelectedItem->DeleteThisItem();
				}
			}
		}

		EditorContext->EndTransaction();
	}

	ClearSelectionSet();
	ResetAddPolygonMode();
}

void FSpriteGeometryEditingHelper::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(WidgetVertexColorMaterial);
}

void FSpriteGeometryEditingHelper::SetEditorContext(class ISpriteSelectionContext* InNewEditorContext)
{
	EditorContext = InNewEditorContext;
}

class ISpriteSelectionContext* FSpriteGeometryEditingHelper::GetEditorContext() const
{
	return EditorContext;
}

void FSpriteGeometryEditingHelper::DrawGeometry(const FSceneView& View, FPrimitiveDrawInterface& PDI, const FLinearColor& GeometryVertexColor, const FLinearColor& NegativeGeometryVertexColor)
{
	if (GeometryBeingEdited == nullptr)
	{
		return;
	}

	FSpriteGeometryCollection& Geometry = GetGeometryChecked();
	const bool bIsHitTesting = PDI.IsHitTesting();
	const float UnitsPerPixel = EditorContext->SelectedItemGetUnitsPerPixel();

	// Run thru the geometry shapes and draw hit proxies for them
	for (int32 ShapeIndex = 0; ShapeIndex < Geometry.Shapes.Num(); ++ShapeIndex)
	{
		const FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];

		const bool bIsShapeSelected = IsGeometrySelected(FShapeVertexPair(ShapeIndex, INDEX_NONE));
		const FLinearColor LineColorRaw = Shape.bNegativeWinding ? NegativeGeometryVertexColor : GeometryVertexColor;
		const FLinearColor VertexColor = Shape.bNegativeWinding ? NegativeGeometryVertexColor : GeometryVertexColor;

		const FLinearColor LineColor = Shape.IsShapeValid() ? LineColorRaw : FMath::Lerp(LineColorRaw, FLinearColor::Red, 0.8f);

		// Draw the interior (allowing selection of the whole shape)
		if (bIsHitTesting)
		{
			TSharedPtr<FSpriteSelectedShape> Data = MakeShareable(new FSpriteSelectedShape(EditorContext, Geometry, ShapeIndex, /*bIsBackground=*/ true));
			PDI.SetHitProxy(new HSpriteSelectableObjectHitProxy(Data));
		}

		FColor BackgroundColor(( bIsShapeSelected ? SpriteEditingConstantsEX::GeometrySelectedColor : LineColor ).ToFColor(true));
		BackgroundColor.A = 4;
		FMaterialRenderProxy* ShapeMaterialProxy = WidgetVertexColorMaterial->GetRenderProxy();

		if (Shape.ShapeType == ESpriteShapeType::Circle)
		{
			//@TODO: This is going to have issues if we ever support ellipses
			const FVector2D PixelSpaceRadius = Shape.BoxSize * 0.5f;
			const float WorldSpaceRadius = PixelSpaceRadius.X * UnitsPerPixel;

			const FVector CircleCenterWorldPos = EditorContext->TextureSpaceToWorldSpace(Shape.BoxPosition);

			DrawDisc(&PDI, CircleCenterWorldPos, PaperAxisX, PaperAxisY, BackgroundColor, WorldSpaceRadius, SpriteEditingConstantsEX::CircleShapeNumSides, ShapeMaterialProxy, SDPG_Foreground);
		}
		else
		{
			TArray<FVector2D> SourceTextureSpaceVertices;
			Shape.GetTextureSpaceVertices(/*out*/ SourceTextureSpaceVertices);

			TArray<FVector2D> TriangulatedPolygonVertices;
			FGeomTools2D::TriangulatePoly(/*out*/ TriangulatedPolygonVertices, SourceTextureSpaceVertices, /*bKeepColinearVertices=*/ true);

			if (((TriangulatedPolygonVertices.Num() % 3) == 0) && (TriangulatedPolygonVertices.Num() > 0))
			{
				FDynamicMeshBuilder MeshBuilder(View.GetFeatureLevel());

				FDynamicMeshVertex MeshVertex;
				MeshVertex.Color = BackgroundColor;
				MeshVertex.TextureCoordinate[0] = FVector2f::ZeroVector;
				MeshVertex.SetTangents((FVector3f)PaperAxisX, (FVector3f)PaperAxisY, (FVector3f)PaperAxisZ);

				for (const FVector2D& SrcTriangleVertex : TriangulatedPolygonVertices)
				{
					MeshVertex.Position = (FVector3f)EditorContext->TextureSpaceToWorldSpace(SrcTriangleVertex);
					MeshBuilder.AddVertex(MeshVertex);
				}

				for (int32 TriangleIndex = 0; TriangleIndex < TriangulatedPolygonVertices.Num(); TriangleIndex += 3)
				{
					MeshBuilder.AddTriangle(TriangleIndex, TriangleIndex + 1, TriangleIndex + 2);
				}

				MeshBuilder.Draw(&PDI, FMatrix::Identity, ShapeMaterialProxy, SDPG_Foreground);
			}
		}

		if (bIsHitTesting)
		{
			PDI.SetHitProxy(nullptr);
		}
	}
}

void FSpriteGeometryEditingHelper::DrawGeometry_CanvasPass(FViewport& InViewport, const FSceneView& View, FCanvas& Canvas, /*inout*/ int32& YPos, const FLinearColor& GeometryVertexColor, const FLinearColor& NegativeGeometryVertexColor)
{
	if (GeometryBeingEdited == nullptr)
	{
		return;
	}

	// Calculate the texture-space position of the mouse
	const FVector MousePositionWorldSpace = View.PixelToWorld(InViewport.GetMouseX(), InViewport.GetMouseY(), 0);
	const FVector2D MousePositionTextureSpace = EditorContext->WorldSpaceToTextureSpace(MousePositionWorldSpace);

	//@TODO: Move all of the line drawing to the PDI pass
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();

	// Display tool help
	{
		static const FText GeomHelpStr = LOCTEXT("GeomEditHelp", "Shift + click to insert a vertex.\nSelect one or more vertices and press Delete to remove them.\nDouble click a vertex to select a polygon\n");
		static const FText GeomClickAddPolygon_NoSubtractive = LOCTEXT("GeomClickAddPolygon_NoSubtractive", "Click to start creating a polygon\n");
		static const FText GeomClickAddPolygon_AllowSubtractive = LOCTEXT("GeomClickAddPolygon_AllowSubtractive", "Click to start creating a polygon\nCtrl + Click to start creating a subtractive polygon\n");
		static const FText GeomAddVerticesHelpStr = LOCTEXT("GeomClickAddVertices", "Click to add points to the polygon\nDouble-click to add a point and close the shape\nClick again on the first point or press Enter to close the shape\nPress Backspace to remove the last added point or Escape to remove the shape\n");
		FLinearColor ToolTextColor = FLinearColor::White;

		const FText* HelpStr;
		if (IsAddingPolygon())
		{
			if (AddingPolygonIndex == INDEX_NONE)
			{
				HelpStr = bAllowSubtractivePolygons ? &GeomClickAddPolygon_AllowSubtractive : &GeomClickAddPolygon_NoSubtractive;
			}
			else
			{
				HelpStr = &GeomAddVerticesHelpStr;
			}
			ToolTextColor = FLinearColor::Yellow;
		}
		else
		{
			HelpStr = &GeomHelpStr;
		}

		FCanvasTextItem TextItem(FVector2D(6, YPos), *HelpStr, GEngine->GetSmallFont(), ToolTextColor);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Draw(&Canvas);
		YPos += 54;
	}

	const bool bIsHitTesting = Canvas.IsHitTesting();

	// Run thru the geometry shapes and draw hit proxies for them
	for (int32 ShapeIndex = 0; ShapeIndex < Geometry.Shapes.Num(); ++ShapeIndex)
	{
		const FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];

		const bool bIsShapeSelected = IsGeometrySelected(FShapeVertexPair(ShapeIndex, INDEX_NONE));
		const FLinearColor LineColorRaw = Shape.bNegativeWinding ? NegativeGeometryVertexColor : GeometryVertexColor;
		const FLinearColor VertexColor = Shape.bNegativeWinding ? NegativeGeometryVertexColor : GeometryVertexColor;

		const FLinearColor LineColor = Shape.IsShapeValid() ? LineColorRaw : FMath::Lerp(LineColorRaw, FLinearColor::Red, 0.8f);

		// Draw the circle shape if necessary
		if (Shape.ShapeType == ESpriteShapeType::Circle)
		{
			if (bIsHitTesting)
			{
				TSharedPtr<FSpriteSelectedShape> Data = MakeShareable(new FSpriteSelectedShape(EditorContext, Geometry, ShapeIndex, /*bIsBackground=*/ false));
				Canvas.SetHitProxy(new HSpriteSelectableObjectHitProxy(Data));
			}

			// Draw the circle
			const float RadiusX = Shape.BoxSize.X * 0.5f;
			const float RadiusY = Shape.BoxSize.Y * 0.5f;

			const float	AngleDelta = 2.0f * PI / SpriteEditingConstantsEX::CircleShapeNumSides;

			const float LastX = Shape.BoxPosition.X + RadiusX;
			const float LastY = Shape.BoxPosition.Y;
			FVector2D LastVertexPos = TextureSpaceToScreenSpace(View, FVector2D(LastX, LastY));


			for (int32 SideIndex = 0; SideIndex < SpriteEditingConstantsEX::CircleShapeNumSides; SideIndex++)
			{
				const float X = Shape.BoxPosition.X + RadiusX * FMath::Cos(AngleDelta * (SideIndex + 1));
				const float Y = Shape.BoxPosition.Y + RadiusY * FMath::Sin(AngleDelta * (SideIndex + 1));
				const FVector2D ScreenPos = TextureSpaceToScreenSpace(View, FVector2D(X, Y));

				FCanvasLineItem LineItem(LastVertexPos, ScreenPos);
				LineItem.SetColor(bIsShapeSelected ? SpriteEditingConstantsEX::GeometrySelectedColor : LineColor);
				LineItem.LineThickness = SpriteEditingConstantsEX::GeometryBorderLineThickness;

				Canvas.DrawItem(LineItem);

				LastVertexPos = ScreenPos;
			}


			if (bIsHitTesting)
			{
				Canvas.SetHitProxy(nullptr);
			}
		}

		// Draw lines connecting the vertices of the shape
		for (int32 VertexIndex = 0; VertexIndex < Shape.Vertices.Num(); ++VertexIndex)
		{
			const int32 NextVertexIndex = (VertexIndex + 1) % Shape.Vertices.Num();

			const FVector2D ScreenPos = TextureSpaceToScreenSpace(View, Shape.ConvertShapeSpaceToTextureSpace(Shape.Vertices[VertexIndex]));
			const FVector2D NextScreenPos = TextureSpaceToScreenSpace(View, Shape.ConvertShapeSpaceToTextureSpace(Shape.Vertices[NextVertexIndex]));

			const bool bIsThisVertexSelected = IsGeometrySelected(FShapeVertexPair(ShapeIndex, VertexIndex));
			const bool bIsNextVertexSelected = IsGeometrySelected(FShapeVertexPair(ShapeIndex, NextVertexIndex));

			const bool bIsEdgeSelected = bIsShapeSelected || (bIsThisVertexSelected && bIsNextVertexSelected);

			// Draw the normal tick
			if (bShowNormals)
			{
				const FVector2D Direction = (NextScreenPos - ScreenPos).GetSafeNormal();
				const FVector2D Normal = FVector2D(-Direction.Y, Direction.X);

				const FVector2D Midpoint = (ScreenPos + NextScreenPos) * 0.5f;
				const FVector2D NormalPoint = Midpoint - Normal * SpriteEditingConstantsEX::GeometryNormalLength;
				FCanvasLineItem LineItem(Midpoint, NormalPoint);
				LineItem.SetColor(SpriteEditingConstantsEX::GeometryNormalColor);

				Canvas.DrawItem(LineItem);
			}

			// Draw the edge
			{
				if (bIsHitTesting)
				{
					TSharedPtr<FSpriteSelectedEdge> Data = MakeShareable(new FSpriteSelectedEdge(EditorContext, Geometry, ShapeIndex, VertexIndex));
					Canvas.SetHitProxy(new HSpriteSelectableObjectHitProxy(Data));
				}

				FCanvasLineItem LineItem(ScreenPos, NextScreenPos);
				LineItem.SetColor(bIsEdgeSelected ? SpriteEditingConstantsEX::GeometrySelectedColor : LineColor);
				LineItem.LineThickness = SpriteEditingConstantsEX::GeometryBorderLineThickness;
				Canvas.DrawItem(LineItem);

				if (bIsHitTesting)
				{
					Canvas.SetHitProxy(nullptr);
				}
			}
		}

		// Draw the vertices
		for (int32 VertexIndex = 0; VertexIndex < Shape.Vertices.Num(); ++VertexIndex)
		{
			const FVector2D ScreenPos = TextureSpaceToScreenSpace(View, Shape.ConvertShapeSpaceToTextureSpace(Shape.Vertices[VertexIndex]));
			const float X = ScreenPos.X;
			const float Y = ScreenPos.Y;

			const bool bIsVertexSelected = IsGeometrySelected(FShapeVertexPair(ShapeIndex, VertexIndex));
			const bool bIsVertexLastAdded = IsAddingPolygon() && (AddingPolygonIndex == ShapeIndex) && (VertexIndex == Shape.Vertices.Num() - 1);
			const bool bNeedHighlightVertex = bIsShapeSelected || bIsVertexSelected || bIsVertexLastAdded;

			if (bIsHitTesting)
			{
				TSharedPtr<FSpriteSelectedVertex> Data = MakeShareable(new FSpriteSelectedVertex(EditorContext, Geometry, ShapeIndex, VertexIndex));
				Canvas.SetHitProxy(new HSpriteSelectableObjectHitProxy(Data));
			}

			const float VertSize = SpriteEditingConstantsEX::GeometryVertexSize;
			Canvas.DrawTile(ScreenPos.X - VertSize*0.5f, ScreenPos.Y - VertSize*0.5f, VertSize, VertSize, 0.f, 0.f, 1.f, 1.f, bNeedHighlightVertex ? SpriteEditingConstantsEX::GeometrySelectedColor : VertexColor, GWhiteTexture);

			if (bIsHitTesting)
			{
				Canvas.SetHitProxy(nullptr);
			}
		}
	}

	// Draw a preview cursor for the add polygon tool
	if (IsAddingPolygon())
	{
		// Figure out where the mouse is back in screen space
		const FVector2D PotentialVertexScreenPos = TextureSpaceToScreenSpace(View, MousePositionTextureSpace);

		bool bWillCloseByClicking = false;
		if (Geometry.Shapes.IsValidIndex(AddingPolygonIndex))
		{
			const FSpriteGeometryShape& Shape = Geometry.Shapes[AddingPolygonIndex];

			const FLinearColor LineColorRaw = Shape.bNegativeWinding ? NegativeGeometryVertexColor : GeometryVertexColor;
			const FLinearColor LineColorValidity = Shape.IsShapeValid() ? LineColorRaw : FMath::Lerp(LineColorRaw, FLinearColor::Red, 0.8f);
			const FLinearColor LineColor = FMath::Lerp(LineColorValidity, SpriteEditingConstantsEX::GeometrySelectedColor, 0.2f);

			if (Shape.Vertices.Num() > 0)
			{
				// Draw a line from the last vertex to the potential insertion point for the new one
				{
					const FVector2D LastScreenPos = TextureSpaceToScreenSpace(View, Shape.ConvertShapeSpaceToTextureSpace(Shape.Vertices[Shape.Vertices.Num() - 1]));

					FCanvasLineItem LineItem(LastScreenPos, PotentialVertexScreenPos);
					LineItem.SetColor(LineColor);
					LineItem.LineThickness = SpriteEditingConstantsEX::GeometryBorderLineThickness;
					Canvas.DrawItem(LineItem);
				}

				// And to the first vertex if there were at least 2
				if (Shape.Vertices.Num() >= 2)
				{
					const FVector2D FirstScreenPos = TextureSpaceToScreenSpace(View, Shape.ConvertShapeSpaceToTextureSpace(Shape.Vertices[0]));

					FCanvasLineItem LineItem(PotentialVertexScreenPos, FirstScreenPos);
					LineItem.SetColor(LineColor);
					LineItem.LineThickness = SpriteEditingConstantsEX::GeometryBorderLineThickness;
					Canvas.DrawItem(LineItem);

					// Determine how close we are to the first vertex (will we close the shape by clicking)?
					bWillCloseByClicking = (Shape.Vertices.Num() >= 3) && (FVector2D::Distance(FirstScreenPos, PotentialVertexScreenPos) < SpriteEditingConstantsEX::AddPolygonVertexWeldScreenSpaceDistance);
				}
			}
		}

		// Draw the prospective vert
		const float VertSize = SpriteEditingConstantsEX::GeometryVertexSize;
		Canvas.DrawTile(PotentialVertexScreenPos.X - VertSize*0.5f, PotentialVertexScreenPos.Y - VertSize*0.5f, VertSize, VertSize, 0.f, 0.f, 1.f, 1.f, SpriteEditingConstantsEX::GeometrySelectedColor, GWhiteTexture);

		// Draw a prompt above and to the right of the cursor
		static const FText CloseButton(LOCTEXT("ClosePolygonPrompt", "Close"));
		static const FText AddButton(LOCTEXT("AddVertexToPolygonPrompt", "+"));

		const FText PromptText(bWillCloseByClicking ? CloseButton : AddButton);
		FCanvasTextItem PromptTextItem(FVector2D(PotentialVertexScreenPos.X + VertSize, PotentialVertexScreenPos.Y - VertSize), PromptText, GEngine->GetSmallFont(), FLinearColor::White);
		PromptTextItem.EnableShadow(FLinearColor::Black);
		PromptTextItem.Draw(&Canvas);
	}
}

void FSpriteGeometryEditingHelper::AddPointToGeometry(const FVector2D& TextureSpacePoint, const int32 SelectedPolygonIndex)
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();

	int32 ClosestShapeIndex = INDEX_NONE;
	int32 ClosestVertexInsertIndex = INDEX_NONE;
	float ClosestDistanceSquared = MAX_FLT;

	int32 StartPolygonIndex = 0;
	int32 EndPolygonIndex = Geometry.Shapes.Num();
	if ((SelectedPolygonIndex >= 0) && (SelectedPolygonIndex < Geometry.Shapes.Num()))
	{
		StartPolygonIndex = SelectedPolygonIndex;
		EndPolygonIndex = SelectedPolygonIndex + 1;
	}

	// Determine where we should insert the vertex
	for (int32 PolygonIndex = StartPolygonIndex; PolygonIndex < EndPolygonIndex; ++PolygonIndex)
	{
		FSpriteGeometryShape& Polygon = Geometry.Shapes[PolygonIndex];
		if (Polygon.Vertices.Num() >= 3)
		{
			for (int32 VertexIndex = 0; VertexIndex < Polygon.Vertices.Num(); ++VertexIndex)
			{
				FVector2D ClosestPoint;
				const FVector2D LineStart = Polygon.ConvertShapeSpaceToTextureSpace(Polygon.Vertices[VertexIndex]);
				int32 NextVertexIndex = (VertexIndex + 1) % Polygon.Vertices.Num();
				const FVector2D LineEnd = Polygon.ConvertShapeSpaceToTextureSpace(Polygon.Vertices[NextVertexIndex]);
				if (ClosestPointOnLine(TextureSpacePoint, LineStart, LineEnd, /*out*/ ClosestPoint))
				{
					const float CurrentDistanceSquared = FVector2D::DistSquared(ClosestPoint, TextureSpacePoint);
					if (CurrentDistanceSquared < ClosestDistanceSquared)
					{
						ClosestShapeIndex = PolygonIndex;
						ClosestDistanceSquared = CurrentDistanceSquared;
						ClosestVertexInsertIndex = NextVertexIndex;
					}
				}
			}
		}
		else
		{
			// Simply insert the vertex
			for (int32 VertexIndex = 0; VertexIndex < Polygon.Vertices.Num(); ++VertexIndex)
			{
				const FVector2D CurrentVertexTS = Polygon.ConvertShapeSpaceToTextureSpace(Polygon.Vertices[VertexIndex]);
				const float CurrentDistanceSquared = FVector2D::DistSquared(CurrentVertexTS, TextureSpacePoint);
				if (CurrentDistanceSquared < ClosestDistanceSquared)
				{
					ClosestShapeIndex = PolygonIndex;
					ClosestDistanceSquared = CurrentDistanceSquared;
					ClosestVertexInsertIndex = VertexIndex + 1;
				}
			}
		}
	}

	if ((ClosestVertexInsertIndex != INDEX_NONE) && (ClosestShapeIndex != INDEX_NONE))
	{
		FSpriteGeometryShape& Shape = Geometry.Shapes[ClosestShapeIndex];
		if (Shape.ShapeType != ESpriteShapeType::Circle)
		{
			EditorContext->BeginTransaction(LOCTEXT("AddPolygonVertexTransaction", "Add Vertex to Polygon"));
			Shape.Vertices.Insert(Shape.ConvertTextureSpaceToShapeSpace(TextureSpacePoint), ClosestVertexInsertIndex);
			Shape.ShapeType = ESpriteShapeType::Polygon;
			Geometry.GeometryType = ESpritePolygonMode::FullyCustom;
			EditorContext->MarkTransactionAsDirty();
			EditorContext->EndTransaction();

			// Select this vertex
			ClearSelectionSet();
			AddPolygonVertexToSelection(ClosestShapeIndex, ClosestVertexInsertIndex);
		}
	}
}

void FSpriteGeometryEditingHelper::SelectGeometry(const FShapeVertexPair& GeometyItem)
{
	SelectedIDSet.Add(GeometyItem);
	EditorContext->InvalidateViewportAndHitProxies();
}

bool FSpriteGeometryEditingHelper::IsGeometrySelected(const FShapeVertexPair& GeometyItem) const
{
	return SelectedIDSet.Contains(GeometyItem);
}

void FSpriteGeometryEditingHelper::AddShapeToSelection(const int32 ShapeIndex)
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();
	if (Geometry.Shapes.IsValidIndex(ShapeIndex))
	{
		if (!IsGeometrySelected(FShapeVertexPair(ShapeIndex, INDEX_NONE)))
		{
			FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];

			TSharedPtr<FSpriteSelectedShape> SelectedShape = MakeShareable(new FSpriteSelectedShape(EditorContext, Geometry, ShapeIndex));

			SelectItem(SelectedShape);
			SelectGeometry(FShapeVertexPair(ShapeIndex, INDEX_NONE));
		}
	}
}

void FSpriteGeometryEditingHelper::AddPolygonVertexToSelection(const int32 ShapeIndex, const int32 VertexIndex)
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();
	if (Geometry.Shapes.IsValidIndex(ShapeIndex))
	{
		FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];
		if (Shape.Vertices.IsValidIndex(VertexIndex))
		{
			if (!IsGeometrySelected(FShapeVertexPair(ShapeIndex, VertexIndex)))
			{
				TSharedPtr<FSpriteSelectedVertex> Vertex = MakeShareable(new FSpriteSelectedVertex(EditorContext, Geometry, ShapeIndex, VertexIndex));

				SelectItem(Vertex);
				SelectGeometry(FShapeVertexPair(ShapeIndex, VertexIndex));
			}
		}
	}
}

void FSpriteGeometryEditingHelper::AddPolygonEdgeToSelection(const int32 ShapeIndex, const int32 FirstVertexIndex)
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();
	if (Geometry.Shapes.IsValidIndex(ShapeIndex))
	{
		FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];
		const int32 NextVertexIndex = (FirstVertexIndex + 1) % Shape.Vertices.Num();

		AddPolygonVertexToSelection(ShapeIndex, FirstVertexIndex);
		AddPolygonVertexToSelection(ShapeIndex, NextVertexIndex);
	}
}

void FSpriteGeometryEditingHelper::SetShowNormals(bool bShouldShowNormals)
{
	bShowNormals = bShouldShowNormals;
	EditorContext->InvalidateViewportAndHitProxies();
}

void FSpriteGeometryEditingHelper::ToggleShowNormals()
{
	SetShowNormals(!bShowNormals);
}

void FSpriteGeometryEditingHelper::SetGeometryBeingEdited(FSpriteGeometryCollection* NewGeometryBeingEdited, bool bInAllowCircles, bool bInAllowSubtractivePolygons)
{
	ClearSelectionSet();
	GeometryBeingEdited = NewGeometryBeingEdited;
	bAllowCircles = bInAllowCircles;
	bAllowSubtractivePolygons = bInAllowSubtractivePolygons;
}

FSpriteGeometryCollection* FSpriteGeometryEditingHelper::GetGeometryBeingEdited() const
{
	return GeometryBeingEdited;
}

void FSpriteGeometryEditingHelper::AddNewCircleShape(const FVector2D& CircleLocation, float Radius)
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();
	EditorContext->BeginTransaction(LOCTEXT("AddCircleShapeTransaction", "Add Circle Shape"));

	// Create the new shape
	const FVector2D CircleSize2D(Radius*2.0f, Radius*2.0f);
	Geometry.AddCircleShape(CircleLocation, CircleSize2D);
	Geometry.GeometryType = ESpritePolygonMode::FullyCustom;

	// Select the new shape
	ClearSelectionSet();
	AddShapeToSelection(Geometry.Shapes.Num() - 1);

	EditorContext->MarkTransactionAsDirty();
	EditorContext->EndTransaction();
}

void FSpriteGeometryEditingHelper::AddNewBoxShape(const FVector2D& BoxLocation, const FVector2D& BoxSize)
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();
	EditorContext->BeginTransaction(LOCTEXT("AddBoxShapeTransaction", "Add Box Shape"));

	// Create the new shape
	Geometry.AddRectangleShape(BoxLocation, BoxSize);
	Geometry.GeometryType = ESpritePolygonMode::FullyCustom;

	// Select the new shape
	ClearSelectionSet();
	AddShapeToSelection(Geometry.Shapes.Num() - 1);

	EditorContext->MarkTransactionAsDirty();
	EditorContext->EndTransaction();
}

void FSpriteGeometryEditingHelper::ResetAddPolygonMode()
{
	if (bIsAddingPolygon)
	{
		bIsAddingPolygon = false;
		if (AddingPolygonIndex != INDEX_NONE)
		{
			ClearSelectionSet();
			AddShapeToSelection(AddingPolygonIndex);
		}
	}
}

void FSpriteGeometryEditingHelper::ToggleAddPolygonMode()
{
	if (IsAddingPolygon())
	{
		ResetAddPolygonMode();
	}
	else
	{
		ClearSelectionSet();

		bIsAddingPolygon = true;
		AddingPolygonIndex = INDEX_NONE;
		EditorContext->InvalidateViewportAndHitProxies();
	}
}

void FSpriteGeometryEditingHelper::AbandonAddPolygonMode()
{
	check(bIsAddingPolygon);

	FSpriteGeometryCollection& Geometry = GetGeometryChecked();

	if ((AddingPolygonIndex != INDEX_NONE) && Geometry.Shapes.IsValidIndex(AddingPolygonIndex))
	{
		EditorContext->BeginTransaction(LOCTEXT("DeletePolygonTransaction", "Delete Polygon"));

		Geometry.Shapes.RemoveAt(AddingPolygonIndex);

		EditorContext->MarkTransactionAsDirty();
		EditorContext->EndTransaction();
	}

	ResetAddPolygonMode();
}

void FSpriteGeometryEditingHelper::SnapAllVerticesToPixelGrid()
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();

	EditorContext->BeginTransaction(LOCTEXT("SnapAllVertsToPixelGridTransaction", "Snap All Verts to Pixel Grid"));

	for (FSpriteGeometryShape& Shape : Geometry.Shapes)
	{
		EditorContext->MarkTransactionAsDirty();

		Shape.BoxPosition.X = FMath::RoundToInt(Shape.BoxPosition.X);
		Shape.BoxPosition.Y = FMath::RoundToInt(Shape.BoxPosition.Y);

		if ((Shape.ShapeType == ESpriteShapeType::Box) || (Shape.ShapeType == ESpriteShapeType::Circle))
		{
			//@TODO: Should we snap BoxPosition also, or just the verts?
			const FVector2D OldHalfSize = Shape.BoxSize * 0.5f;
			FVector2D TopLeft = Shape.BoxPosition - OldHalfSize;
			FVector2D BottomRight = Shape.BoxPosition + OldHalfSize;
			TopLeft.X = FMath::RoundToInt(TopLeft.X);
			TopLeft.Y = FMath::RoundToInt(TopLeft.Y);
			BottomRight.X = FMath::RoundToInt(BottomRight.X);
			BottomRight.Y = FMath::RoundToInt(BottomRight.Y);
			Shape.BoxPosition = (TopLeft + BottomRight) * 0.5f;
			Shape.BoxSize = BottomRight - TopLeft;
		}

		for (FVector2D& Vertex : Shape.Vertices)
		{
			FVector2D TextureSpaceVertex = Shape.ConvertShapeSpaceToTextureSpace(Vertex);
			TextureSpaceVertex.X = FMath::RoundToInt(TextureSpaceVertex.X);
			TextureSpaceVertex.Y = FMath::RoundToInt(TextureSpaceVertex.Y);
			Vertex = Shape.ConvertTextureSpaceToShapeSpace(TextureSpaceVertex);
		}
	}

	EditorContext->EndTransaction();
}

void FSpriteGeometryEditingHelper::HandleAddPolygonClick(const FVector2D& TexturePoint, bool bWantsSubtractive, const FSceneView& View, EInputEvent Event)
{
	FSpriteGeometryCollection& Geometry = GetGeometryChecked();

	// Determine what the action is
	bool bCloseShape = false;
	bool bAddVertex = Event == IE_Pressed;

	// When we've already got at least one vertex in the shape, we disallow identical clicks and eventually allow closing the shape via various means
	if ((AddingPolygonIndex != INDEX_NONE) && Geometry.Shapes.IsValidIndex(AddingPolygonIndex))
	{
		const FSpriteGeometryShape& Shape = Geometry.Shapes[AddingPolygonIndex];

		// See if we're allowed to close the shape yet
		if (Shape.Vertices.Num() >= 3)
		{
			if (Event == IE_DoubleClick)
			{
				// Double-clicking with enough verts to form a polygon will close the shape
				bCloseShape = true;
			}
			else if (Event == IE_Pressed)
			{
				// Clicking on the starting vertex if we're close enough and have enough points will finish the shape
				const FVector2D ClickScreenPos = TextureSpaceToScreenSpace(View, TexturePoint);
				const FVector2D StartingScreenPos = TextureSpaceToScreenSpace(View, Shape.ConvertShapeSpaceToTextureSpace(Shape.Vertices[0]));
				bCloseShape = FVector2D::Distance(StartingScreenPos, ClickScreenPos) < SpriteEditingConstantsEX::AddPolygonVertexWeldScreenSpaceDistance;
			}
		}

		// Prevent adding if we're closing a shape
		if (bCloseShape)
		{
			bAddVertex = false;
		}

		// Prevent adding if we're really close to an existing vertex in the shape
		if (bAddVertex)
		{
			for (const FVector2D& ExistingShapeSpaceVertex : Shape.Vertices)
			{
				const FVector2D ExistingTextureSpaceVertex = Shape.ConvertShapeSpaceToTextureSpace(ExistingShapeSpaceVertex);
				const float DistanceToExisting = FVector2D::Distance(ExistingTextureSpaceVertex, TexturePoint);
				if (DistanceToExisting < 0.25f)
				{
					bAddVertex = false;
					break;
				}
			}
		}
	}

	if (bCloseShape)
	{
		ResetAddPolygonMode();
	}
	else if (bAddVertex)
	{
		EditorContext->BeginTransaction(LOCTEXT("AddPolygonVertexTransaction", "Add Vertex to Polygon"));

		if (AddingPolygonIndex == INDEX_NONE)
		{
			FSpriteGeometryShape NewPolygon;
			NewPolygon.ShapeType = ESpriteShapeType::Polygon;
			NewPolygon.bNegativeWinding = CanAddSubtractivePolygon() && bWantsSubtractive;

			Geometry.Shapes.Add(NewPolygon);
			AddingPolygonIndex = Geometry.Shapes.Num() - 1;
		}

		if (Geometry.Shapes.IsValidIndex(AddingPolygonIndex))
		{
			FSpriteGeometryShape& Shape = Geometry.Shapes[AddingPolygonIndex];
			Shape.ShapeType = ESpriteShapeType::Polygon;
			Shape.Vertices.Add(Shape.ConvertTextureSpaceToShapeSpace(TexturePoint));

			// Reorder the vertices when a triangle is first made to make sure the winding is facing outwards
			// After that it is up to the user to add verts in the order they want
			if (Shape.Vertices.Num() == 3)
			{
				FVector2D& A = Shape.Vertices[0];
				FVector2D& B = Shape.Vertices[1];
				FVector2D& C = Shape.Vertices[2];

				if (FVector2D::CrossProduct(B - A, C - A) < 0.0f)
				{
					Swap(B, C);
				}
			}
		}
		else
		{
			ResetAddPolygonMode();
		}

		Geometry.GeometryType = ESpritePolygonMode::FullyCustom;

		EditorContext->MarkTransactionAsDirty();
		EditorContext->EndTransaction();
	}
}

void FSpriteGeometryEditingHelper::DeleteLastVertexFromAddPolygonMode()
{
	check(bIsAddingPolygon);

	FSpriteGeometryCollection& Geometry = GetGeometryChecked();

	if ((AddingPolygonIndex != INDEX_NONE) && Geometry.Shapes.IsValidIndex(AddingPolygonIndex))
	{
		EditorContext->BeginTransaction(LOCTEXT("DeleteLastAddedPointTransaction", "Delete last added point"));

		FSpriteGeometryShape& Shape = Geometry.Shapes[AddingPolygonIndex];
		if (Shape.Vertices.Num() > 0)
		{
			Shape.Vertices.Pop();
		}
		else
		{
			Geometry.Shapes.RemoveAt(AddingPolygonIndex);
			AddingPolygonIndex = INDEX_NONE;
		}

		EditorContext->MarkTransactionAsDirty();
		EditorContext->EndTransaction();
	}
	else
	{
		ResetAddPolygonMode();
	}
}

bool FSpriteGeometryEditingHelper::ClosestPointOnLine(const FVector2D& Point, const FVector2D& LineStart, const FVector2D& LineEnd, FVector2D& OutClosestPoint)
{
	const FVector2D DL = LineEnd - LineStart;
	const FVector2D DP = Point - LineStart;
	const float DotL = FVector2D::DotProduct(DP, DL);
	const float DotP = FVector2D::DotProduct(DL, DL);
	if (DotP > 0.0001f)
	{
		float T = DotL / DotP;
		if ((T >= -0.0001f) && (T <= 1.0001f))
		{
			T = FMath::Clamp(T, 0.0f, 1.0f);
			OutClosestPoint = LineStart + T * DL;
			return true;
		}
	}

	return false;
}

bool FSpriteGeometryEditingHelper::DeleteVertexInPolygonInternal(FSpriteGeometryCollection& Geometry, const int32 ShapeIndex, const int32 VertexIndex)
{
	if (Geometry.Shapes.IsValidIndex(ShapeIndex))
	{
		FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];
		if (Shape.Vertices.IsValidIndex(VertexIndex))
		{
			Geometry.GeometryType = ESpritePolygonMode::FullyCustom;
			Shape.ShapeType = ESpriteShapeType::Polygon;
			Shape.Vertices.RemoveAt(VertexIndex);

			if (Shape.Vertices.Num() == 0)
			{
				// Tell the caller they should delete the polygon since it has no more verts
				return true;
			}
		}
	}

	return false;
}


FVector2D FSpriteGeometryEditingHelper::TextureSpaceToScreenSpace(const FSceneView& View, const FVector2D& SourcePoint) const
{
	const FVector WorldSpacePoint = EditorContext->TextureSpaceToWorldSpace(SourcePoint);

	FVector2D PixelLocation;
	View.WorldToPixel(WorldSpacePoint, /*out*/ PixelLocation);
	return PixelLocation;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
