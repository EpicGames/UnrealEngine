// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "UObject/GCObject.h"

enum EInputEvent : int;

class FCanvas;
class FPrimitiveDrawInterface;
class FSceneView;
class FSelectedItem;
class FViewport;
class UMaterial;
struct FSpriteGeometryCollection;

//////////////////////////////////////////////////////////////////////////
// FShapeVertexPair

struct FShapeVertexPair
{
public:
	const int32 ShapeIndex;
	const int32 VertexIndex;

public:
	FShapeVertexPair()
		: ShapeIndex(INDEX_NONE)
		, VertexIndex(INDEX_NONE)
	{
	}

	FShapeVertexPair(int32 InShapeIndex, int32 InVertexIndex)
		: ShapeIndex(InShapeIndex)
		, VertexIndex(InVertexIndex)
	{
	}
};

inline bool operator==(const FShapeVertexPair& A, const FShapeVertexPair& B)
{
	return (A.ShapeIndex == B.ShapeIndex) && (A.VertexIndex == B.VertexIndex);
}

inline uint32 GetTypeHash(const FShapeVertexPair& Item)
{
	return Item.VertexIndex + (Item.ShapeIndex * 311);
}

//////////////////////////////////////////////////////////////////////////
// FSpriteSelectionHelper

class FSpriteSelectionHelper
{
public:
	bool HasAnySelectedItems() const
	{
		return SelectedItemSet.Num() > 0;
	}

	const TSet< TSharedPtr<FSelectedItem> >& GetSelectionSet() const
	{
		return SelectedItemSet;
	}

	void SelectItem(TSharedPtr<FSelectedItem> NewItem);

	virtual void ClearSelectionSet();

	virtual bool CanDeleteSelection() const;

	virtual ~FSpriteSelectionHelper() = default;

private:
	// Set of selected objects
	TSet< TSharedPtr<FSelectedItem> > SelectedItemSet;
};

//////////////////////////////////////////////////////////////////////////
// FSpriteGeometryEditingHelper

class FSpriteGeometryEditingHelper : public FSpriteSelectionHelper, public FGCObject
{
public:
	FSpriteGeometryEditingHelper(class ISpriteSelectionContext* InEditorContext);

	// FSpriteSelectionHelper interface
	virtual void ClearSelectionSet() override;
	// End of FSpriteSelectionHelper interface

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSpriteGeometryEditingHelper");
	}
	// End of FGCObject interface

	void DeleteSelectedItems();
	void SetEditorContext(class ISpriteSelectionContext* InNewEditorContext);
	class ISpriteSelectionContext* GetEditorContext() const;

	void DrawGeometry(const FSceneView& View, FPrimitiveDrawInterface& PDI, const FLinearColor& GeometryVertexColor, const FLinearColor& NegativeGeometryVertexColor);
	void DrawGeometry_CanvasPass(FViewport& InViewport, const FSceneView& View, FCanvas& Canvas, /*inout*/ int32& YPos, const FLinearColor& GeometryVertexColor, const FLinearColor& NegativeGeometryVertexColor);

	// Adds a point to the currently edited index
	// If SelectedPolygonIndex is set, only that polygon is considered
	void AddPointToGeometry(const FVector2D& TextureSpacePoint, const int32 SelectedPolygonIndex = INDEX_NONE);

	void SelectGeometry(const FShapeVertexPair& GeometyItem);
	bool IsGeometrySelected(const FShapeVertexPair& GeometyItem) const;


	void AddShapeToSelection(const int32 ShapeIndex);
	void AddPolygonVertexToSelection(const int32 ShapeIndex, const int32 VertexIndex);
	void AddPolygonEdgeToSelection(const int32 ShapeIndex, const int32 FirstVertexIndex);

	void SetShowNormals(bool bShouldShowNormals);
	void ToggleShowNormals();
	bool IsShowNormalsEnabled() const { return bShowNormals; }

	// Changes the geometry being edited (clears the selection set in the process)
	void SetGeometryBeingEdited(FSpriteGeometryCollection* NewGeometryBeingEdited, bool bInAllowCircles, bool bInAllowSubtractivePolygons);
	FSpriteGeometryCollection* GetGeometryBeingEdited() const;

	// Adds a new circle shape and selects it
	void AddNewCircleShape(const FVector2D& CircleLocation, float Radius);
	bool CanAddCircleShape() const { return bAllowCircles && (GeometryBeingEdited != nullptr); }

	// Adds a new box shape and selects it
	void AddNewBoxShape(const FVector2D& BoxLocation, const FVector2D& BoxSize);
	bool CanAddBoxShape() const { return GeometryBeingEdited != nullptr; }

	void ResetAddPolygonMode();
	void ToggleAddPolygonMode();
	bool IsAddingPolygon() const { return bIsAddingPolygon; }
	bool CanAddPolygon() const { return GeometryBeingEdited != nullptr; }
	bool CanAddSubtractivePolygon() const { return CanAddPolygon() && bAllowSubtractivePolygons; }
	void AbandonAddPolygonMode();

	void SnapAllVerticesToPixelGrid();
	bool CanSnapVerticesToPixelGrid() const { return GeometryBeingEdited != nullptr; }

	void HandleAddPolygonClick(const FVector2D& TexturePoint, bool bWantsSubtractive, const FSceneView& View, EInputEvent Event);

	void DeleteLastVertexFromAddPolygonMode();

	bool IsEditingGeometry() const
	{
		return GeometryBeingEdited != nullptr;
	}
private:
	static bool ClosestPointOnLine(const FVector2D& Point, const FVector2D& LineStart, const FVector2D& LineEnd, FVector2D& OutClosestPoint);

	// Be sure to call this with polygonIndex and VertexIndex in descending order
	// Returns true if the shape went to zero points and should be deleted itself
	static bool DeleteVertexInPolygonInternal(FSpriteGeometryCollection& Geometry, const int32 ShapeIndex, const int32 VertexIndex);

	FSpriteGeometryCollection& GetGeometryChecked()
	{
		check(GeometryBeingEdited);
		return *GeometryBeingEdited;
	}

	//@TODO: This is needed for the canvas pass, but most of that code should go into the PDI pass
	FVector2D TextureSpaceToScreenSpace(const FSceneView& View, const FVector2D& SourcePoint) const;

private:
	class ISpriteSelectionContext* EditorContext;

	TObjectPtr<UMaterial> WidgetVertexColorMaterial;

	// Set of selected vertices/shapes
	TSet<FShapeVertexPair> SelectedIDSet;

	// Active geometry being edited
	FSpriteGeometryCollection* GeometryBeingEdited;

	// Is waiting to add geometry
	bool bIsAddingPolygon;

	// The polygon index being added to, INDEX_NONE if we don't have a polygon yet
	int32 AddingPolygonIndex;

	// Should we show polygon edge normals?
	bool bShowNormals;

	// Do we allow subtractive polygons?
	bool bAllowSubtractivePolygons;

	// Do we allow circles?
	bool bAllowCircles;
};
