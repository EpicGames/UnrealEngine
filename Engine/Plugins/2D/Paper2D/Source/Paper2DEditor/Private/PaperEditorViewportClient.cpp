// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperEditorViewportClient.h"
#include "Editor/UnrealEdTypes.h"
#include "Paper2DModule.h"
#include "CanvasTypes.h"
#include "Engine/Texture2D.h"
#include "ImageUtils.h"
#include "GlobalRenderResources.h"

//////////////////////////////////////////////////////////////////////////
// FPaperEditorViewportClient

FPaperEditorViewportClient::FPaperEditorViewportClient(const TWeakPtr<SEditorViewport>& InEditorViewportWidget)
	: FEditorViewportClient(nullptr, nullptr, InEditorViewportWidget)
	, CheckerboardTexture(nullptr)
{
	ZoomPos = FVector2D::ZeroVector;
	ZoomAmount = 1.0f;

	//ModifyCheckerboardTextureColors();

	//@TODO: Pretty lame hardcoding
	//@TODO: Doesn't handle negatives either (not really)
	const bool XX = FMath::IsNearlyEqual(PaperAxisX.X, (FVector::FReal)1.0f);
	const bool XY = FMath::IsNearlyEqual(PaperAxisX.Y, (FVector::FReal)1.0f);
	const bool YY = FMath::IsNearlyEqual(PaperAxisY.Y, (FVector::FReal)1.0f);
	const bool YZ = FMath::IsNearlyEqual(PaperAxisY.Z, (FVector::FReal)1.0f);

	ELevelViewportType NewViewportType = LVT_OrthoXZ;
	if (XX && YY)
	{
		NewViewportType = LVT_OrthoXY;
	}
	else if (XX && YZ)
	{
		NewViewportType = LVT_OrthoXZ;
	}
	else if (XY && YZ)
	{
		NewViewportType = LVT_OrthoYZ;
	}
	else
	{
		//@TODO: Unsupported axes
	}
	SetViewModes(VMI_Lit, VMI_Lit);
	SetViewportType(NewViewportType);

	bDeferZoomToSprite = true;
	bDeferZoomToSpriteIsInstant = true;

	// Get the correct general direction of the perspective mode; the distance doesn't matter much as we've queued up a deferred zoom that will calculate a much better distance
	SetInitialViewTransform(LVT_Perspective, -100.0f * PaperAxisZ, PaperAxisZ.Rotation(), DEFAULT_ORTHOZOOM);
}

FPaperEditorViewportClient::~FPaperEditorViewportClient()
{
}

FLinearColor FPaperEditorViewportClient::GetBackgroundColor() const
{
	//@TODO: Make adjustable - TextureEditorPtr.Pin()->GetBackgroundColor());
	return FLinearColor(0, 0, 127, 0);
}

void FPaperEditorViewportClient::Tick(float DeltaSeconds)
{
	// Zoom in on the sprite
	//@TODO: Fix this properly so it doesn't need to be deferred, or wait for the viewport to initialize
	FIntPoint Size = Viewport->GetSizeXY();
	if (bDeferZoomToSprite && (Size.X > 0) && (Size.Y > 0))
	{
		FBox BoundsToFocus = GetDesiredFocusBounds();
		if (ViewportType != LVT_Perspective)
		{
			TGuardValue<ELevelViewportType> SaveViewportType(ViewportType, LVT_Perspective);
			FocusViewportOnBox(BoundsToFocus, bDeferZoomToSpriteIsInstant);
		}

		FocusViewportOnBox(BoundsToFocus, bDeferZoomToSpriteIsInstant);
		bDeferZoomToSprite = false;
	}

	FEditorViewportClient::Tick(DeltaSeconds);
}

void FPaperEditorViewportClient::DrawSelectionRectangles(FViewport* InViewport, FCanvas* Canvas)
{
	for (auto RectangleIt = SelectionRectangles.CreateConstIterator(); RectangleIt; ++RectangleIt)
	{
		const FViewportSelectionRectangle& Rect = *RectangleIt;

		const float X = (Rect.TopLeft.X - ZoomPos.X) * ZoomAmount;
		const float Y = (Rect.TopLeft.Y - ZoomPos.Y) * ZoomAmount;
		const float W = Rect.Dimensions.X * ZoomAmount;
		const float H = Rect.Dimensions.Y * ZoomAmount;
		const bool bAlphaBlend = true;

		Canvas->DrawTile(X, Y, W, H, 0, 0, 1, 1, Rect.Color, GWhiteTexture, bAlphaBlend);
	}
}

void FPaperEditorViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEditorViewportClient::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(CheckerboardTexture);
}

// Called to request a focus on the current selection
void FPaperEditorViewportClient::RequestFocusOnSelection(bool bInstant)
{
	bDeferZoomToSprite = true;
	bDeferZoomToSpriteIsInstant = bInstant;
}

void FPaperEditorViewportClient::ModifyCheckerboardTextureColors()
{
	const FColor ColorOne = FColor(128, 128, 128);//TextureEditorPtr.Pin()->GetCheckeredBackground_ColorOne();
	const FColor ColorTwo = FColor(64, 64, 64);//TextureEditorPtr.Pin()->GetCheckeredBackground_ColorTwo();
	const int32 CheckerSize = 32;//TextureEditorPtr.Pin()->GetCheckeredBackground_Size();

	DestroyCheckerboardTexture();
	SetupCheckerboardTexture(ColorOne, ColorTwo, CheckerSize);
}

void FPaperEditorViewportClient::SetupCheckerboardTexture(const FColor& ColorOne, const FColor& ColorTwo, int32 CheckerSize)
{
	if (CheckerboardTexture == nullptr)
	{
		CheckerboardTexture = FImageUtils::CreateCheckerboardTexture(ColorOne, ColorTwo, CheckerSize);
	}
}

void FPaperEditorViewportClient::DestroyCheckerboardTexture()
{
	if (CheckerboardTexture)
	{
		if (CheckerboardTexture->GetResource())
		{
			CheckerboardTexture->ReleaseResource();
		}
		CheckerboardTexture->MarkAsGarbage();
		CheckerboardTexture = nullptr;
	}
}
