// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"

class FCanvas;

struct FViewportSelectionRectangle
{
	FVector2D TopLeft;
	FVector2D Dimensions;
	FLinearColor Color;
};

//////////////////////////////////////////////////////////////////////////
// FPaperEditorViewportClient

class FPaperEditorViewportClient : public FEditorViewportClient
{
public:
	/** Constructor */
	explicit FPaperEditorViewportClient(const TWeakPtr<class SEditorViewport>& InEditorViewportWidget = nullptr);
	~FPaperEditorViewportClient();

	// FViewportClient interface
	virtual void Tick(float DeltaSeconds) override;
	// End of FViewportClient interface

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	// End of FEditorViewportClient interface

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FSerializableObject interface

	// Called to request a focus on the current selection
	virtual void RequestFocusOnSelection(bool bInstant);

	/** Modifies the checkerboard texture's data */
	void ModifyCheckerboardTextureColors();

	void SetZoomPos(FVector2D InNewPos, float InNewZoom)
	{
		ZoomPos = InNewPos;
		ZoomAmount = InNewZoom;
	}

	// List of selection rectangles to draw
	TArray<FViewportSelectionRectangle> SelectionRectangles;
private:
	/** Initialize the checkerboard texture for the texture preview, if necessary */
	void SetupCheckerboardTexture(const FColor& ColorOne, const FColor& ColorTwo, int32 CheckerSize);

	/** Destroy the checkerboard texture if one exists */
	void DestroyCheckerboardTexture();

protected:
	void DrawSelectionRectangles(FViewport* Viewport, FCanvas* Canvas);

	virtual FBox GetDesiredFocusBounds() const
	{
		return FBox(ForceInitToZero);
	}

protected:
	/** Checkerboard texture */
	TObjectPtr<UTexture2D> CheckerboardTexture;
	FVector2D ZoomPos;
	float ZoomAmount;

private:
	// Should we zoom to the focus bounds next tick?
	bool bDeferZoomToSprite;
	bool bDeferZoomToSpriteIsInstant;
};
