// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbookActorFactory.h"
#include "AssetRegistry/AssetData.h"
#include "PaperFlipbookActor.h"
#include "PaperFlipbook.h"
#include "PaperFlipbookComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperFlipbookActorFactory)

//////////////////////////////////////////////////////////////////////////
// UPaperFlipbookActorFactory

UPaperFlipbookActorFactory::UPaperFlipbookActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = NSLOCTEXT("Paper2D", "PaperFlipbookFactoryDisplayName", "Add Animated Sprite");
	NewActorClass = APaperFlipbookActor::StaticClass();
}

void UPaperFlipbookActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	if (UPaperFlipbook* Flipbook = Cast<UPaperFlipbook>(Asset))
	{
		APaperFlipbookActor* TypedActor = CastChecked<APaperFlipbookActor>(NewActor);
		UPaperFlipbookComponent* RenderComponent = TypedActor->GetRenderComponent();
		check(RenderComponent);

		RenderComponent->UnregisterComponent();
		RenderComponent->SetFlipbook(Flipbook);
		RenderComponent->RegisterComponent();
	}
}

bool UPaperFlipbookActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (AssetData.IsValid() && AssetData.IsInstanceOf(UPaperFlipbook::StaticClass()))
	{
		return true;
	}
	else
	{
		OutErrorMsg = NSLOCTEXT("Paper2D", "CanCreateActorFrom_NoFlipbook", "No flipbook was specified.");
		return false;
	}
}

