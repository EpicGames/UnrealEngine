// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentAssetBroker.h"
#include "PaperFlipbookComponent.h"
#include "PaperFlipbook.h"

//////////////////////////////////////////////////////////////////////////
// FPaperFlipbookAssetBroker

class FPaperFlipbookAssetBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UPaperFlipbook::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UPaperFlipbookComponent* RenderComp = Cast<UPaperFlipbookComponent>(InComponent))
		{
			UPaperFlipbook* Flipbook = Cast<UPaperFlipbook>(InAsset);

			if ((Flipbook != nullptr) || (InAsset == nullptr))
			{
				RenderComp->SetFlipbook(Flipbook);
				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UPaperFlipbookComponent* RenderComp = Cast<UPaperFlipbookComponent>(InComponent))
		{
			return RenderComp->GetFlipbook();
		}
		return nullptr;
	}
};

