// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbookFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperFlipbookFactory)

#define LOCTEXT_NAMESPACE "Paper2D"

/////////////////////////////////////////////////////
// UPaperFlipbookFactory

UPaperFlipbookFactory::UPaperFlipbookFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPaperFlipbook::StaticClass();
}

UObject* UPaperFlipbookFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPaperFlipbook* NewFlipbook = NewObject<UPaperFlipbook>(InParent, Class, Name, Flags | RF_Transactional);
	{
		FScopedFlipbookMutator EditLock(NewFlipbook);
		EditLock.KeyFrames = KeyFrames;
	}
	return NewFlipbook;
}

#undef LOCTEXT_NAMESPACE

