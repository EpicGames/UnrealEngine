// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbookHelpers.h"
#include "ObjectTools.h"
#include "PaperSprite.h"

//////////////////////////////////////////////////////////////////////////
// FPaperFlipbookHelpers

FString FPaperFlipbookHelpers::GetCleanerSpriteName(const FString& Name)
{
	int32 LastCharacter = Name.Len() - 1;

	// Strip off "_Sprite"
	if (Name.Right(7) == TEXT("_Sprite"))
	{
		LastCharacter = Name.Len() - 7;
	}

	while (LastCharacter >= 0 && FChar::IsPunct(Name[LastCharacter]))
	{
		LastCharacter--;
	}

	if (LastCharacter > 0)
	{
		return Name.Left(LastCharacter + 1);
	}
	else
	{
		return Name;
	}
}

bool FPaperFlipbookHelpers::ExtractSpriteNumber(const FString& String, FString& BareString, int32& Number)
{
	bool bExtracted = false;

	int32 LastCharacter = String.Len() - 1;
	if (LastCharacter >= 0)
	{
		// Find the last character that isn't a digit (Handle sprite names with numbers inside inverted commas / parentheses)
		while (LastCharacter >= 0 && !FChar::IsDigit(String[LastCharacter]))
		{
			LastCharacter--;
		}

		// Only proceed if we found a number in the sprite name
		if (LastCharacter >= 0)
		{
			while (LastCharacter > 0 && FChar::IsDigit(String[LastCharacter - 1]))
			{
				LastCharacter--;
			}

			if (LastCharacter >= 0)
			{
				const int32 FirstDigit = LastCharacter;
				int32 EndCharacter = FirstDigit;
				while (EndCharacter > 0 && !FChar::IsAlnum(String[EndCharacter - 1]))
				{
					EndCharacter--;
				}

				if (EndCharacter == 0)
				{
					// This string consists of non alnum + number, eg. _42
					// The flipbook / category name in this case will be _
					// Otherwise, we strip out all trailing non-alnum chars
					EndCharacter = FirstDigit;
				}

				const FString NumberString = String.Mid(FirstDigit);
				BareString = GetCleanerSpriteName((EndCharacter > 0) ? String.Left(EndCharacter) : String);
				Number = FCString::Atoi(*NumberString);
				bExtracted = true;
			}
		}
	}

	if (!bExtracted)
	{
		BareString = String;
		Number = -1;
	}
	return bExtracted;
}

void FPaperFlipbookHelpers::ExtractFlipbooksFromSprites(TMap<FString, TArray<UPaperSprite*> >& OutSpriteFlipbookMap, const TArray<UPaperSprite*>& Sprites, const TArray<FString>& InSpriteNames)
{
	OutSpriteFlipbookMap.Reset();

	// Local copy
	check((InSpriteNames.Num() == 0) || (InSpriteNames.Num() == Sprites.Num()));
	TArray<FString> SpriteNames = InSpriteNames;
	if (InSpriteNames.Num() == 0)
	{
		SpriteNames.Reset();
		for (int32 SpriteIndex = 0; SpriteIndex < Sprites.Num(); ++SpriteIndex)
		{
			check(Sprites[SpriteIndex] != nullptr);
			SpriteNames.Add(Sprites[SpriteIndex]->GetName());
		}
	}

	// Group them
	TMap<FString, UPaperSprite*> SpriteNameMap;
	TArray<UPaperSprite*> RemainingSprites;

	for (int32 SpriteIndex = 0; SpriteIndex < Sprites.Num(); ++SpriteIndex)
	{
		UPaperSprite* Sprite = Sprites[SpriteIndex];
		const FString SpriteName = SpriteNames[SpriteIndex];

		SpriteNameMap.Add(SpriteName, Sprite);

		int32 SpriteNumber = 0;
		FString SpriteBareString;
		if (ExtractSpriteNumber(SpriteName, /*out*/ SpriteBareString, /*out*/ SpriteNumber))
		{
			SpriteBareString = ObjectTools::SanitizeObjectName(SpriteBareString);
			OutSpriteFlipbookMap.FindOrAdd(SpriteBareString).Add(Sprite);
		}
		else
		{
			RemainingSprites.Add(Sprite);
		}
	}

	// Natural sort using the same method as above
	struct FSpriteSortPredicate
	{
		FSpriteSortPredicate() {}

		// Sort predicate operator
		bool operator()(UPaperSprite& LHS, UPaperSprite& RHS) const
		{
			FString LeftString;
			int32 LeftNumber;
			ExtractSpriteNumber(LHS.GetName(), /*out*/ LeftString, /*out*/ LeftNumber);

			FString RightString;
			int32 RightNumber;
			ExtractSpriteNumber(RHS.GetName(), /*out*/ RightString, /*out*/ RightNumber);

			return (LeftString == RightString) ? (LeftNumber < RightNumber) : (LeftString < RightString);
		}
	};

	// Sort sprites
	TArray<FString> Keys;
	OutSpriteFlipbookMap.GetKeys(Keys);
	for (auto SpriteName : Keys)
	{
		OutSpriteFlipbookMap[SpriteName].Sort(FSpriteSortPredicate());
	}

	// Create a flipbook from all remaining sprites
	// Not sure if this is desirable behavior, might want one flipbook per sprite
	if (RemainingSprites.Num() > 0)
	{
		RemainingSprites.Sort(FSpriteSortPredicate());

		const FString DesiredName = GetCleanerSpriteName(RemainingSprites[0]->GetName()) + TEXT("_Flipbook");
		const FString SanitizedName = ObjectTools::SanitizeObjectName(DesiredName);

		OutSpriteFlipbookMap.Add(SanitizedName, RemainingSprites);
	}
}
