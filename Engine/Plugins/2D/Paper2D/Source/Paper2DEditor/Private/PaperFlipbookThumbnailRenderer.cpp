// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperFlipbookThumbnailRenderer.h"
#include "Math/BoxSphereBounds.h"
#include "Misc/App.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"
#include "PaperFlipbook.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperFlipbookThumbnailRenderer)

//////////////////////////////////////////////////////////////////////////
// UPaperFlipbookThumbnailRenderer

UPaperFlipbookThumbnailRenderer::UPaperFlipbookThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPaperFlipbookThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UPaperFlipbook* Flipbook = Cast<UPaperFlipbook>(Object))
	{
		const double DeltaTime = FApp::GetCurrentTime() - GStartTime;
		const double TotalDuration = Flipbook->GetTotalDuration();
		const float PlayTime = (TotalDuration > 0.0f) ? FMath::Fmod(DeltaTime, TotalDuration) : 0.0f;

		if (UPaperSprite* Sprite = Flipbook->GetSpriteAtTime(PlayTime))
		{
			FBoxSphereBounds FlipbookBounds = Flipbook->GetRenderBounds();
			DrawFrame(Sprite, X, Y, Width, Height, RenderTarget, Canvas, &FlipbookBounds);
			return;
		}
		else
		{
			// Fallback for empty frames or newly created flipbooks
			DrawGrid(X, Y, Width, Height, Canvas);
		}

		if (TotalDuration == 0.0f)
		{
			// Warning text for no frames
			const FText ErrorText = NSLOCTEXT("FlipbookEditorApp", "ThumbnailWarningNoFrames", "No frames");
			FCanvasTextItem TextItem(FVector2D(5.0f, 5.0f), ErrorText, GEngine->GetLargeFont(), FLinearColor::Red);
			TextItem.EnableShadow(FLinearColor::Black);
			TextItem.Scale = FVector2D(Width / 128.0f, Height / 128.0f);
			TextItem.Draw(Canvas);
		}
	}
}

