// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperImporterSettings.h"
#include "Paper2DEditorLog.h"
#include "Materials/MaterialInterface.h"
#include "PaperTileMap.h"
#include "PaperTileSet.h"
#include "AlphaBitmap.h"
#include "TileMapEditing/TileMapEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperImporterSettings)

//////////////////////////////////////////////////////////////////////////
// UPaperImporterSettings

UPaperImporterSettings::UPaperImporterSettings()
	: bPickBestMaterialWhenCreatingSprites(true)
	, bPickBestMaterialWhenCreatingTileMaps(true)
	, DefaultPixelsPerUnrealUnit(1.0f)
	, DefaultSpriteTextureGroup(TEXTUREGROUP_Pixels2D)
	, bOverrideTextureCompression(true)
	, DefaultSpriteTextureCompression(TC_EditorIcon)
{
	NormalMapTextureSuffixes.Add(TEXT("_N"));
	NormalMapTextureSuffixes.Add(TEXT("_Normal"));

	BaseMapTextureSuffixes.Add(TEXT("_D"));
	BaseMapTextureSuffixes.Add(TEXT("_Diffuse"));

	if (!IsRunningCommandlet())
	{
		UnlitDefaultMaskedMaterialName = FSoftObjectPath("/Paper2D/MaskedUnlitSpriteMaterial.MaskedUnlitSpriteMaterial");
		UnlitDefaultTranslucentMaterialName = FSoftObjectPath("/Paper2D/TranslucentUnlitSpriteMaterial.TranslucentUnlitSpriteMaterial");
		UnlitDefaultOpaqueMaterialName = FSoftObjectPath("/Paper2D/OpaqueUnlitSpriteMaterial.OpaqueUnlitSpriteMaterial");

		LitDefaultMaskedMaterialName = FSoftObjectPath("/Paper2D/MaskedLitSpriteMaterial.MaskedLitSpriteMaterial");
		LitDefaultTranslucentMaterialName = FSoftObjectPath("/Paper2D/TranslucentLitSpriteMaterial.TranslucentLitSpriteMaterial");
		LitDefaultOpaqueMaterialName = FSoftObjectPath("/Paper2D/OpaqueLitSpriteMaterial.OpaqueLitSpriteMaterial");
	}
}

FString UPaperImporterSettings::RemoveSuffixFromBaseMapName(const FString& InName) const
{
	FString Result = InName;

	for (const FString& PossibleSuffix : BaseMapTextureSuffixes)
	{
		if (Result.RemoveFromEnd(PossibleSuffix))
		{
			return Result;
		}
	}

	return Result;
}

void UPaperImporterSettings::GenerateNormalMapNamesToTest(const FString& InRoot, TArray<FString>& InOutNames) const
{
	for (const FString& PossibleSuffix : NormalMapTextureSuffixes)
	{
		InOutNames.Add(InRoot + PossibleSuffix);
	}
}

void UPaperImporterSettings::ApplyTextureSettings(UTexture2D* Texture) const
{
	if (Texture->IsNormalMap())
	{
		// Leave normal maps alone
	}
	else
	{
		Texture->Modify();

		Texture->LODGroup = DefaultSpriteTextureGroup;

		if (bOverrideTextureCompression)
		{
			Texture->CompressionSettings = DefaultSpriteTextureCompression;
		}

		Texture->PostEditChange();
	}
}

ESpriteInitMaterialType UPaperImporterSettings::AnalyzeTextureForDesiredMaterialType(UTexture* Texture, const FIntPoint& Offset, const FIntPoint& Dimensions) const
{
	// Analyze the texture if desired (to see if it's got greyscale alpha or just binary alpha, picking either a translucent or masked material)
	if (Texture != nullptr)
	{
		FAlphaBitmap AlphaBitmap(Texture);
		bool bHasIntermediateValues;
		bool bHasZeros;
		AlphaBitmap.AnalyzeImage(Offset.X, Offset.Y, Dimensions.X, Dimensions.Y, /*out*/ bHasZeros, /*out*/ bHasIntermediateValues);

		if (bAnalysisCanUseOpaque && !bHasIntermediateValues && !bHasZeros)
		{
			return ESpriteInitMaterialType::Opaque;
		}
		else
		{
			return bHasIntermediateValues ? ESpriteInitMaterialType::Translucent : ESpriteInitMaterialType::Masked;
		}
	}

	return ESpriteInitMaterialType::Automatic;
}

void UPaperImporterSettings::ApplySettingsForSpriteInit(FSpriteAssetInitParameters& InitParams, ESpriteInitMaterialLightingMode LightingMode, ESpriteInitMaterialType MaterialTypeMode) const
{
	InitParams.SetPixelsPerUnrealUnit(DefaultPixelsPerUnrealUnit);

	ESpriteInitMaterialType DesiredMaterialType = MaterialTypeMode;
	if (DesiredMaterialType == ESpriteInitMaterialType::Automatic)
	{
		// Analyze the texture if desired (to see if it's got greyscale alpha or just binary alpha, picking either a translucent or masked material)
		if (bPickBestMaterialWhenCreatingSprites)
		{
			DesiredMaterialType = AnalyzeTextureForDesiredMaterialType(InitParams.Texture, InitParams.Offset, InitParams.Dimension);
		}
	}

	if (DesiredMaterialType == ESpriteInitMaterialType::Automatic)
	{
		// Fall back to masked if we wanted automatic and couldn't analyze things
		DesiredMaterialType = ESpriteInitMaterialType::Masked;
	}

	if (DesiredMaterialType != ESpriteInitMaterialType::LeaveAsIs)
	{
		// Determine whether to use lit or unlit materials
		const bool bUseLitMaterial = LightingMode == ESpriteInitMaterialLightingMode::ForceLit;

		// Apply the materials
		InitParams.DefaultMaterialOverride = GetDefaultMaterial(DesiredMaterialType, bUseLitMaterial);
		InitParams.AlternateMaterialOverride = GetDefaultMaterial(ESpriteInitMaterialType::Opaque, bUseLitMaterial);
	}
}

void UPaperImporterSettings::ApplySettingsForTileMapInit(UPaperTileMap* TileMap, UPaperTileSet* DefaultTileSet, ESpriteInitMaterialLightingMode LightingMode, ESpriteInitMaterialType MaterialTypeMode, bool bCreateEmptyLayer) const
{
	if (DefaultTileSet != nullptr)
	{
		const FIntPoint TileSetTileSize(DefaultTileSet->GetTileSize());
		TileMap->TileWidth = TileSetTileSize.X;
		TileMap->TileHeight = TileSetTileSize.Y;
		TileMap->SelectedTileSet = DefaultTileSet;
	}

	TileMap->PixelsPerUnrealUnit = DefaultPixelsPerUnrealUnit;

	// Set Defaults based on the editor preferences 
	const UTileMapEditorSettings* Settings = GetMutableDefault<UTileMapEditorSettings>();
	if (!Settings)
	{
		Settings = GetDefault<UTileMapEditorSettings>();
	}

	TileMap->BackgroundColor = Settings->DefaultBackgroundColor;
	TileMap->TileGridColor = Settings->DefaultTileGridColor;
	TileMap->MultiTileGridColor = Settings->DefaultMultiTileGridColor;
	TileMap->MultiTileGridWidth = Settings->DefaultMultiTileGridWidth;
	TileMap->MultiTileGridHeight = Settings->DefaultMultiTileGridHeight;
	TileMap->MultiTileGridOffsetX = Settings->DefaultMultiTileGridOffsetX;
	TileMap->MultiTileGridOffsetY = Settings->DefaultMultiTileGridOffsetY;
	TileMap->LayerGridColor = Settings->DefaultLayerGridColor;

	ESpriteInitMaterialType DesiredMaterialType = MaterialTypeMode;
	if (DesiredMaterialType == ESpriteInitMaterialType::Automatic)
	{
		// Analyze the texture if desired (to see if it's got greyscale alpha or just binary alpha, picking either a translucent or masked material)
		if (bPickBestMaterialWhenCreatingTileMaps && (DefaultTileSet != nullptr))
		{
			if (UTexture2D* TileSheetTexture = DefaultTileSet->GetTileSheetTexture())
			{
				DesiredMaterialType = AnalyzeTextureForDesiredMaterialType(TileSheetTexture, FIntPoint::ZeroValue, TileSheetTexture->GetImportedSize());
			}
		}
	}

	if (DesiredMaterialType == ESpriteInitMaterialType::Automatic)
	{
		// Fall back to masked if we wanted automatic and couldn't analyze things
		DesiredMaterialType = ESpriteInitMaterialType::Masked;
	}

	if (DesiredMaterialType != ESpriteInitMaterialType::LeaveAsIs)
	{
		// Determine whether to use lit or unlit materials
		const bool bUseLitMaterial = LightingMode == ESpriteInitMaterialLightingMode::ForceLit;

		// Apply the material
		if (UMaterialInterface* MaterialOverride = GetDefaultMaterial(DesiredMaterialType, bUseLitMaterial))
		{
			TileMap->Material = MaterialOverride;
		}
	}

	if (bCreateEmptyLayer)
	{
		// Add a new empty layer
		TileMap->AddNewLayer();
	}
}

UMaterialInterface* UPaperImporterSettings::GetDefaultTranslucentMaterial(bool bLit) const
{
	return Cast<UMaterialInterface>((bLit ? LitDefaultTranslucentMaterialName : UnlitDefaultTranslucentMaterialName).TryLoad());
}

UMaterialInterface* UPaperImporterSettings::GetDefaultOpaqueMaterial(bool bLit) const
{
	return Cast<UMaterialInterface>((bLit ? LitDefaultOpaqueMaterialName : UnlitDefaultOpaqueMaterialName).TryLoad());
}

UMaterialInterface* UPaperImporterSettings::GetDefaultMaskedMaterial(bool bLit) const
{
	return Cast<UMaterialInterface>((bLit ? LitDefaultMaskedMaterialName : UnlitDefaultMaskedMaterialName).TryLoad());
}

UMaterialInterface* UPaperImporterSettings::GetDefaultMaterial(ESpriteInitMaterialType MaterialType, bool bUseLitMaterial) const
{
	UMaterialInterface* Result = nullptr;

	// Apply the materials
	switch (MaterialType)
	{
	default:
		ensureMsgf(false, TEXT("Unexpected material type in UPaperImporterSettings::GetDefaultMaterial"));
		// Fall thru
	case ESpriteInitMaterialType::LeaveAsIs:
	case ESpriteInitMaterialType::Automatic:
	case ESpriteInitMaterialType::Masked:
		Result = GetDefaultMaskedMaterial(bUseLitMaterial);
		break;
	case ESpriteInitMaterialType::Translucent:
		Result = GetDefaultTranslucentMaterial(bUseLitMaterial);
		break;
	case ESpriteInitMaterialType::Opaque:
		Result = GetDefaultOpaqueMaterial(bUseLitMaterial);
		break;
	}

	if (Result == nullptr)
	{
		UE_LOGF(LogPaper2DEditor, Warning, "Failed to load material specified in Paper2D import settings (%ls %d)", bUseLitMaterial ? TEXT("lit") : TEXT("unlit"), (int32)MaterialType);
	}

	return Result;
}

