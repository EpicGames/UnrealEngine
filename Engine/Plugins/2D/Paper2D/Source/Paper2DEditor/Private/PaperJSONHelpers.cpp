// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperJSONHelpers.h"
#include "Dom/JsonObject.h"

//////////////////////////////////////////////////////////////////////////
// FPaperJSONHelpers

static TArray< TSharedPtr<FJsonValue> > EmptyArray;

FString FPaperJSONHelpers::ReadString(TSharedPtr<class FJsonObject> Item, const FString& Key, const FString& DefaultValue)
{
	FString Result;
	if (!Item->TryGetStringField(Key, /*out*/ Result))
	{
		Result = DefaultValue;
	}
	return Result;
}

TSharedPtr<class FJsonObject> FPaperJSONHelpers::ReadObject(TSharedPtr<class FJsonObject> Item, const FString& Key)
{
	if (Item->HasTypedField<EJson::Object>(Key))
	{
		return Item->GetObjectField(Key);
	}
	else
	{
		return nullptr;
	}
}

const TArray< TSharedPtr<FJsonValue> >& FPaperJSONHelpers::ReadArray(TSharedPtr<class FJsonObject> Item, const FString& Key)
{
	if (Item->HasTypedField<EJson::Array>(Key))
	{
		return Item->GetArrayField(Key);
	}
	else
	{
		return EmptyArray;
	}
}

bool FPaperJSONHelpers::ReadBoolean(const TSharedPtr<class FJsonObject> Item, const FString& Key, bool bDefaultIfMissing)
{
	bool bResult;
	if (!Item->TryGetBoolField(Key, /*out*/ bResult))
	{
		bResult = bDefaultIfMissing;
	}
	return bResult;
}

bool FPaperJSONHelpers::ReadFloatNoDefault(const TSharedPtr<class FJsonObject> Item, const FString& Key, float& Out_Value)
{
	double DoubleOutValue;
	const bool bResult = ReadFloatNoDefault(Item, Key, DoubleOutValue);
	Out_Value = DoubleOutValue;
	return bResult;
}

bool FPaperJSONHelpers::ReadFloatNoDefault(const TSharedPtr<class FJsonObject> Item, const FString& Key, double& Out_Value)
{
	double DoubleOutValue;
	if (Item->TryGetNumberField(Key, /*out*/ DoubleOutValue))
	{
		Out_Value = DoubleOutValue;
		return true;
	}
	else
	{
		Out_Value = 0.0f;
		return false;
	}
}

// Returns true if the field named Key is a Number (truncating it to an integer), populating Out_Value, or false if it is missing or the wrong type
bool FPaperJSONHelpers::ReadIntegerNoDefault(const TSharedPtr<class FJsonObject> Item, const FString& Key, int32& Out_Value)
{
	if (Item->TryGetNumberField(Key, Out_Value))
	{
		return true;
	}
	else
	{
		Out_Value = 0;
		return false;
	}
}

bool FPaperJSONHelpers::ReadRectangle(const TSharedPtr<class FJsonObject> Item, const FString& Key, FIntPoint& Out_XY, FIntPoint& Out_WH)
{
	const TSharedPtr<FJsonObject> Struct = ReadObject(Item, Key);
	if (Struct.IsValid())
	{
		if (ReadIntegerNoDefault(Struct, TEXT("x"), /*out*/ Out_XY.X) &&
			ReadIntegerNoDefault(Struct, TEXT("y"), /*out*/ Out_XY.Y) &&
			ReadIntegerNoDefault(Struct, TEXT("w"), /*out*/ Out_WH.X) &&
			ReadIntegerNoDefault(Struct, TEXT("h"), /*out*/ Out_WH.Y))
		{
			return true;
		}
		else
		{
			Out_XY = FIntPoint::ZeroValue;
			Out_WH = FIntPoint::ZeroValue;
			return false;
		}
	}
	else
	{
		return false;
	}
}

bool FPaperJSONHelpers::ReadSize(const TSharedPtr<class FJsonObject> Item, const FString& Key, FVector2D& Out_WH)
{
	const TSharedPtr<FJsonObject> Struct = ReadObject(Item, Key);
	if (Struct.IsValid())
	{
		if (ReadFloatNoDefault(Struct, TEXT("w"), /*out*/ Out_WH.X) &&
			ReadFloatNoDefault(Struct, TEXT("h"), /*out*/ Out_WH.Y))
		{
			return true;
		}
		else
		{
			Out_WH = FVector2D::ZeroVector;
			return false;
		}
	}
	else
	{
		return false;
	}
}

bool FPaperJSONHelpers::ReadXY(const TSharedPtr<class FJsonObject> Item, const FString& Key, FVector2D& Out_XY)
{
	const TSharedPtr<FJsonObject> Struct = ReadObject(Item, Key);
	if (Struct.IsValid())
	{
		if (ReadFloatNoDefault(Struct, TEXT("x"), /*out*/ Out_XY.X) &&
			ReadFloatNoDefault(Struct, TEXT("y"), /*out*/ Out_XY.Y))
		{
			return true;
		}
		else
		{
			Out_XY = FVector2D::ZeroVector;
			return false;
		}
	}
	else
	{
		return false;
	}
}

bool FPaperJSONHelpers::ReadIntPoint(const TSharedPtr<class FJsonObject> Item, const FString& Key, FIntPoint& Out_XY)
{
	const TSharedPtr<FJsonObject> Struct = ReadObject(Item, Key);
	if (Struct.IsValid())
	{
		if (ReadIntegerNoDefault(Struct, TEXT("x"), /*out*/ Out_XY.X) &&
			ReadIntegerNoDefault(Struct, TEXT("y"), /*out*/ Out_XY.Y))
		{
			return true;
		}
		else
		{
			Out_XY = FIntPoint::ZeroValue;
			return false;
		}
	}
	else
	{
		return false;
	}
}
