// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteActorFactory.h"
#include "AssetRegistry/AssetData.h"
#include "PhysicsEngine/BodySetup.h"
#include "PaperSpriteActor.h"
#include "PaperSprite.h"
#include "PaperSpriteComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteActorFactory)

//////////////////////////////////////////////////////////////////////////
// UPaperSpriteActorFactory

UPaperSpriteActorFactory::UPaperSpriteActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = NSLOCTEXT("Paper2D", "PaperSpriteFactoryDisplayName", "Add Sprite");
	NewActorClass = APaperSpriteActor::StaticClass();
}

void UPaperSpriteActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	if (UPaperSprite* Sprite = Cast<UPaperSprite>(Asset))
	{
		APaperSpriteActor* TypedActor = CastChecked<APaperSpriteActor>(NewActor);
		UPaperSpriteComponent* RenderComponent = TypedActor->GetRenderComponent();
		check(RenderComponent);

		RenderComponent->UnregisterComponent();
		RenderComponent->SetSprite(Sprite);

		if (Sprite->BodySetup != nullptr)
		{
			RenderComponent->BodyInstance.CopyBodyInstancePropertiesFrom(&(Sprite->BodySetup->DefaultInstance));
		}

		RenderComponent->RegisterComponent();
	}
}

bool UPaperSpriteActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (AssetData.IsValid() && AssetData.IsInstanceOf(UPaperSprite::StaticClass()))
	{
		return true;
	}
	else
	{
		OutErrorMsg = NSLOCTEXT("Paper2D", "CanCreateActorFrom_NoSprite", "No sprite was specified.");
		return false;
	}
}

