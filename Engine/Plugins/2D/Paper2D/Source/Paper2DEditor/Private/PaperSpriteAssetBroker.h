// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperSprite.h"
#include "ComponentAssetBroker.h"
#include "PaperSpriteComponent.h"
#include "PhysicsEngine/BodySetup.h"

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteAssetBroker

class FPaperSpriteAssetBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UPaperSprite::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UPaperSpriteComponent* RenderComponent = Cast<UPaperSpriteComponent>(InComponent))
		{
			UPaperSprite* Sprite = Cast<UPaperSprite>(InAsset);

			if ((Sprite != nullptr) || (InAsset == nullptr))
			{
				RenderComponent->SetSprite(Sprite);

				if (Sprite != nullptr && Sprite->BodySetup != nullptr)
				{
					RenderComponent->BodyInstance.CopyBodyInstancePropertiesFrom(&(Sprite->BodySetup->DefaultInstance));
				}

				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UPaperSpriteComponent* RenderComponent = Cast<UPaperSpriteComponent>(InComponent))
		{
			return RenderComponent->GetSprite();
		}
		return nullptr;
	}
};

