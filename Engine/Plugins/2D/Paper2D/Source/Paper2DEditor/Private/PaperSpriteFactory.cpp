// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteFactory.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"
#include "PaperImporterSettings.h"
#include "PaperSprite.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteFactory)

#define LOCTEXT_NAMESPACE "Paper2D"

/////////////////////////////////////////////////////
// UPaperSpriteFactory

UPaperSpriteFactory::UPaperSpriteFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseSourceRegion(false)
	, InitialSourceUV(0, 0)
	, InitialSourceDimension(0, 0)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPaperSprite::StaticClass();
}

bool UPaperSpriteFactory::ConfigureProperties()
{
	//@TODO: Maybe create a texture picker here?
	return true;
}

UObject* UPaperSpriteFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPaperSprite* NewSprite = NewObject<UPaperSprite>(InParent, Class, Name, Flags | RF_Transactional);

	FSpriteAssetInitParameters SpriteInitParams;

	if (bUseSourceRegion)
	{
		SpriteInitParams.Texture = InitialTexture;
		SpriteInitParams.Offset = InitialSourceUV;
		SpriteInitParams.Dimension = InitialSourceDimension;
	}
	else
	{
		SpriteInitParams.SetTextureAndFill(InitialTexture);
	}

	const UPaperImporterSettings* ImporterSettings = GetDefault<UPaperImporterSettings>();

	bool bFoundNormalMap = false;
	if (InitialTexture != nullptr)
	{
		// Look for an associated normal map to go along with the base map
		const FString SanitizedBasePackageName = UPackageTools::SanitizePackageName(InitialTexture->GetOutermost()->GetName());
		const FString PackagePath = FPackageName::GetLongPackagePath(SanitizedBasePackageName);
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		const FString NormalMapNameNoSuffix = ImporterSettings->RemoveSuffixFromBaseMapName(InitialTexture->GetName());

		TArray<FString> NamesToTest;
		ImporterSettings->GenerateNormalMapNamesToTest(NormalMapNameNoSuffix, /*inout*/ NamesToTest);
		ImporterSettings->GenerateNormalMapNamesToTest(InitialTexture->GetName(), /*inout*/ NamesToTest);

		// Test each name for an existing asset
		for (const FString& NameToTest : NamesToTest)
		{
			const FString ObjectPathToTest = PackagePath / (NameToTest + FString(TEXT(".")) + NameToTest);
			FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPathToTest));

			if (AssetData.IsValid())
			{
				if (UTexture2D* NormalMapTexture = Cast<UTexture2D>(AssetData.GetAsset()))
				{
					bFoundNormalMap = true;
					SpriteInitParams.AdditionalTextures.Add(NormalMapTexture);
					break;
				}
			}
		}
	}

	ImporterSettings->ApplySettingsForSpriteInit(/*inout*/ SpriteInitParams, bFoundNormalMap ? ESpriteInitMaterialLightingMode::ForceLit : ESpriteInitMaterialLightingMode::Automatic);
	NewSprite->InitializeSprite(SpriteInitParams);

	return NewSprite;
}

#undef LOCTEXT_NAMESPACE

