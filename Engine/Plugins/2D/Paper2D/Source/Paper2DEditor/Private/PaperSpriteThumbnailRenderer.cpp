// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteThumbnailRenderer.h"
#include "CanvasItem.h"
#include "Paper2DModule.h"
#include "CanvasTypes.h"
#include "PaperSprite.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteThumbnailRenderer)

//////////////////////////////////////////////////////////////////////////
// UPaperSpriteThumbnailRenderer

UPaperSpriteThumbnailRenderer::UPaperSpriteThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPaperSpriteThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UPaperSprite* Sprite = Cast<UPaperSprite>(Object);
	DrawFrame(Sprite, X, Y, Width, Height, RenderTarget, Canvas, nullptr);
}

void UPaperSpriteThumbnailRenderer::DrawGrid(int32 X, int32 Y, uint32 Width, uint32 Height, FCanvas* Canvas)
{
	static UTexture2D* GridTexture = nullptr;
	if (GridTexture == nullptr)
	{
		GridTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineMaterials/DefaultDiffuse_TC_Masks.DefaultDiffuse_TC_Masks"), nullptr, LOAD_None, nullptr);
	}

	const bool bAlphaBlend = false;

	Canvas->DrawTile(
		(float)X,
		(float)Y,
		(float)Width,
		(float)Height,
		0.0f,
		0.0f,
		4.0f,
		4.0f,
		FLinearColor::White,
		GridTexture->GetResource(),
		bAlphaBlend);
}

void UPaperSpriteThumbnailRenderer::DrawFrame(class UPaperSprite* Sprite, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, FBoxSphereBounds* OverrideRenderBounds)
{
	const UTexture2D* SourceTexture = nullptr;
	if (Sprite != nullptr)
	{
		SourceTexture = Sprite->GetBakedTexture() ? Sprite->GetBakedTexture() : Sprite->GetSourceTexture();
	}

	if (SourceTexture != nullptr)
	{
		const bool bUseTranslucentBlend = SourceTexture->HasAlphaChannel();

		// Draw the grid behind the sprite
		if (bUseTranslucentBlend)
		{
			DrawGrid(X, Y, Width, Height, Canvas);
		}

		// Draw the sprite itself
		// Use the baked render data, so we don't have to care about rotations and possibly
		// other sprites overlapping in source, UV region, etc.
		const TArray<FVector4>& BakedRenderData = Sprite->BakedRenderData;
		TArray<FVector2D> CanvasPositions;
		TArray<FVector2D> CanvasUVs;

		for (int Vertex = 0; Vertex < BakedRenderData.Num(); ++Vertex)
		{
			new (CanvasPositions) FVector2D(BakedRenderData[Vertex].X, BakedRenderData[Vertex].Y);
			new (CanvasUVs) FVector2D(BakedRenderData[Vertex].Z, BakedRenderData[Vertex].W);
		}

		// Determine the bounds to use
		FBoxSphereBounds* RenderBounds = OverrideRenderBounds;
		FBoxSphereBounds FrameBounds;
		if (RenderBounds == nullptr)
		{
			FrameBounds = Sprite->GetRenderBounds();
			RenderBounds = &FrameBounds;
		}

		const FVector MinPoint3D = RenderBounds->GetBoxExtrema(0);
		const FVector MaxPoint3D = RenderBounds->GetBoxExtrema(1);
		const FVector2D MinPoint(FVector::DotProduct(MinPoint3D, PaperAxisX), FVector::DotProduct(MinPoint3D, PaperAxisY));
		const FVector2D MaxPoint(FVector::DotProduct(MaxPoint3D, PaperAxisX), FVector::DotProduct(MaxPoint3D, PaperAxisY));

		const float UnscaledWidth = MaxPoint.X - MinPoint.X;
		const float UnscaledHeight = MaxPoint.Y - MinPoint.Y;
		const FVector2D Origin(X + Width * 0.5f, Y + Height * 0.5f);
		const bool bIsWider = (UnscaledWidth > 0.0f) && (UnscaledHeight > 0.0f) && (UnscaledWidth > UnscaledHeight);
		const float ScaleFactor = bIsWider ? (Width / UnscaledWidth) : (Height / UnscaledHeight);

		// Scale and recenter
		const FVector2D CanvasPositionCenter = (MaxPoint + MinPoint) * 0.5f;
		for (int Vertex = 0; Vertex < CanvasPositions.Num(); ++Vertex)
		{
			CanvasPositions[Vertex] = (CanvasPositions[Vertex] - CanvasPositionCenter) * ScaleFactor + Origin;
			CanvasPositions[Vertex].Y = Height - CanvasPositions[Vertex].Y;
		}

		// Draw triangles
		if ((CanvasPositions.Num() > 0) && (SourceTexture->GetResource() != nullptr))
		{
			TArray<FCanvasUVTri> Triangles;
			const FLinearColor SpriteColor(FLinearColor::White);
			for (int Vertex = 0; Vertex < CanvasPositions.Num(); Vertex += 3)
			{
				FCanvasUVTri* Triangle = new (Triangles) FCanvasUVTri();
				Triangle->V0_Pos = CanvasPositions[Vertex + 0]; Triangle->V0_UV = CanvasUVs[Vertex + 0]; Triangle->V0_Color = SpriteColor;
				Triangle->V1_Pos = CanvasPositions[Vertex + 1]; Triangle->V1_UV = CanvasUVs[Vertex + 1]; Triangle->V1_Color = SpriteColor;
				Triangle->V2_Pos = CanvasPositions[Vertex + 2]; Triangle->V2_UV = CanvasUVs[Vertex + 2]; Triangle->V2_Color = SpriteColor;
			}
			FCanvasTriangleItem CanvasTriangle(Triangles, SourceTexture->GetResource());
			CanvasTriangle.BlendMode = bUseTranslucentBlend ? ESimpleElementBlendMode::SE_BLEND_Translucent : ESimpleElementBlendMode::SE_BLEND_Opaque;
			Canvas->DrawItem(CanvasTriangle);
		}
	}
	else
	{
		// Fallback for a bogus sprite
		DrawGrid(X, Y, Width, Height, Canvas);
	}
}

