// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperStyle.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyle.h"


#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FPaperStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( FPaperStyle::InContent( RelativePath, ".svg" ), __VA_ARGS__ )
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ...) FSlateBoxBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define DEFAULT_FONT(...) FCoreStyle::GetDefaultFontStyle(__VA_ARGS__)

FString FPaperStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("Paper2D"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FPaperStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FPaperStyle::Get() { return StyleSet; }

FName FPaperStyle::GetStyleSetName()
{
	static FName PaperStyleName(TEXT("PaperStyle"));
	return PaperStyleName;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FPaperStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Shared editors
	{
		StyleSet->Set("Paper2D.Common.ViewportZoomTextStyle", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("BoldCondensed", 16))
			);

		StyleSet->Set("Paper2D.Common.ViewportTitleTextStyle", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 18))
			.SetColorAndOpacity(FLinearColor(1.0, 1.0f, 1.0f, 0.5f))
			);

		StyleSet->Set("Paper2D.Common.ViewportTitleBackground", new BOX_BRUSH("Old/Graph/GraphTitleBackground", FMargin(0)));
	}

	// Tile map editor
	{
		StyleSet->Set("TileMapEditor.EnterTileMapEditMode", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_TileMapEdModeIcon_40x"), Icon40x40));

		StyleSet->Set("TileMapEditor.RotateSelectionCW", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_RotateCW_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.RotateSelectionCW.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_RotateCW_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.RotateSelectionCCW", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_RotateCCW_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.RotateSelectionCCW.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_RotateCCW_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.FlipSelectionHorizontally", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_MirrorHorizontal_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.FlipSelectionHorizontally.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_MirrorHorizontal_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.FlipSelectionVertically", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_MirrorVertical_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.FlipSelectionVertically.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_MirrorVertical_40x"), Icon20x20));

		StyleSet->Set("TileMapEditor.SelectPaintTool", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_PaintBrush_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SelectPaintTool.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_PaintBrush_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SelectEraserTool", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_Eraser_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SelectEraserTool.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_Eraser_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SelectFillTool", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_PaintBucket_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SelectFillTool.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_PaintBucket_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SelectEyeDropperTool", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_EyeDropper_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SelectEyeDropperTool.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_EyeDropper_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SelectTerrainTool", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TerrainPaint_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SelectTerrainTool.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TerrainPaint_40x"), Icon20x20));

		StyleSet->Set("TileMapEditor.AddNewLayerAbove", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_AddNewLayerAbove_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.AddNewLayerAbove.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_AddNewLayerAbove_40x"), Icon20x20));

		StyleSet->Set("TileMapEditor.AddNewLayerBelow", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_AddNewLayerBelow_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.AddNewLayerBelow.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_AddNewLayerBelow_40x"), Icon20x20));

		StyleSet->Set("TileMapEditor.DeleteLayer", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_DeleteLayer_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.DeleteLayer.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_DeleteLayer_40x"), Icon20x20));

// 		StyleSet->Set("TileMapEditor.RenameLayer", new FSlateNoResource(Icon16x16));
// 		StyleSet->Set("TileMapEditor.RenameLayer.Small", new FSlateNoResource(Icon16x16));
		
		StyleSet->Set("TileMapEditor.DuplicateLayer", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_DuplicateLayer_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.DuplicateLayer.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_DuplicateLayer_40x"), Icon20x20));

		StyleSet->Set("TileMapEditor.MergeLayerDown", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_MergeLayerDown_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.MergeLayerDown.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_MergeLayerDown_40x"), Icon20x20));

// 		StyleSet->Set("TileMapEditor.MoveLayerToTop", new FSlateNoResource(Icon16x16));
// 		StyleSet->Set("TileMapEditor.MoveLayerToTop.Small", new FSlateNoResource(Icon16x16));

		StyleSet->Set("TileMapEditor.MoveLayerUp", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_MoveLayerUp_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.MoveLayerUp.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_MoveLayerUp_40x"), Icon20x20));

		StyleSet->Set("TileMapEditor.MoveLayerDown", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_MoveLayerDown_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.MoveLayerDown.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_TileMapEditor_MoveLayerDown_40x"), Icon20x20));

// 		StyleSet->Set("TileMapEditor.MoveLayerToBottom", new FSlateNoResource(Icon16x16));
// 		StyleSet->Set("TileMapEditor.MoveLayerToBottom.Small", new FSlateNoResource(Icon16x16));

		StyleSet->Set("TileMapEditor.LayerEyeClosed", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_EyeClosed_40x"), Icon16x16));
		StyleSet->Set("TileMapEditor.LayerEyeOpened", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_EyeOpened_40x"), Icon16x16));

		const FLinearColor LayerSelectionColor = FLinearColor(0.13f, 0.70f, 1.00f);

		// Selection color for the active row should be ??? to align with the editor viewport.
		const FTableRowStyle& NormalTableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");

		StyleSet->Set("TileMapEditor.LayerBrowser.TableViewRow",
			FTableRowStyle(NormalTableRowStyle)
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, LayerSelectionColor))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, LayerSelectionColor))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, LayerSelectionColor))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, LayerSelectionColor))
			);

		StyleSet->Set("TileMapEditor.LayerBrowser.SelectionColor", LayerSelectionColor);

		StyleSet->Set("TileMapEditor.TileSetPalette.NothingSelectedText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("BoldCondensed", 18))
			.SetColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.0f, 0.8f))
			.SetShadowOffset(FVector2D(1.0f, 1.0f))
			.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.9f))
			);

		StyleSet->Set("TileMapEditor.SetShowPivot", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowPivot_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SetShowPivot.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowPivot_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SetShowGrid", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SetShowGrid.Small", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SetShowTileGrid", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_ShowTileGrid_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SetShowTileGrid.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_ShowTileGrid_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SetShowLayerGrid", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_ShowLayerGrid_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SetShowLayerGrid.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_ShowLayerGrid_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SetShowTileMapStats", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_ShowStats_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SetShowTileMapStats.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileMapEditor/icon_ShowStats_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SetShowBounds", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Bounds_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SetShowBounds.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Bounds_40x"), Icon20x20));
		StyleSet->Set("TileMapEditor.SetShowCollision", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Collision_40x"), Icon40x40));
		StyleSet->Set("TileMapEditor.SetShowCollision.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Collision_40x"), Icon20x20));
	}

	// Tile set editor
	{
		StyleSet->Set("TileSetEditor.SetShowGrid", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon40x40));
		StyleSet->Set("TileSetEditor.SetShowGrid.Small", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon20x20));

		StyleSet->Set("TileSetEditor.SetShowTileStats", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_ShowStats_40x"), Icon40x40));
		StyleSet->Set("TileSetEditor.SetShowTileStats.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_ShowStats_40x"), Icon20x20));

		StyleSet->Set("TileSetEditor.SetShowTilesWithCollision", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_ShowTilesWithCollision_40x"), Icon40x40));
		StyleSet->Set("TileSetEditor.SetShowTilesWithCollision.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_ShowTilesWithCollision_40x"), Icon20x20));
		StyleSet->Set("TileSetEditor.SetShowTilesWithMetaData", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_ShowTilesWithMetadata_40x"), Icon40x40));
		StyleSet->Set("TileSetEditor.SetShowTilesWithMetaData.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_ShowTilesWithMetadata_40x"), Icon20x20));

		StyleSet->Set("TileSetEditor.ApplyCollisionEdits", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_TileSet_Refresh_40x"), Icon40x40));
		StyleSet->Set("TileSetEditor.ApplyCollisionEdits.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_TileSet_Refresh_40x"), Icon20x20));
		
		StyleSet->Set("TileSetEditor.SwapTileSetEditorViewports", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_AlternateView_40x"), Icon40x40));
		StyleSet->Set("TileSetEditor.SwapTileSetEditorViewports.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/TileSetEditor/icon_AlternateView_40x"), Icon20x20));
	}

	// Sprite editor
	{
		StyleSet->Set("SpriteEditor.SetShowGrid", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.SetShowGrid.Small", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon20x20));
		StyleSet->Set("SpriteEditor.SetShowSourceTexture", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_ShowSpriteSheetButton_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.SetShowSourceTexture.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_ShowSpriteSheetButton_40x"), Icon20x20));
		StyleSet->Set("SpriteEditor.SetShowBounds", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Bounds_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.SetShowBounds.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Bounds_40x"), Icon20x20));
		StyleSet->Set("SpriteEditor.SetShowCollision", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Collision_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.SetShowCollision.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Collision_40x"), Icon20x20));

		StyleSet->Set("SpriteEditor.ExtractSprites", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/SpriteEditor/icon_ExtractSprites_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.ExtractSprites.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/SpriteEditor/icon_ExtractSprites_40x"), Icon20x20));
		StyleSet->Set("SpriteEditor.ToggleShowRelatedSprites", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/SpriteEditor/icon_ShowOtherSprites_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.ToggleShowRelatedSprites.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/SpriteEditor/icon_ShowOtherSprites_40x"), Icon20x20));
		StyleSet->Set("SpriteEditor.ToggleShowSpriteNames", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/SpriteEditor/icon_ShowSpriteNames_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.ToggleShowSpriteNames.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/SpriteEditor/icon_ShowSpriteNames_40x"), Icon20x20));

		StyleSet->Set("SpriteEditor.SetShowSockets", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowSockets_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.SetShowSockets.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowSockets_40x"), Icon20x20));
		StyleSet->Set("SpriteEditor.SetShowPivot", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowPivot_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.SetShowPivot.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowPivot_40x"), Icon20x20));

		StyleSet->Set("SpriteEditor.EnterViewMode", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_ViewSprite_40x"), Icon40x40));
		StyleSet->Set("SpriteEditor.EnterViewMode.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_ViewSprite_40x"), Icon20x20));
		StyleSet->Set("SpriteEditor.EnterCollisionEditMode", new IMAGE_PLUGIN_BRUSH("Icons/icon_Paper2D_EditCollision_40x", Icon40x40));
		StyleSet->Set("SpriteEditor.EnterCollisionEditMode.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_Paper2D_EditCollision_40x", Icon20x20));
		StyleSet->Set("SpriteEditor.EnterSourceRegionEditMode", new IMAGE_PLUGIN_BRUSH("Icons/icon_Paper2D_EditSourceRegion_40x", Icon40x40));
		StyleSet->Set("SpriteEditor.EnterSourceRegionEditMode.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_Paper2D_EditSourceRegion_40x", Icon20x20));
		StyleSet->Set("SpriteEditor.EnterRenderingEditMode", new IMAGE_PLUGIN_BRUSH("Icons/icon_Paper2D_RenderGeom_40x", Icon40x40));
		StyleSet->Set("SpriteEditor.EnterRenderingEditMode.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_Paper2D_RenderGeom_40x", Icon20x20));
		StyleSet->Set("SpriteEditor.SpriteActions", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Icons/SpriteActions"), Icon16x16));
	}

	// Sprite geometry editor (shared between the sprite editor, tile set editor, etc...)
	{
		StyleSet->Set("SpriteGeometryEditor.AddBoxShape", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_AddBoxShape_40x"), Icon40x40));
		StyleSet->Set("SpriteGeometryEditor.AddBoxShape.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_AddBoxShape_40x"), Icon20x20));
		StyleSet->Set("SpriteGeometryEditor.AddCircleShape", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_AddCircleShape_40x"), Icon40x40));
		StyleSet->Set("SpriteGeometryEditor.AddCircleShape.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_AddCircleShape_40x"), Icon20x20));

		StyleSet->Set("SpriteGeometryEditor.ToggleAddPolygonMode", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_AddPolygon_40x"), Icon40x40));
		StyleSet->Set("SpriteGeometryEditor.ToggleAddPolygonMode.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_AddPolygon_40x"), Icon20x20));

		StyleSet->Set("SpriteGeometryEditor.SnapAllVertices", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_SnapToPixelGrid_40x"), Icon40x40));
		StyleSet->Set("SpriteGeometryEditor.SnapAllVertices.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Paper2D_SnapToPixelGrid_40x"), Icon20x20));

		StyleSet->Set("SpriteGeometryEditor.SetShowNormals", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Normals_40x"), Icon40x40));
		StyleSet->Set("SpriteGeometryEditor.SetShowNormals.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Normals_40x"), Icon20x20));
	}

	// Flipbook editor
	{
		StyleSet->Set("FlipbookEditor.SetShowGrid", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon40x40));
		StyleSet->Set("FlipbookEditor.SetShowGrid.Small", new IMAGE_BRUSH(TEXT("Icons/icon_MatEd_Grid_40x"), Icon20x20));
		StyleSet->Set("FlipbookEditor.SetShowBounds", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Bounds_40x"), Icon40x40));
		StyleSet->Set("FlipbookEditor.SetShowBounds.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Bounds_40x"), Icon20x20));
		StyleSet->Set("FlipbookEditor.SetShowCollision", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Collision_40x"), Icon40x40));
		StyleSet->Set("FlipbookEditor.SetShowCollision.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_Collision_40x"), Icon20x20));
		StyleSet->Set("FlipbookEditor.SetShowPivot", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowPivot_40x"), Icon40x40));
		StyleSet->Set("FlipbookEditor.SetShowPivot.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowPivot_40x"), Icon20x20));
		StyleSet->Set("FlipbookEditor.SetShowSockets", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowSockets_40x"), Icon40x40));
		StyleSet->Set("FlipbookEditor.SetShowSockets.Small", new IMAGE_BRUSH(TEXT("Icons/icon_StaticMeshEd_ShowSockets_40x"), Icon20x20));

		StyleSet->Set("FlipbookEditor.AddKeyFrame", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/FlipbookEditor/icon_Flipbook_AddKey_40x"), Icon40x40));
		StyleSet->Set("FlipbookEditor.AddKeyFrame.Small", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/FlipbookEditor/icon_Flipbook_AddKey_40x"), Icon20x20));

		StyleSet->Set("FlipbookEditor.RegionGrabHandle", new BOX_BRUSH("Sequencer/ScrubHandleWhole", FMargin(6.f / 13.f, 10 / 24.f, 6 / 13.f, 10 / 24.f)));
		StyleSet->Set("FlipbookEditor.RegionBody", new BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)));
		StyleSet->Set("FlipbookEditor.RegionBorder", new BOX_BRUSH("Common/CurrentCellBorder", FMargin(4.f / 16.f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)));
	}

	// Asset actions
	{
		StyleSet->Set("AssetActions.CreateSprite", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Texture_CreateSprite_16x"), Icon16x16));
		StyleSet->Set("AssetActions.ExtractSprites", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Texture_ExtractSprites_16x"), Icon16x16));
		StyleSet->Set("AssetActions.ConfigureForRetroSprites", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Texture_ConfigureForRetroSprites_16x"), Icon16x16));
		StyleSet->Set("AssetActions.CreateTileSet", new IMAGE_PLUGIN_BRUSH(TEXT("Icons/icon_Texture_CreateTileSet_16x"), Icon16x16));
	}

	{
		// Sprites (asset, component, actor)
		StyleSet->Set("ClassIcon.PaperSprite", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperSprite", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteActor_64", Icon64x64));
		StyleSet->Set("ClassIcon.PaperSpriteComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperSpriteComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteActor_64", Icon64x64));
		StyleSet->Set("ClassIcon.PaperSpriteActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperSpriteActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteActor_64", Icon64x64));

		// Flipbooks (asset, component, actor)
		StyleSet->Set("ClassIcon.PaperFlipbook", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperFlipbookActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperFlipbook", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperFlipbookActor_64", Icon64x64));
		StyleSet->Set("ClassIcon.PaperFlipbookComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperFlipbookActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperFlipbookComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperFlipbookActor_64", Icon64x64));
		StyleSet->Set("ClassIcon.PaperFlipbookActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperFlipbookActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperFlipbookActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperFlipbookActor_64", Icon64x64));

		// Tile maps (asset, component, actor)
		StyleSet->Set("ClassIcon.PaperTileMap", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperTileMap", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_64", Icon64x64));
		StyleSet->Set("ClassIcon.PaperTileMapComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperTileMapComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_64", Icon64x64));
		StyleSet->Set("ClassIcon.PaperTileMapActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperTileMapActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_64", Icon64x64));

		// UPaperSpriteAtlas assets
		//@TODO: Paper2D: These icons don't match the naming scheme
		StyleSet->Set("ClassIcon.PaperSpriteAtlas", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteAtlas_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperSpriteAtlas", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperSpriteAtlas_64", Icon64x64));

		//@TODO: UPaperSpriteSheet icons?

		// APaperCharacter icons
		StyleSet->Set("ClassIcon.PaperCharacter", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperCharacter_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperCharacter", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperCharacter_64", Icon64x64));

		// UPaperTileSet icons
		StyleSet->Set("ClassIcon.PaperTileSet", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperTileSet", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTileMapActor_64", Icon64x64));

		// UPaperTerrainMaterial icons
		StyleSet->Set("ClassIcon.PaperTerrainMaterial", new IMAGE_PLUGIN_BRUSH("Icons/AssetIcons/PaperTerrainMaterial_16x", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperTerrainMaterial", new IMAGE_PLUGIN_BRUSH("Icons/AssetIcons/PaperTerrainMaterial_64x", Icon64x64));

		// Terrain splines (component, actor)
		StyleSet->Set("ClassIcon.PaperTerrainComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTerrainActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperTerrainComponent", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTerrainActor_16", Icon64x64));
		StyleSet->Set("ClassIcon.PaperTerrainActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTerrainActor_16", Icon16x16));
		StyleSet->Set("ClassThumbnail.PaperTerrainActor", new IMAGE_PLUGIN_BRUSH_SVG("Icons/AssetIcons/PaperTerrainActor_64", Icon64x64));
	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef DEFAULT_FONT

void FPaperStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
