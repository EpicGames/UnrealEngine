// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileMapFactory.h"
#include "PaperTileMap.h"
#include "PaperImporterSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileMapFactory)

#define LOCTEXT_NAMESPACE "Paper2D"

/////////////////////////////////////////////////////
// UPaperTileMapFactory

UPaperTileMapFactory::UPaperTileMapFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPaperTileMap::StaticClass();
}

UObject* UPaperTileMapFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPaperTileMap* NewTileMap = NewObject<UPaperTileMap>(InParent, Class, Name, Flags | RF_Transactional);

	GetDefault<UPaperImporterSettings>()->ApplySettingsForTileMapInit(NewTileMap, InitialTileSet);

	return NewTileMap;
}

#undef LOCTEXT_NAMESPACE

