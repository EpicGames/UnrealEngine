// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileMapPromotionFactory.h"
#include "PaperTileMap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileMapPromotionFactory)

#define LOCTEXT_NAMESPACE "Paper2D"

/////////////////////////////////////////////////////
// UPaperTileMapPromotionFactory

UPaperTileMapPromotionFactory::UPaperTileMapPromotionFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = UPaperTileMap::StaticClass();
}

UObject* UPaperTileMapPromotionFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	AssetToRename->SetFlags(Flags | RF_Transactional);
	AssetToRename->Modify();
	AssetToRename->Rename(*Name.ToString(), InParent);

	return AssetToRename;
}

#undef LOCTEXT_NAMESPACE

