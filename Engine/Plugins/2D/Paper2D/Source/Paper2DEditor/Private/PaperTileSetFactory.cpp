// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileSetFactory.h"
#include "PaperTileSet.h"
#include "TileSetEditor/TileSetEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileSetFactory)

#define LOCTEXT_NAMESPACE "Paper2D"

/////////////////////////////////////////////////////
// UPaperTileSetFactory

UPaperTileSetFactory::UPaperTileSetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPaperTileSet::StaticClass();
}

UObject* UPaperTileSetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPaperTileSet* NewTileSet = NewObject<UPaperTileSet>(InParent, Class, Name, Flags | RF_Transactional);
	NewTileSet->SetTileSheetTexture(InitialTexture);
	NewTileSet->SetBackgroundColor(GetDefault<UTileSetEditorSettings>()->DefaultBackgroundColor);

	NewTileSet->PostEditChange();

	return NewTileSet;
}

#undef LOCTEXT_NAMESPACE

