// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTileSetThumbnailRenderer.h"
#include "CanvasTypes.h"
#include "PaperTileSet.h"
#include "TextureResource.h"
#include "GlobalRenderResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTileSetThumbnailRenderer)

//////////////////////////////////////////////////////////////////////////
// UPaperTileSetThumbnailRenderer

UPaperTileSetThumbnailRenderer::UPaperTileSetThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPaperTileSetThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UPaperTileSet* TileSet = Cast<UPaperTileSet>(Object);
	if ((TileSet != nullptr) && (TileSet->GetTileSheetTexture() != nullptr))
	{
		UTexture2D* TileSheetTexture = TileSet->GetTileSheetTexture();
		const bool bUseTranslucentBlend = TileSheetTexture->HasAlphaChannel();

		// Draw the grid behind the sprite
		if (bUseTranslucentBlend)
		{
			static UTexture2D* GridTexture = nullptr;
			if (GridTexture == nullptr)
			{
				GridTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineMaterials/DefaultWhiteGrid.DefaultWhiteGrid"), nullptr, LOAD_None, nullptr);
			}

			const bool bAlphaBlend = false;

			Canvas->DrawTile(
				(float)X,
				(float)Y,
				(float)Width,
				(float)Height,
				0.0f,
				0.0f,
				4.0f,
				4.0f,
				FLinearColor::White,
				GridTexture->GetResource(),
				bAlphaBlend);
		}

		// Draw the sprite itself
		const float TextureWidth = TileSheetTexture->GetSurfaceWidth();
		const float TextureHeight = TileSheetTexture->GetSurfaceHeight();

		const FIntMargin Margin = TileSet->GetMargin();

		float FinalX = (float)X;
		float FinalY = (float)Y;
		float FinalWidth = (float)Width;
		float FinalHeight = (float)Height;
		const float DesiredWidth = TextureWidth - Margin.GetDesiredSize().X;
		const float DesiredHeight = TextureHeight - Margin.GetDesiredSize().Y;

		const FLinearColor BlackBarColor(0.0f, 0.0f, 0.0f, 0.5f);

		if (DesiredWidth > DesiredHeight)
		{
			const float ScaleFactor = Width / DesiredWidth;
			FinalHeight = ScaleFactor * DesiredHeight;
			FinalY += (Height - FinalHeight) * 0.5f;

			// Draw black bars (on top and bottom)
			const bool bAlphaBlend = true;
			Canvas->DrawTile(X, Y, Width, FinalY-Y, 0, 0, 1, 1, BlackBarColor, GWhiteTexture, bAlphaBlend);
			Canvas->DrawTile(X, FinalY+FinalHeight, Width, Height-FinalHeight, 0, 0, 1, 1, BlackBarColor, GWhiteTexture, bAlphaBlend);
		}
		else
		{
			const float ScaleFactor = Height / DesiredHeight;
			FinalWidth = ScaleFactor * DesiredWidth;
			FinalX += (Width - FinalWidth) * 0.5f;

			// Draw black bars (on either side)
			const bool bAlphaBlend = true;
			Canvas->DrawTile(X, Y, FinalX-X, Height, 0, 0, 1, 1, BlackBarColor, GWhiteTexture, bAlphaBlend);
			Canvas->DrawTile(FinalX+FinalWidth, Y, Width-FinalWidth, Height, 0, 0, 1, 1, BlackBarColor, GWhiteTexture, bAlphaBlend);
		}

		// Draw the tile sheet 
		const float InvWidth = 1.0f / TextureWidth;
		const float InvHeight = 1.0f / TextureHeight;
		Canvas->DrawTile(
			FinalX,
			FinalY,
			FinalWidth,
			FinalHeight,
			Margin.Left * InvWidth,
			Margin.Top * InvHeight,
			(TextureWidth - Margin.Right) * InvWidth,
			(TextureHeight - Margin.Bottom) * InvHeight,
			FLinearColor::White,
			TileSheetTexture->GetResource(),
			bUseTranslucentBlend);

		// Draw a label overlay
		//@TODO: Looks very ugly: DrawShadowedStringZ(Canvas, X, Y + Height * 0.8f, 1.0f, TEXT("Tile\nSet"), GEngine->GetSmallFont(), FLinearColor::White);
	}
}

