// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPaperEditorViewport.h"
#include "Math/ColorList.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "PaperEditorViewportClient.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "PaperStyle.h"

const int32 DefaultZoomLevel = 7;
const int32 NumZoomLevels = 17;

struct FZoomLevelEntry
{
	FZoomLevelEntry(float InZoomAmount, const FText& InDisplayText)
		: DisplayText(FText::Format(NSLOCTEXT("PaperEditor", "Zoom", "Zoom {0}"), InDisplayText))
		, ZoomAmount(InZoomAmount)
	{
	}

	FText DisplayText;
	float ZoomAmount;
};

static const FZoomLevelEntry& GetZoomLevel(int Index)
{
	static const FZoomLevelEntry ZoomLevels[NumZoomLevels] =
	{
		FZoomLevelEntry(0.03125f, NSLOCTEXT("PaperEditor", "ZoomLevel_1_32", "1:32")),
		FZoomLevelEntry(0.0625f, NSLOCTEXT("PaperEditor", "ZoomLevel_1_16", "1:16")),
		FZoomLevelEntry(0.125f, NSLOCTEXT("PaperEditor", "ZoomLevel_1_8", "1:8")),
		FZoomLevelEntry(0.250f, NSLOCTEXT("PaperEditor", "ZoomLevel_1_4", "1:4")),
		FZoomLevelEntry(0.500f, NSLOCTEXT("PaperEditor", "ZoomLevel_1_2", "1:2")),
		FZoomLevelEntry(0.750f, NSLOCTEXT("PaperEditor", "ZoomLevel_3_4", "3:4")),
		FZoomLevelEntry(0.875f, NSLOCTEXT("PaperEditor", "ZoomLevel_7_8", "7:8")),
		FZoomLevelEntry(1.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_1_1", "1:1")),
		FZoomLevelEntry(2.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_2x", "2x")),
		FZoomLevelEntry(3.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_3x", "3x")),
		FZoomLevelEntry(4.500f, NSLOCTEXT("PaperEditor", "ZoomLevel_4x", "4x")),
		FZoomLevelEntry(5.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_5x", "5x")),
		FZoomLevelEntry(6.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_6x", "6x")),
		FZoomLevelEntry(7.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_7x", "7x")),
		FZoomLevelEntry(8.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_8x", "8x")),
		FZoomLevelEntry(16.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_16x", "16x")),
		FZoomLevelEntry(32.000f, NSLOCTEXT("PaperEditor", "ZoomLevel_32x", "32x"))
	};
	return ZoomLevels[Index];
}

//////////////////////////////////////////////////////////////////////////
// SPaperEditorViewport

SPaperEditorViewport::~SPaperEditorViewport()
{
	PaperViewportClient.Reset();
}

TSharedRef<FEditorViewportClient> SPaperEditorViewport::MakeEditorViewportClient()
{
	return PaperViewportClient.ToSharedRef();
}

void SPaperEditorViewport::Construct(const FArguments& InArgs, TSharedRef<FPaperEditorViewportClient> InViewportClient)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;

	PaperViewportClient = InViewportClient;
	PaperViewportClient->SetRealtime(false);

	SEditorViewport::Construct(SEditorViewport::FArguments());

	// Indicator of current zoom level
	ViewportOverlay->AddSlot()
		.Padding(5)
		.VAlign(VAlign_Top)
		[
			SNew(STextBlock)
			.TextStyle(FPaperStyle::Get(), "Paper2D.Common.ViewportZoomTextStyle")
			.Text(this, &SPaperEditorViewport::GetZoomText)
			.ColorAndOpacity(this, &SPaperEditorViewport::GetZoomTextColorAndOpacity)
		];

	// Title for the viewport
	ViewportOverlay->AddSlot()
		.VAlign(VAlign_Top)
		[
			SNew(SBorder)
			.BorderImage(FPaperStyle::Get()->GetBrush("Paper2D.Common.ViewportTitleBackground"))
			.HAlign(HAlign_Fill)
			.Visibility(EVisibility::HitTestInvisible)
			[
				SNew(SVerticalBox)
				// Title text/icon
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.TextStyle(FPaperStyle::Get(), "Paper2D.Common.ViewportTitleTextStyle")
						.Text(this, &SPaperEditorViewport::GetTitleText)
					]
				]
			]
		];

	ZoomLevel = DefaultZoomLevel;
	PreviousZoomLevel = DefaultZoomLevel;
	ViewOffset = FVector2D::ZeroVector;
	TotalMouseDelta = 0;

	bIsPanning = false;

	ZoomLevelFade = FCurveSequence( 0.0f, 0.75f );
	ZoomLevelFade.Play(this->AsShared());

	DeferredPanPosition = FVector2D::ZeroVector;
	bRequestDeferredPan = false;
}

void SPaperEditorViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Handle any deferred panning
	if (bRequestDeferredPan)
	{
		bRequestDeferredPan = false;
		UpdateViewOffset(AllottedGeometry, DeferredPanPosition);
	}

	if (!HasMouseCapture())
	{
		bShowSoftwareCursor = false;
		bIsPanning = false;
	}

	PaperViewportClient->SetZoomPos( ViewOffset, GetZoomAmount() );
	PaperViewportClient->bNeedsRedraw = true;

	bool bSelectionModified = false;
	if (Marquee.IsValid())
	{
		bSelectionModified = true;

		OnSelectionChanged.ExecuteIfBound(Marquee, true);
	}

	if (bSelectionModified || bIsPanning || FSlateThrottleManager::Get().IsAllowingExpensiveTasks())
	{
		// Setup the selection set for the viewport
		PaperViewportClient->SelectionRectangles.Empty();

		if (Marquee.IsValid())
		{
			FViewportSelectionRectangle& Rect = *(new (PaperViewportClient->SelectionRectangles) FViewportSelectionRectangle);
			Rect.Color = FColorList::Yellow;
			Rect.Color.A = 0.45f;
			Rect.TopLeft = Marquee.Rect.GetUpperLeft();
			Rect.Dimensions = Marquee.Rect.GetSize();
		}

		// Tick and render the viewport
		PaperViewportClient->Tick(InDeltaTime);
		GEditor->UpdateSingleViewportClient(PaperViewportClient.Get(), /*bInAllowNonRealtimeViewportToDraw=*/ true, /*bLinkedOrthoMovement=*/ false);
	}
}

FReply SPaperEditorViewport::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TotalMouseDelta = 0;

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		// RIGHT BUTTON is for dragging and Context Menu.
		FReply ReplyState = FReply::Handled();
		ReplyState.CaptureMouse( SharedThis(this) );
		ReplyState.UseHighPrecisionMouseMovement( SharedThis(this) );

		SoftwareCursorPosition = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) * MyGeometry.Scale );

		// clear any interpolation when you manually pan
		//DeferredMovementTargetObject = nullptr;

		return ReplyState;
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		// START MARQUEE SELECTION.
		const FVector2D GraphMousePos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) * MyGeometry.Scale );
		Marquee.Start( GraphMousePos, FMarqueeOperation::OperationTypeFromMouseEvent(MouseEvent) );

		// Trigger a selection update now so that single-clicks without a drag still select something
		OnSelectionChanged.ExecuteIfBound(Marquee, true);
		PaperViewportClient->Invalidate();

		return FReply::Handled().CaptureMouse( SharedThis(this) );
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SPaperEditorViewport::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Did the user move the cursor sufficiently far, or is it in a dead zone?
	// In Dead zone     - implies actions like summoning context menus and general clicking.
	// Out of Dead Zone - implies dragging actions like moving nodes and marquee selection.
	const bool bCursorInDeadZone = TotalMouseDelta <= FSlateApplication::Get().GetDragTriggerDistance();

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FReply ReplyState = FReply::Handled();

		if (HasMouseCapture())
		{
			FSlateRect ThisPanelScreenSpaceRect = MyGeometry.GetLayoutBoundingRect();
			const FVector2D ScreenSpaceCursorPos = MyGeometry.LocalToAbsolute( GraphCoordToPanelCoord( SoftwareCursorPosition ) / MyGeometry.Scale );

			FIntPoint BestPositionInViewport(
				FMath::RoundToInt( FMath::Clamp( ScreenSpaceCursorPos.X, ThisPanelScreenSpaceRect.Left, ThisPanelScreenSpaceRect.Right ) ),
				FMath::RoundToInt( FMath::Clamp( ScreenSpaceCursorPos.Y, ThisPanelScreenSpaceRect.Top, ThisPanelScreenSpaceRect.Bottom ) )
				);

			if (!bCursorInDeadZone)
			{
				ReplyState.SetMousePos(BestPositionInViewport);
			}
		}

		TSharedPtr<SWidget> WidgetToFocus;
		if (bCursorInDeadZone)
		{
			//WidgetToFocus = OnSummonContextMenu(MyGeometry, MouseEvent);
		}

		bShowSoftwareCursor = false;

		bIsPanning = false;
		return (WidgetToFocus.IsValid())
			? ReplyState.ReleaseMouseCapture().SetUserFocus(WidgetToFocus.ToSharedRef(), EFocusCause::SetDirectly)
			: ReplyState.ReleaseMouseCapture();
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (OnHandleLeftMouseRelease(MyGeometry, MouseEvent))
		{

		}
		else if (bCursorInDeadZone)
		{
			//@TODO: Move to selection manager
// 			if ( NodeUnderMousePtr.IsValid() )
// 			{
// 				// We clicked on a node!
// 				TSharedRef<SNode> NodeWidgetUnderMouse = NodeUnderMousePtr.Pin().ToSharedRef();
// 
// 				SelectionManager.ClickedOnNode(NodeWidgetUnderMouse->GetObjectBeingDisplayed(), MouseEvent);
// 
// 				// We're done interacting with this node.
// 				NodeUnderMousePtr.Reset();
// 			}
// 			else 
			if (HasMouseCapture())
 			{
 				// We clicked on the panel background

				//@TODO: There isn't a marquee operation for clear, need to signal to remove existing sets too!
				//Marquee.End();
 				//OnSelectionChanged.ExecuteIfBound(Marquee, false);
 			}
		}
		else if (Marquee.IsValid())
		{
			//ApplyMarqueeSelection(Marquee, SelectionManager.SelectedNodes, SelectionManager.SelectedNodes);
			//SelectionManager.OnSelectionChanged.ExecuteIfBound(SelectionManager.SelectedNodes);
			OnSelectionChanged.ExecuteIfBound(Marquee, true);
		}

		// The existing marquee operation ended; reset it.
		Marquee = FMarqueeOperation();

		if (bIsPanning)
		{
			// We have released the left mouse button. But we're still panning
			// (i.e. RIGHT MOUSE is down), so we want to hold on to capturing mouse input.
			return FReply::Handled();
		}
		else
		{
			// We aren't panning, so we can release the mouse capture.
			return FReply::Handled().ReleaseMouseCapture();
		}	
	}

	return FReply::Unhandled();	
}

FReply SPaperEditorViewport::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
	const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	
	if (HasMouseCapture())
	{
		// Track how much the mouse moved since the mouse down.
		const FVector2D CursorDelta = MouseEvent.GetCursorDelta();
		TotalMouseDelta += CursorDelta.Size();

		if (bIsRightMouseButtonDown)
		{
			FReply ReplyState = FReply::Handled();

			if (!CursorDelta.IsZero())
			{
				bShowSoftwareCursor = true;
			}

			bIsPanning = true;
			ViewOffset -= CursorDelta / GetZoomAmount();

			return ReplyState;
		}
		else if (bIsLeftMouseButtonDown)
		{
//			TSharedPtr<SNode> NodeBeingDragged = NodeUnderMousePtr.Pin();

			// Update the amount to pan panel
			UpdateViewOffset(MyGeometry, MouseEvent.GetScreenSpacePosition());

			const bool bCursorInDeadZone = TotalMouseDelta <= FSlateApplication::Get().GetDragTriggerDistance();

			{
				// We are marquee selecting
				const FVector2D GraphMousePos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) * MyGeometry.Scale );
				Marquee.Rect.UpdateEndPoint(GraphMousePos);

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SPaperEditorViewport::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We want to zoom into this point; i.e. keep it the same fraction offset into the panel
	const FVector2D WidgetSpaceCursorPos = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) * MyGeometry.Scale;
	FVector2D PointToMaintainGraphSpace = PanelCoordToGraphCoord( WidgetSpaceCursorPos );


	const int32 ZoomLevelDelta = FMath::FloorToInt(MouseEvent.GetWheelDelta());

	const bool bAllowFullZoomRange = true;
	const float OldZoomLevel = ZoomLevel;

	if (bAllowFullZoomRange)
	{
		ZoomLevel = FMath::Clamp( ZoomLevel + ZoomLevelDelta, 0, NumZoomLevels-1 );
	}
	else
	{
		// Without control, we do not allow zooming out past 1:1.
		ZoomLevel = FMath::Clamp( ZoomLevel + ZoomLevelDelta, 0, DefaultZoomLevel );
	}

	ZoomLevelFade.Play(this->AsShared());


	// Re-center the screen so that it feels like zooming around the cursor.
	ViewOffset = PointToMaintainGraphSpace - WidgetSpaceCursorPos / GetZoomAmount();

	return FReply::Handled();
}

FCursorReply SPaperEditorViewport::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	return bShowSoftwareCursor ? 
		FCursorReply::Cursor( EMouseCursor::None ) :
		FCursorReply::Cursor( EMouseCursor::Default );
}

int32 SPaperEditorViewport::FindNearestZoomLevel(float InZoomAmount, bool bRoundDown) const
{
	bool bFoundItem = false;
	int32 Result = 0;

	for (int32 ZoomLevelIndex=0; ZoomLevelIndex < NumZoomLevels; ++ZoomLevelIndex)
	{
		if (InZoomAmount <= GetZoomLevel(ZoomLevelIndex).ZoomAmount)
		{
			Result = ZoomLevelIndex;
			bFoundItem = true;
			break;
		}
	}

	if (!bFoundItem)
	{
		return DefaultZoomLevel;
	}
	else
	{
		if (bFoundItem && bRoundDown && (Result > 0))
		{
			return Result - 1;
		}
		else
		{
			return Result;
		}
	}
}

float SPaperEditorViewport::GetZoomAmount() const
{
	return GetZoomLevel(ZoomLevel).ZoomAmount;
}

FText SPaperEditorViewport::GetZoomText() const
{
	return GetZoomLevel(ZoomLevel).DisplayText;
}

FSlateColor SPaperEditorViewport::GetZoomTextColorAndOpacity() const
{
	return FLinearColor( 1, 1, 1, 1.25f - ZoomLevelFade.GetLerp() * 0.75f );
}

FVector2D SPaperEditorViewport::GetViewOffset() const
{
	return ViewOffset;
}

FVector2D SPaperEditorViewport::ComputeEdgePanAmount(const FGeometry& MyGeometry, const FVector2D& TargetPosition)
{
	// How quickly to ramp up the pan speed as the user moves the mouse further past the edge of the graph panel.
	static const float EdgePanSpeedCoefficient = 0.1f;

	// Never pan slower than this, it's just unpleasant.
	static const float MinPanSpeed = 5.0f;

	// Start panning before we reach the edge of the graph panel.
	static const float EdgePanForgivenessZone = 30.0f;

	const FVector2D LocalCursorPos = MyGeometry.AbsoluteToLocal( TargetPosition );

	// If the mouse is outside of the graph area, then we want to pan in that direction.
	// The farther out the mouse is, the more we want to pan.

	FVector2D EdgePanThisTick(0,0);
	if ( LocalCursorPos.X <= EdgePanForgivenessZone )
	{
		EdgePanThisTick.X += FMath::Min( -MinPanSpeed, EdgePanSpeedCoefficient * (EdgePanForgivenessZone - LocalCursorPos.X) );
	}
	else if( LocalCursorPos.X >= MyGeometry.Size.X - EdgePanForgivenessZone )
	{
		EdgePanThisTick.X = FMath::Max( MinPanSpeed, EdgePanSpeedCoefficient * (MyGeometry.Size.X - EdgePanForgivenessZone - LocalCursorPos.X) );
	}

	if ( LocalCursorPos.Y <= EdgePanForgivenessZone )
	{
		EdgePanThisTick.Y += FMath::Min( -MinPanSpeed, EdgePanSpeedCoefficient * (EdgePanForgivenessZone - LocalCursorPos.Y) );
	}
	else if( LocalCursorPos.Y >= MyGeometry.Size.Y - EdgePanForgivenessZone )
	{
		EdgePanThisTick.Y = FMath::Max( MinPanSpeed, EdgePanSpeedCoefficient * (MyGeometry.Size.Y - EdgePanForgivenessZone - LocalCursorPos.Y) );
	}

	return EdgePanThisTick;
}

void SPaperEditorViewport::UpdateViewOffset(const FGeometry& MyGeometry, const FVector2D& TargetPosition)
{
	const FVector2D PanAmount = ComputeEdgePanAmount( MyGeometry, TargetPosition ) / GetZoomAmount();
	ViewOffset += PanAmount;
}

void SPaperEditorViewport::RequestDeferredPan(const FVector2D& UpdatePosition)
{
	bRequestDeferredPan = true;
	DeferredPanPosition = UpdatePosition;
}

FVector2D SPaperEditorViewport::GraphCoordToPanelCoord( const FVector2D& GraphSpaceCoordinate ) const
{
	return (GraphSpaceCoordinate - GetViewOffset()) * GetZoomAmount();
}

FVector2D SPaperEditorViewport::PanelCoordToGraphCoord( const FVector2D& PanelSpaceCoordinate ) const
{
	return PanelSpaceCoordinate / GetZoomAmount() + GetViewOffset();
}

FSlateRect SPaperEditorViewport::PanelRectToGraphRect( const FSlateRect& PanelSpaceRect ) const
{
	FVector2D UpperLeft = PanelCoordToGraphCoord( FVector2D(PanelSpaceRect.Left, PanelSpaceRect.Top) );
	FVector2D LowerRight = PanelCoordToGraphCoord(  FVector2D(PanelSpaceRect.Right, PanelSpaceRect.Bottom) );

	return FSlateRect(
		UpperLeft.X, UpperLeft.Y,
		LowerRight.X, LowerRight.Y );
}

void SPaperEditorViewport::PaintSoftwareCursor(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const
{
	if (bShowSoftwareCursor)
	{
		const FSlateBrush* Brush = FAppStyle::GetBrush(TEXT("SoftwareCursor_Grab"));
		const FVector2D CursorSize = Brush->ImageSize / AllottedGeometry.Scale;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			DrawLayerId,
			AllottedGeometry.ToPaintGeometry( CursorSize, FSlateLayoutTransform(GraphCoordToPanelCoord(SoftwareCursorPosition) / AllottedGeometry.Scale - ( CursorSize * .5f )) ),
			Brush);
	}
}

int32 SPaperEditorViewport::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 MaxLayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	++MaxLayerId;
	PaintSoftwareCursor(AllottedGeometry, MyCullingRect, OutDrawElements, MaxLayerId);

	return MaxLayerId;
}

FText SPaperEditorViewport::GetTitleText() const
{
	return NSLOCTEXT("PaperEditor", "TileSetPaletteTitle", "tile set palette");
}
