// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "SEditorViewport.h"
#include "Framework/MarqueeRect.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FMarqueeOperation;
struct Rect;

/** Helper for managing marquee operations */
struct FMarqueeOperation
{
	FMarqueeOperation()
		: Operation(Add)
	{
	}

	enum Type
	{
		/** Holding down Ctrl toggles affected nodes */
		Toggle,
		/** Holding down Shift adds to the selection */
		Add,
		/** When nothing is pressed, marquee replaces selection */
		Replace
	} Operation;

	bool IsValid() const
	{
		return Rect.IsValid();
	}

	void Start(const FVector2D& InStartLocation, FMarqueeOperation::Type InOperationType)
	{
		Rect = FMarqueeRect(InStartLocation);
		Operation = InOperationType;
	}

	void End()
	{
		Rect = FMarqueeRect();
	}


	/** Given a mouse event, figure out what the marquee selection should do based on the state of Shift and Ctrl keys */
	static FMarqueeOperation::Type OperationTypeFromMouseEvent(const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.IsControlDown())
		{
			return FMarqueeOperation::Toggle;
		}
		else if (MouseEvent.IsShiftDown())
		{
			return FMarqueeOperation::Add;
		}
		else
		{
			return FMarqueeOperation::Replace;
		}
	}

public:
	/** The marquee rectangle being dragged by the user */
	FMarqueeRect Rect;

	/** Nodes that will be selected or unselected by the current marquee operation */
	TSet<class UObject*> AffectedNodes;
};

//////////////////////////////////////////////////////////////////////////
// SPaperEditorViewport

class SPaperEditorViewport : public SEditorViewport
{
public:
	DECLARE_DELEGATE_TwoParams(FOnSelectionChanged, FMarqueeOperation /*MarqueeAction*/, bool /*bPreview*/);
public:
	SLATE_BEGIN_ARGS(SPaperEditorViewport) {}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
	SLATE_END_ARGS()

	/** Destructor */
	~SPaperEditorViewport();

	// SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	// End of SWidget interface

	void Construct(const FArguments& InArgs, TSharedRef<class FPaperEditorViewportClient> InViewportClient);

	float GetZoomAmount() const;
	FText GetZoomText() const;
	FSlateColor GetZoomTextColorAndOpacity() const;
	FVector2D GetViewOffset() const;

protected:

	// SEditorViewport interface
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	// End of SEditorViewport interface


	int32 FindNearestZoomLevel(float InZoomAmount, bool bRoundDown) const;


	FVector2D ComputeEdgePanAmount(const FGeometry& MyGeometry, const FVector2D& TargetPosition);
	void UpdateViewOffset(const FGeometry& MyGeometry, const FVector2D& TargetPosition);
	void RequestDeferredPan(const FVector2D& UpdatePosition);
	FVector2D GraphCoordToPanelCoord(const FVector2D& GraphSpaceCoordinate) const;
	FVector2D PanelCoordToGraphCoord(const FVector2D& PanelSpaceCoordinate) const;
	FSlateRect PanelRectToGraphRect(const FSlateRect& PanelSpaceRect) const;
	virtual bool OnHandleLeftMouseRelease(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return false; }

	void PaintSoftwareCursor(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const;

	virtual FText GetTitleText() const;
protected:
	/** The position within the graph at which the user is looking */
	FVector2D ViewOffset;

	/** How zoomed in/out we are. e.g. 0.25f results in quarter-sized nodes. */
	int32 ZoomLevel;

	/** Previous Zoom Level */
	int32 PreviousZoomLevel;

	/** Are we panning the view at the moment? */
	bool bIsPanning;

	/** The total distance that the mouse has been dragged while down */
	float TotalMouseDelta;

	/** A pending marquee operation if it's active */
	FMarqueeOperation Marquee;

	/** Curve that handles fading the 'Zoom +X' text */
	FCurveSequence ZoomLevelFade;

	/** Position to pan to */
	FVector2D DeferredPanPosition;

	/** true if pending request for deferred panning */
	bool bRequestDeferredPan;

	/**	The current position of the software cursor */
	FVector2D SoftwareCursorPosition;

	/**	Whether the software cursor should be drawn */
	bool bShowSoftwareCursor;

	/** Level viewport client */
	TSharedPtr<class FPaperEditorViewportClient> PaperViewportClient;

	// Selection changed delegate
	FOnSelectionChanged OnSelectionChanged;
};
