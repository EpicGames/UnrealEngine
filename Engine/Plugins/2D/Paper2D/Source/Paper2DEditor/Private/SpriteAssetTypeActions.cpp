// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteAssetTypeActions.h"
#include "Misc/PackageName.h"
#include "Misc/FeedbackContext.h"

#include "SpriteEditor/SpriteEditor.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "PaperFlipbookHelpers.h"
#include "PaperFlipbookFactory.h"
#include "PaperSprite.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

//////////////////////////////////////////////////////////////////////////
// FSpriteAssetTypeActions

FSpriteAssetTypeActions::FSpriteAssetTypeActions(EAssetTypeCategories::Type InAssetCategory)
	: MyAssetCategory(InAssetCategory)
{
}

FText FSpriteAssetTypeActions::GetName() const
{
	return LOCTEXT("FSpriteAssetTypeActionsName", "Sprite");
}

FColor FSpriteAssetTypeActions::GetTypeColor() const
{
	return FColor::Cyan;
}

UClass* FSpriteAssetTypeActions::GetSupportedClass() const
{
	return UPaperSprite::StaticClass();
}

void FSpriteAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	const EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UPaperSprite* Sprite = Cast<UPaperSprite>(*ObjIt))
		{
			TSharedRef<FSpriteEditor> NewSpriteEditor(new FSpriteEditor());
			NewSpriteEditor->InitSpriteEditor(Mode, EditWithinLevelEditor, Sprite);
		}
	}
}

uint32 FSpriteAssetTypeActions::GetCategories()
{
	return MyAssetCategory;
}

void FSpriteAssetTypeActions::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	auto Sprites = GetTypedWeakObjectPtrs<UPaperSprite>(InObjects);

	Section.AddMenuEntry(
		"Sprite_CreateFlipbook",
		LOCTEXT("Sprite_CreateFlipbook", "Create Flipbook"),
		LOCTEXT("Sprite_CreateFlipbookTooltip", "Creates flipbooks from the selected sprites."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PaperFlipbook"),
		FUIAction(
		FExecuteAction::CreateSP(this, &FSpriteAssetTypeActions::ExecuteCreateFlipbook, Sprites),
		FCanExecuteAction()
		)
	);
}

//////////////////////////////////////////////////////////////////////////


void FSpriteAssetTypeActions::ExecuteCreateFlipbook(TArray<TWeakObjectPtr<UPaperSprite>> Objects)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<UPaperSprite*> AllSprites;

	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		UPaperSprite* Object = (*ObjIt).Get();
		if (Object && Object->IsValidLowLevel())
		{
			AllSprites.Add(Object);
		}
	}

	TMap<FString, TArray<UPaperSprite*> > SpriteFlipbookMap;
	FPaperFlipbookHelpers::ExtractFlipbooksFromSprites(/*out*/SpriteFlipbookMap, AllSprites, TArray<FString>());
	TArray<UObject*> ObjectsToSync;

	if (SpriteFlipbookMap.Num() > 0)
	{
		GWarn->BeginSlowTask(NSLOCTEXT("Paper2D", "Paper2D_CreateFlipbooks", "Creating flipbooks from selection"), true, true);

		int Progress = 0;
		int TotalProgress = SpriteFlipbookMap.Num();

		// Create the flipbook
		bool bOneFlipbookCreated = SpriteFlipbookMap.Num() == 1;
		for (auto Iter : SpriteFlipbookMap)
		{
			GWarn->UpdateProgress(Progress++, TotalProgress);

			const FString& FlipbookName = Iter.Key;
			TArray<UPaperSprite*> Sprites = Iter.Value;

			const FString SpritePathName = AllSprites[0]->GetOutermost()->GetPathName();
			const FString LongPackagePath = FPackageName::GetLongPackagePath(AllSprites[0]->GetOutermost()->GetPathName());

			const FString NewFlipBookDefaultPath = LongPackagePath + TEXT("/") + FlipbookName;
			FString DefaultSuffix;
			FString AssetName;
			FString PackageName;

			UPaperFlipbookFactory* FlipbookFactory = NewObject<UPaperFlipbookFactory>();
			for (int32 SpriteIndex = 0; SpriteIndex < Sprites.Num(); ++SpriteIndex)
			{
				UPaperSprite* Sprite = Sprites[SpriteIndex];
				FPaperFlipbookKeyFrame* KeyFrame = new (FlipbookFactory->KeyFrames) FPaperFlipbookKeyFrame();
				KeyFrame->Sprite = Sprite;
				KeyFrame->FrameRun = 1;
			}

			AssetToolsModule.Get().CreateUniqueAssetName(NewFlipBookDefaultPath, DefaultSuffix, /*out*/ PackageName, /*out*/ AssetName);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
			if (bOneFlipbookCreated)
			{
				ContentBrowserModule.Get().CreateNewAsset(AssetName, PackagePath, UPaperFlipbook::StaticClass(), FlipbookFactory);
			}
			else
			{
				if (UObject* NewAsset = AssetToolsModule.Get().CreateAsset(AssetName, PackagePath, UPaperFlipbook::StaticClass(), FlipbookFactory))
				{
					ObjectsToSync.Add(NewAsset);
				}
			}

			if (GWarn->ReceivedUserCancel())
			{
				break;
			}
		}

		GWarn->EndSlowTask();

		if (ObjectsToSync.Num() > 0)
		{
			ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

}


#undef LOCTEXT_NAMESPACE
