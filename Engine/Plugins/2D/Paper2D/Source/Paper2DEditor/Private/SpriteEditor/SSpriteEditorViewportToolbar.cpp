// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SSpriteEditorViewportToolbar.h"
#include "Framework/Commands/UICommandList.h"
#include "SpriteEditor/SpriteEditorCommands.h"
#include "PaperEditorShared/SpriteGeometryEditCommands.h"
#include "SEditorViewport.h"

#define LOCTEXT_NAMESPACE "SSpriteEditorViewportToolbar"

///////////////////////////////////////////////////////////
// SSpriteEditorViewportToolbar

void SSpriteEditorViewportToolbar::Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider)
{
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InInfoProvider);
}

TSharedRef<SWidget> SSpriteEditorViewportToolbar::GenerateShowMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();
	
	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		ShowMenuBuilder.AddMenuEntry(FSpriteEditorCommands::Get().SetShowSockets);
		ShowMenuBuilder.AddMenuEntry(FSpriteEditorCommands::Get().SetShowPivot);

		ShowMenuBuilder.AddMenuSeparator();

		ShowMenuBuilder.AddMenuEntry(FSpriteEditorCommands::Get().SetShowGrid);
		ShowMenuBuilder.AddMenuEntry(FSpriteEditorCommands::Get().SetShowBounds);
		ShowMenuBuilder.AddMenuEntry(FSpriteGeometryEditCommands::Get().SetShowNormals);

		ShowMenuBuilder.AddMenuSeparator();

		ShowMenuBuilder.AddMenuEntry(FSpriteEditorCommands::Get().SetShowCollision);
		ShowMenuBuilder.AddMenuEntry(FSpriteEditorCommands::Get().SetShowMeshEdges);
	}

	return ShowMenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
