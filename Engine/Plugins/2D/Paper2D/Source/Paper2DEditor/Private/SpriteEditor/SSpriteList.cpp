// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SSpriteList.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "PaperSprite.h"
#include "SpriteEditor/SpriteEditor.h"

//////////////////////////////////////////////////////////////////////////
// SSpriteList

void SSpriteList::Construct(const FArguments& InArgs, TSharedPtr<FSpriteEditor> InSpriteEditor)
{
	SpriteEditorPtr = InSpriteEditor;
	
	UTexture2D* CurrentTexture = InSpriteEditor->GetSourceTexture();
	SourceTexturePtr = CurrentTexture;

	RebuildWidget(CurrentTexture);
}

void SSpriteList::RebuildWidget(UTexture2D* NewTextureFilter)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	// Configure filter for asset picker
	FAssetPickerConfig Config;

	//Config.Filter.bRecursiveClasses = true;
	Config.Filter.ClassPaths.Add(UPaperSprite::StaticClass()->GetClassPathName());

	Config.InitialAssetViewType = EAssetViewType::Tile;
	Config.SyncToAssetsDelegates.Add(&SyncToAssetsDelegate);

	if (NewTextureFilter != nullptr)
	{
		const FName SourceTexturePropName(TEXT("SourceTexture")); //@TODO: Protected: GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceTexture);
		Config.Filter.TagsAndValues.Add(SourceTexturePropName, TSoftObjectPtr<UTexture2D>(NewTextureFilter).ToString());
		// Legacy format for files that haven't been resaved
		Config.Filter.TagsAndValues.Add(SourceTexturePropName, FAssetData(NewTextureFilter).GetExportTextName());

		// Ignore the source texture since we're filtering against it
		AssetRegistryTagsToIgnore.Add(SourceTexturePropName);
	}
	else
	{
		AssetRegistryTagsToIgnore.Empty();
	}

	// Configure response to click and double-click
	Config.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SSpriteList::OnSpriteSelected);
	Config.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateSP(this, &SSpriteList::OnSpriteDoubleClicked);
	//Config.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SSpriteList::OnGetAssetContextMenu);
	Config.OnAssetTagWantsToBeDisplayed = FOnShouldDisplayAssetTag::CreateSP(this, &SSpriteList::CanShowColumnForAssetRegistryTag);
	Config.bFocusSearchBoxWhenOpened = false;

	this->ChildSlot
	[
		ContentBrowserModule.Get().CreateAssetPicker(Config)
	];
}

void SSpriteList::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	TSharedPtr<FSpriteEditor> SpriteEditor = SpriteEditorPtr.Pin();
	if (SpriteEditor.IsValid())
	{
		// Check to see if the source texture has changed; if so, we need to rebuild the list
		UTexture2D* CurrentTexture = SpriteEditor->GetSourceTexture();
		if (CurrentTexture != SourceTexturePtr.Get())
		{
			SourceTexturePtr = CurrentTexture;
			RebuildWidget(CurrentTexture);
		}
	}
}

void SSpriteList::OnSpriteSelected(const FAssetData& AssetData)
{
}

void SSpriteList::OnSpriteDoubleClicked(const FAssetData& AssetData)
{
	TSharedPtr<FSpriteEditor> SpriteEditor = SpriteEditorPtr.Pin();
	if (SpriteEditor.IsValid())
	{
		if (UPaperSprite* NewSpriteToEdit = Cast<UPaperSprite>(AssetData.GetAsset()))
		{
			SpriteEditor->SetSpriteBeingEdited(NewSpriteToEdit);
		}
	}
}

bool SSpriteList::CanShowColumnForAssetRegistryTag(FName AssetType, FName TagName) const
{
	return !AssetRegistryTagsToIgnore.Contains(TagName);
}

void SSpriteList::SelectAsset(UObject* Asset)
{
	FAssetData AssetData(Asset);

	if (AssetData.IsValid())
	{
		TArray<FAssetData> AssetsToSelect;
		AssetsToSelect.Add(AssetData);

		SyncToAssetsDelegate.Execute(AssetsToSelect);
	}
}
