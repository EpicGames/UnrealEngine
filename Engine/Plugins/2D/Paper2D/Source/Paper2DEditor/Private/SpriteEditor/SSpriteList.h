// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "ContentBrowserDelegates.h"

class UTexture2D;

//////////////////////////////////////////////////////////////////////////
// SSpriteList

class SSpriteList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSpriteList){}
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, TSharedPtr<class FSpriteEditor> InSpriteEditor);

	// SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	void SelectAsset(UObject* Asset);

protected:
	void RebuildWidget(UTexture2D* NewTextureFilter);

	void OnSpriteSelected(const struct FAssetData& AssetData);
	void OnSpriteDoubleClicked(const struct FAssetData& AssetData);
	bool CanShowColumnForAssetRegistryTag(FName AssetType, FName TagName) const;

protected:
	// Last source texture we saw (used to discern when the texture has changed)
	TWeakObjectPtr<UTexture2D> SourceTexturePtr;

	// Pointer back to owning sprite editor instance (the keeper of state)
	TWeakPtr<class FSpriteEditor> SpriteEditorPtr;

	// Set of tags to prevent creating details view columns for (infrequently used)
	TSet<FName> AssetRegistryTagsToIgnore;

	// Delegate to sync the asset picker to selected assets
	FSyncToAssetsDelegate SyncToAssetsDelegate;
};
