// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SpriteComponentDetailsCustomization.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "GroupedSprites/PaperGroupedSpriteUtilities.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////
// FSpriteComponentDetailsCustomization

TSharedRef<IDetailCustomization> FSpriteComponentDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FSpriteComponentDetailsCustomization);
}

void FSpriteComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Create a category so this is displayed early in the properties
	IDetailCategoryBuilder& SpriteCategory = DetailBuilder.EditCategory("Sprite", FText::GetEmpty(), ECategoryPriority::Important);

	ObjectsBeingCustomized.Empty();
	DetailBuilder.GetObjectsBeingCustomized(/*out*/ ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() > 1)
	{
		// Expose merge buttons
		FDetailWidgetRow& MergeRow = SpriteCategory.AddCustomRow(LOCTEXT("MergeSearchText", "Merge"))
			.WholeRowContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(SButton)
					.Text(LOCTEXT("MergeSprites", "Merge Sprites"))
					.ToolTipText(LOCTEXT("MergeSprites_Tooltip", "Merges all selected sprite components into entries on a single grouped sprite component"))
					.OnClicked(this, &FSpriteComponentDetailsCustomization::MergeSprites)
				]
			];
	}
}

FReply FSpriteComponentDetailsCustomization::MergeSprites()
{
	TArray<UObject*> StrongObjects;
	CopyFromWeakArray(StrongObjects, ObjectsBeingCustomized);

	FPaperGroupedSpriteUtilities::MergeSprites(StrongObjects);

	return FReply::Handled();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
