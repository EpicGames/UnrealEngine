// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SpriteDetailsCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Materials/Material.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "SpriteEditor/SpriteEditorViewportClient.h"

#include "PhysicsEngine/BodySetup.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "PropertyRestriction.h"
#include "PropertyCustomizationHelpers.h"
#include "MaterialExpressionSpriteTextureSampler.h"
#include "PaperRuntimeSettings.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////
// FSpriteDetailsCustomization

TSharedRef<IDetailCustomization> FSpriteDetailsCustomization::MakeInstance()
{
	TAttribute<ESpriteEditorMode::Type> DummyEditMode(ESpriteEditorMode::ViewMode);
	return MakeInstanceForSpriteEditor(DummyEditMode);
}

TSharedRef<IDetailCustomization> FSpriteDetailsCustomization::MakeInstanceForSpriteEditor(TAttribute<ESpriteEditorMode::Type> InEditMode)
{
	return MakeShareable(new FSpriteDetailsCustomization(InEditMode));
}

FSpriteDetailsCustomization::FSpriteDetailsCustomization(TAttribute<ESpriteEditorMode::Type> InEditMode)
{
	SpriteEditMode = InEditMode;
}

FDetailWidgetRow& FSpriteDetailsCustomization::GenerateWarningRow(IDetailCategoryBuilder& WarningCategory, bool bExperimental, const FText& WarningText, const FText& Tooltip, const FString& ExcerptLink, const FString& ExcerptName)
{
	const FText SearchString = WarningText;
	const FSlateBrush* WarningIcon = FAppStyle::GetBrush(bExperimental ? "PropertyEditor.ExperimentalClass" : "PropertyEditor.EarlyAccessClass");

	FDetailWidgetRow& WarningRow = WarningCategory.AddCustomRow(SearchString)
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			.ToolTip(IDocumentation::Get()->CreateToolTip(Tooltip, nullptr, ExcerptLink, ExcerptName))
			.Visibility(EVisibility::Visible)

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[	SNew(SImage)
				.Image(WarningIcon)
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(WarningText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	return WarningRow;
}

void FSpriteDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Make sure sprite properties are near the top
	IDetailCategoryBuilder& SpriteCategory = DetailLayout.EditCategory("Sprite", FText::GetEmpty(), ECategoryPriority::Important);
	BuildSpriteSection(SpriteCategory, DetailLayout);

	// Build the socket category
	{
		IDetailCategoryBuilder& SocketCategory = DetailLayout.EditCategory("Sockets");
		SocketCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, Sockets));
	}

	// Build the collision category
	IDetailCategoryBuilder& CollisionCategory = DetailLayout.EditCategory("Collision");
	BuildCollisionSection(CollisionCategory, DetailLayout);

	// Build the rendering category
	IDetailCategoryBuilder& RenderingCategory = DetailLayout.EditCategory("Rendering");
	BuildRenderingSection(RenderingCategory, DetailLayout);
}

void FSpriteDetailsCustomization::BuildSpriteSection(IDetailCategoryBuilder& SpriteCategory, IDetailLayoutBuilder& DetailLayout)
{
	// Show other normal properties in the sprite category so that desired ordering doesn't get messed up
	SpriteCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceUV));
	SpriteCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceDimension));
	BuildTextureSection(SpriteCategory, DetailLayout);

	SpriteCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, DefaultMaterial));
	SpriteCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, PixelsPerUnrealUnit));

	// Show/hide the experimental atlas group support based on whether or not it is enabled
	TSharedPtr<IPropertyHandle> AtlasGroupProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, AtlasGroup));
	TAttribute<EVisibility> AtlasGroupPropertyVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FSpriteDetailsCustomization::GetAtlasGroupVisibility));
	SpriteCategory.AddProperty(AtlasGroupProperty, EPropertyLocation::Advanced).Visibility(AtlasGroupPropertyVisibility);

	// Show/hide the custom pivot point based on the pivot mode
	TSharedPtr<IPropertyHandle> PivotModeProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, PivotMode));
	TSharedPtr<IPropertyHandle> CustomPivotPointProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, CustomPivotPoint));
	TAttribute<EVisibility> CustomPivotPointVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSpriteDetailsCustomization::GetCustomPivotVisibility, PivotModeProperty));
	SpriteCategory.AddProperty(PivotModeProperty);
	SpriteCategory.AddProperty(CustomPivotPointProperty).Visibility(CustomPivotPointVisibility);
}

void FSpriteDetailsCustomization::BuildRenderingSection(IDetailCategoryBuilder& RenderingCategory, IDetailLayoutBuilder& DetailLayout)
{
	TAttribute<EVisibility> HideWhenInCollisionMode = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSpriteDetailsCustomization::EditorModeIsNot, ESpriteEditorMode::EditCollisionMode));
	TAttribute<EVisibility> ShowWhenInCollisionMode = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSpriteDetailsCustomization::EditorModeMatches, ESpriteEditorMode::EditCollisionMode));

	static const FText EditRenderingInRenderingMode = LOCTEXT("RenderingPropertiesHiddenInCollisionMode", "Switch to 'Edit RenderGeom' mode\nto edit Rendering settings");
	RenderingCategory.AddCustomRow(EditRenderingInRenderingMode)
		.Visibility(ShowWhenInCollisionMode)
		.WholeRowContent()
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailLayout.GetDetailFontItalic())
			.Justification(ETextJustify::Center)
			.Text(EditRenderingInRenderingMode)
		];

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailLayout.GetObjectsBeingCustomized(/*out*/ ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() > 0)
	{
		if (UPaperSprite* SpriteBeingEdited = Cast<UPaperSprite>(ObjectsBeingCustomized[0].Get()))
		{
			static const FText TypesOfMaterialsTooltip = LOCTEXT("TypesOfMaterialsTooltip", "Translucent materials can have smooth alpha edges, blending with the background\nMasked materials have on or off alpha, useful for cutouts\nOpaque materials have no transparency but render faster");

			RenderingCategory.HeaderContent
			(
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(FMargin(5.0f, 0.0f))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("TinyText"))
						.Text(this, &FSpriteDetailsCustomization::GetRenderingHeaderContentText, MakeWeakObjectPtr(SpriteBeingEdited))
						.ToolTipText(TypesOfMaterialsTooltip)
					]
				]
			);

		}
	}

	// Add the rendering geometry mode into the parent container (renamed)
	const FString RenderGeometryTypePropertyPath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPaperSprite, RenderGeometry), GET_MEMBER_NAME_STRING_CHECKED(FSpriteGeometryCollection, GeometryType));
	TSharedPtr<IPropertyHandle> RenderGeometryTypeProperty = DetailLayout.GetProperty(*RenderGeometryTypePropertyPath);
	RenderingCategory.AddProperty(RenderGeometryTypeProperty)
		.DisplayName(LOCTEXT("RenderGeometryType", "Render Geometry Type"))
		.Visibility(HideWhenInCollisionMode);

	// Show the alternate material, but only when the mode is Diced
	TAttribute<EVisibility> ShowWhenModeIsDiced = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSpriteDetailsCustomization::PolygonModeMatches, RenderGeometryTypeProperty, ESpritePolygonMode::Diced));
	TSharedPtr<IPropertyHandle> AlternateMaterialProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, AlternateMaterial));
	RenderingCategory.AddProperty(AlternateMaterialProperty)
		.Visibility(ShowWhenModeIsDiced);

	// Show the rendering geometry settings
	TSharedRef<IPropertyHandle> RenderGeometry = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, RenderGeometry));
	IDetailPropertyRow& RenderGeometryProperty = RenderingCategory.AddProperty(RenderGeometry)
		.Visibility(HideWhenInCollisionMode);

	// Add the render polygons into advanced (renamed)
	const FString RenderGeometryPolygonsPropertyPath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPaperSprite, RenderGeometry), GET_MEMBER_NAME_STRING_CHECKED(FSpriteGeometryCollection, Shapes));
	RenderingCategory.AddProperty(DetailLayout.GetProperty(*RenderGeometryPolygonsPropertyPath), EPropertyLocation::Advanced)
		.DisplayName(LOCTEXT("RenderShapes", "Render Shapes"))
		.Visibility(HideWhenInCollisionMode);
}

void FSpriteDetailsCustomization::BuildCollisionSection(IDetailCategoryBuilder& CollisionCategory, IDetailLayoutBuilder& DetailLayout)
{
	TSharedPtr<IPropertyHandle> SpriteCollisionDomainProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, SpriteCollisionDomain));

	CollisionCategory.HeaderContent
	(
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(5.0f, 0.0f))
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("TinyText"))
				.Text(this, &FSpriteDetailsCustomization::GetCollisionHeaderContentText, SpriteCollisionDomainProperty)
			]
		]
	);

	TAttribute<EVisibility> ParticipatesInPhysics = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP( this, &FSpriteDetailsCustomization::AnyPhysicsMode, SpriteCollisionDomainProperty));
	TAttribute<EVisibility> ParticipatesInPhysics3D = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSpriteDetailsCustomization::PhysicsModeMatches, SpriteCollisionDomainProperty, ESpriteCollisionMode::Use3DPhysics));
	TAttribute<EVisibility> HideWhenInRenderingMode = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSpriteDetailsCustomization::EditorModeIsNot, ESpriteEditorMode::EditRenderingGeomMode));
	TAttribute<EVisibility> ShowWhenInRenderingMode = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSpriteDetailsCustomization::EditorModeMatches, ESpriteEditorMode::EditRenderingGeomMode));

	static const FText EditCollisionInCollisionMode = LOCTEXT("CollisionPropertiesHiddenInRenderingMode", "Switch to 'Edit Collsion' mode\nto edit Collision settings");
	CollisionCategory.AddCustomRow(EditCollisionInCollisionMode)
		.Visibility(ShowWhenInRenderingMode)
		.WholeRowContent()
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailLayout.GetDetailFontItalic())
			.Justification(ETextJustify::Center)
			.Text(EditCollisionInCollisionMode)
		];

	CollisionCategory.AddProperty(SpriteCollisionDomainProperty).Visibility(HideWhenInRenderingMode);

	// Add the collision geometry mode into the parent container (renamed)
	{
		// Restrict the diced value
		TSharedPtr<FPropertyRestriction> PreventDicedRestriction = MakeShareable(new FPropertyRestriction(LOCTEXT("CollisionGeometryDoesNotSupportDiced", "Collision geometry can not be set to Diced")));
		const UEnum* const SpritePolygonModeEnum = StaticEnum<ESpritePolygonMode::Type>();
		PreventDicedRestriction->AddDisabledValue(SpritePolygonModeEnum->GetNameStringByValue((uint8)ESpritePolygonMode::Diced));

		// Find and add the property
		const FString CollisionGeometryTypePropertyPath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPaperSprite, CollisionGeometry), GET_MEMBER_NAME_STRING_CHECKED(FSpriteGeometryCollection, GeometryType));
		TSharedPtr<IPropertyHandle> CollisionGeometryTypeProperty = DetailLayout.GetProperty(*CollisionGeometryTypePropertyPath);

		CollisionGeometryTypeProperty->AddRestriction(PreventDicedRestriction.ToSharedRef());

		CollisionCategory.AddProperty(CollisionGeometryTypeProperty)
			.DisplayName(LOCTEXT("CollisionGeometryType", "Collision Geometry Type"))
			.Visibility(ParticipatesInPhysics);
	}

	// Show the collision thickness only in 3D mode
	CollisionCategory.AddProperty( DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, CollisionThickness)) )
		.Visibility(ParticipatesInPhysics3D);

	// Show the default body instance (and only it) from the body setup (if it exists)
	DetailLayout.HideProperty("BodySetup");
	IDetailPropertyRow& BodySetupDefaultInstance = CollisionCategory.AddProperty("BodySetup.DefaultInstance");
	
	TArray<TWeakObjectPtr<UObject>> SpritesBeingEdited;
	DetailLayout.GetObjectsBeingCustomized(/*out*/ SpritesBeingEdited);

	TArray<UObject*> BodySetupList;
	for (auto WeakSpritePtr : SpritesBeingEdited)
	{
		if (UPaperSprite* Sprite = Cast<UPaperSprite>(WeakSpritePtr.Get()))
		{
			if (UBodySetup* BodySetup = Sprite->BodySetup)
			{
				BodySetupList.Add(BodySetup);
			}
		}
	}
	
	if (BodySetupList.Num() > 0)
	{
		IDetailPropertyRow* DefaultInstanceRow = CollisionCategory.AddExternalObjectProperty(BodySetupList, GET_MEMBER_NAME_CHECKED(UBodySetup, DefaultInstance));
		if (DefaultInstanceRow != nullptr)
		{
			DefaultInstanceRow->Visibility(ParticipatesInPhysics);
		}
	}

	// Show the collision geometry when not None
	CollisionCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, CollisionGeometry)))
		.Visibility(ParticipatesInPhysics);

	// Add the collision polygons into advanced (renamed)
	const FString CollisionGeometryPolygonsPropertyPath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPaperSprite, CollisionGeometry), GET_MEMBER_NAME_STRING_CHECKED(FSpriteGeometryCollection, Shapes));
	CollisionCategory.AddProperty(DetailLayout.GetProperty(*CollisionGeometryPolygonsPropertyPath), EPropertyLocation::Advanced)
		.DisplayName(LOCTEXT("CollisionShapes", "Collision Shapes"))
		.Visibility(ParticipatesInPhysics);
}

void FSpriteDetailsCustomization::BuildTextureSection(IDetailCategoryBuilder& SpriteCategory, IDetailLayoutBuilder& DetailLayout)
{
	// Grab information about the material
	TSharedPtr<IPropertyHandle> DefaultMaterialProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, DefaultMaterial));

	FText SourceTextureOverrideLabel;
	if (DefaultMaterialProperty.IsValid())
	{
		UObject* DefaultMaterialAsObject;
		if (DefaultMaterialProperty->GetValue(/*out*/ DefaultMaterialAsObject) == FPropertyAccess::Success)
		{
			if (UMaterialInterface* DefaultMaterialInterface = Cast<UMaterialInterface>(DefaultMaterialAsObject))
			{
				if (UMaterial* DefaultMaterial = DefaultMaterialInterface->GetMaterial())
				{
					// Get a list of sprite samplers
					TArray<const UMaterialExpressionSpriteTextureSampler*> SpriteSamplerExpressions;
					DefaultMaterial->GetAllExpressionsOfType(/*inout*/ SpriteSamplerExpressions);

					// Turn that into a set of labels
					for (const UMaterialExpressionSpriteTextureSampler* Sampler : SpriteSamplerExpressions)
					{
						if (!Sampler->SlotDisplayName.IsEmpty())
						{
							if (Sampler->bSampleAdditionalTextures)
							{
								AdditionalTextureLabels.FindOrAdd(Sampler->AdditionalSlotIndex) = Sampler->SlotDisplayName;
							}
							else
							{
								SourceTextureOverrideLabel = Sampler->SlotDisplayName;
							}
						}
					}
				}
			}
		}
	}

	// Create the base texture widget
	TSharedPtr<IPropertyHandle> SourceTextureProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, SourceTexture));
	DetailLayout.HideProperty(SourceTextureProperty);
	SpriteCategory.AddCustomRow(SourceTextureProperty->GetPropertyDisplayName())
		.NameContent()
		[
			CreateTextureNameWidget(SourceTextureProperty, SourceTextureOverrideLabel)
		]
		.ValueContent()
		.MaxDesiredWidth(TOptional<float>())
		[
			SourceTextureProperty->CreatePropertyValueWidget()
		];

	// Create the additional textures widget
	TSharedPtr<IPropertyHandle> AdditionalSourceTexturesProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, AdditionalSourceTextures));
	TSharedRef<FDetailArrayBuilder> AdditionalSourceTexturesBuilder = MakeShareable(new FDetailArrayBuilder(AdditionalSourceTexturesProperty.ToSharedRef()));
	AdditionalSourceTexturesBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FSpriteDetailsCustomization::GenerateAdditionalTextureWidget));
	SpriteCategory.AddCustomBuilder(AdditionalSourceTexturesBuilder);
}

void FSpriteDetailsCustomization::GenerateAdditionalTextureWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	IDetailPropertyRow& TextureRow = ChildrenBuilder.AddProperty(PropertyHandle);

	FText ExtraText;
	if (FText* pExtraText = AdditionalTextureLabels.Find(ArrayIndex))
	{
		ExtraText = *pExtraText;
	}

	FNumberFormattingOptions NoCommas;
	NoCommas.UseGrouping = false;
	const FText SlotDesc = FText::Format(LOCTEXT("AdditionalTextureSlotIndex", "Slot #{0}"), FText::AsNumber(ArrayIndex, &NoCommas));

	TextureRow.DisplayName(SlotDesc);

	TextureRow.ShowPropertyButtons(false);

	TextureRow.CustomWidget(false)
		.NameContent()
		[
			CreateTextureNameWidget(PropertyHandle, ExtraText)
		]
		.ValueContent()
		.MaxDesiredWidth(TOptional<float>())
		[
			PropertyHandle->CreatePropertyValueWidget()
		];
}

TSharedRef<SWidget> FSpriteDetailsCustomization::CreateTextureNameWidget(TSharedPtr<IPropertyHandle> PropertyHandle, const FText& OverrideText)
{
	TSharedRef<SWidget> PropertyNameWidget = PropertyHandle->CreatePropertyNameWidget();
	if (OverrideText.IsEmpty())
	{
		return PropertyNameWidget;
	}
	else
	{
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				PropertyNameWidget
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(OverrideText)
			];
	}
}

EVisibility FSpriteDetailsCustomization::EditorModeMatches(ESpriteEditorMode::Type DesiredMode) const
{
	return (SpriteEditMode.Get() == DesiredMode) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FSpriteDetailsCustomization::EditorModeIsNot(ESpriteEditorMode::Type DesiredMode) const
{
	return (SpriteEditMode.Get() != DesiredMode) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FSpriteDetailsCustomization::PhysicsModeMatches(TSharedPtr<IPropertyHandle> Property, ESpriteCollisionMode::Type DesiredMode) const
{
	if (SpriteEditMode.Get() == ESpriteEditorMode::EditRenderingGeomMode)
	{
		return EVisibility::Collapsed;
	}

	if (Property.IsValid())
	{
		uint8 ValueAsByte;
		FPropertyAccess::Result Result = Property->GetValue(/*out*/ ValueAsByte);

		if (Result == FPropertyAccess::Success)
		{
			return (((ESpriteCollisionMode::Type)ValueAsByte) == DesiredMode) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	// If there are multiple values, show all properties
	return EVisibility::Visible;
}

EVisibility FSpriteDetailsCustomization::AnyPhysicsMode(TSharedPtr<IPropertyHandle> Property) const
{
	if (SpriteEditMode.Get() == ESpriteEditorMode::EditRenderingGeomMode)
	{
		return EVisibility::Collapsed;
	}

	if (Property.IsValid())
	{
		uint8 ValueAsByte;
		FPropertyAccess::Result Result = Property->GetValue(/*out*/ ValueAsByte);

		if (Result == FPropertyAccess::Success)
		{
			return (((ESpriteCollisionMode::Type)ValueAsByte) != ESpriteCollisionMode::None) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	// If there are multiple values, show all properties
	return EVisibility::Visible;
}

FText FSpriteDetailsCustomization::GetCollisionHeaderContentText(TSharedPtr<IPropertyHandle> Property) const
{
	FText CollisionContentText;

	if (Property.IsValid())
	{
		uint8 ValueAsByte;
		if (Property->GetValue(/*out*/ ValueAsByte) == FPropertyAccess::Success)
		{
			ESpriteCollisionMode::Type CollisionMode = ((ESpriteCollisionMode::Type)ValueAsByte);

			switch (CollisionMode)
			{
			case ESpriteCollisionMode::None:
				CollisionContentText = LOCTEXT("CollisionHeader_NoCollision", "(no collision)");
				break;
			case ESpriteCollisionMode::Use3DPhysics:
				CollisionContentText = LOCTEXT("CollisionHeader_Use3D", "Uses 3D Physics");
				break;
			}
		}
	}

	return CollisionContentText;
}


FText FSpriteDetailsCustomization::GetRenderingHeaderContentText(TWeakObjectPtr<UPaperSprite> WeakSprite) const
{
	FText HeaderDisplayText;

	if (UPaperSprite* SpriteBeingEdited = WeakSprite.Get())
	{
		int32 NumOpaqueTriangles = 0;
		int32 NumMaskedTriangles = 0;
		int32 NumTranslucentTriangles = 0;
		FSpriteEditorViewportClient::AnalyzeSpriteMaterialType(SpriteBeingEdited, /*out*/ NumOpaqueTriangles, /*out*/ NumMaskedTriangles, /*out*/ NumTranslucentTriangles);

		FText MaterialType;
		if (UMaterialInterface* Material = SpriteBeingEdited->GetDefaultMaterial())
		{
			FMaterialShadingModelField ShadingModels = Material->GetShadingModels();
			if (ShadingModels.IsUnlit())
			{
				MaterialType = LOCTEXT("Unlit", "Unlit");
			}
			else if (ShadingModels.HasOnlyShadingModel(MSM_DefaultLit))
			{
				MaterialType = LOCTEXT("Lit", "Lit");
			}
			else
			{
				MaterialType = LOCTEXT("Exotic", "Exotic");
			}
		}

		static const FText Opaque = LOCTEXT("OpaqueMaterial", "Opaque");
		static const FText Translucent = LOCTEXT("TranslucentMaterial", "Translucent");
		static const FText Masked = LOCTEXT("MaskedMaterial", "Masked");
		static const FText OneMaterial = LOCTEXT("SpriteWithOneMaterialRenderHeaderText", "{0} - {1}");
		static const FText TwoMaterials = LOCTEXT("SpriteWithTwoMaterialsRenderHeaderText", "{0} - {1} and {2}");
		static const FText ThreeMaterials = LOCTEXT("SpriteWithThreeMaterialsRenderHeaderText", "{0} - {1}, {2}, {3}"); // Should never happen right now!

		TArray<const FText*, TInlineAllocator<3>> MaterialTypes;
		if (NumOpaqueTriangles > 0)
		{
			MaterialTypes.Add(&Opaque);
		}

		if (NumMaskedTriangles > 0)
		{
			MaterialTypes.Add(&Masked);
		}

		if (NumTranslucentTriangles > 0)
		{
			MaterialTypes.Add(&Translucent);
		}

		switch (MaterialTypes.Num())
		{
		case 0:
			break;
		case 1:
			HeaderDisplayText = FText::Format(OneMaterial, MaterialType, *MaterialTypes[0]);
			break;
		case 2:
			HeaderDisplayText = FText::Format(TwoMaterials, MaterialType, *MaterialTypes[0], *MaterialTypes[1]);
			break;
		case 3:
			HeaderDisplayText = FText::Format(ThreeMaterials, MaterialType, *MaterialTypes[0], *MaterialTypes[1], *MaterialTypes[2]);
			break;
		};
	}

	return HeaderDisplayText;
}

EVisibility FSpriteDetailsCustomization::GetAtlasGroupVisibility()
{
	return GetDefault<UPaperRuntimeSettings>()->bEnableSpriteAtlasGroups ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FSpriteDetailsCustomization::GetCustomPivotVisibility(TSharedPtr<IPropertyHandle> Property) const
{
	if (Property.IsValid())
	{
		uint8 ValueAsByte;
		FPropertyAccess::Result Result = Property->GetValue(/*out*/ ValueAsByte);

		if (Result == FPropertyAccess::Success)
		{
			return (((ESpritePivotMode::Type)ValueAsByte) == ESpritePivotMode::Custom) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	// If there are multiple values, show all properties
	return EVisibility::Visible;
}

EVisibility FSpriteDetailsCustomization::PolygonModeMatches(TSharedPtr<IPropertyHandle> Property, ESpritePolygonMode::Type DesiredMode) const
{
	if (DesiredMode == ESpritePolygonMode::Diced)
	{
		if (SpriteEditMode.Get() == ESpriteEditorMode::EditCollisionMode)
		{
			return EVisibility::Collapsed;
		}
	}

	if (Property.IsValid())
	{
		uint8 ValueAsByte;
		FPropertyAccess::Result Result = Property->GetValue(/*out*/ ValueAsByte);

		if (Result == FPropertyAccess::Success)
		{
			return (((ESpritePolygonMode::Type)ValueAsByte) == DesiredMode) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	// If there are multiple values, show all properties
	return EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE
