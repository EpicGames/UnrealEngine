// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "SpriteEditor/SpriteEditor.h"

namespace ESpriteCollisionMode { enum Type : int; }
namespace ESpritePolygonMode { enum Type : int; }

class FDetailWidgetRow;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;

//////////////////////////////////////////////////////////////////////////
// FSpriteDetailsCustomization

class FSpriteDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstanceForSpriteEditor(TAttribute<ESpriteEditorMode::Type> InEditMode);

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

protected:
	FSpriteDetailsCustomization(TAttribute<ESpriteEditorMode::Type> InEditMode);

	EVisibility EditorModeMatches(ESpriteEditorMode::Type DesiredMode) const;
	EVisibility EditorModeIsNot(ESpriteEditorMode::Type DesiredMode) const;
	EVisibility PhysicsModeMatches(TSharedPtr<IPropertyHandle> Property, ESpriteCollisionMode::Type DesiredMode) const;
	EVisibility AnyPhysicsMode(TSharedPtr<IPropertyHandle> Property) const;
	EVisibility GetCustomPivotVisibility(TSharedPtr<IPropertyHandle> Property) const;

	EVisibility PolygonModeMatches(TSharedPtr<IPropertyHandle> Property, ESpritePolygonMode::Type DesiredMode) const;

	FText GetCollisionHeaderContentText(TSharedPtr<IPropertyHandle> Property) const;
	FText GetRenderingHeaderContentText(TWeakObjectPtr<class UPaperSprite> WeakSprite) const;

	static EVisibility GetAtlasGroupVisibility();

	static FDetailWidgetRow& GenerateWarningRow(IDetailCategoryBuilder& WarningCategory, bool bExperimental, const FText& WarningText, const FText& Tooltip, const FString& ExcerptLink, const FString& ExcerptName);

	void BuildSpriteSection(IDetailCategoryBuilder& SpriteCategory, IDetailLayoutBuilder& DetailLayout);
	void BuildCollisionSection(IDetailCategoryBuilder& CollisionCategory, IDetailLayoutBuilder& DetailLayout);
	void BuildRenderingSection(IDetailCategoryBuilder& RenderingCategory, IDetailLayoutBuilder& DetailLayout);

	void BuildTextureSection(IDetailCategoryBuilder& SpriteCategory, IDetailLayoutBuilder& DetailLayout);
	void GenerateAdditionalTextureWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder);

	// Creates a name widget with optional captioning label
	TSharedRef<SWidget> CreateTextureNameWidget(TSharedPtr<IPropertyHandle> PropertyHandle, const FText& OverrideText);

protected:
	TAttribute<ESpriteEditorMode::Type> SpriteEditMode;

	// These are labels harvested from the material
	TMap<int32, FText> AdditionalTextureLabels;
};
