// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SpriteEditor.h"

#include "EditorModeManager.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailsView.h"
#include "SSingleObjectDetailsPanel.h"


#include "SpriteEditor/SpriteEditorViewportClient.h"
#include "SpriteEditor/SpriteEditorCommands.h"
#include "PaperEditorShared/SpriteGeometryEditCommands.h"
#include "Paper2DEditorModule.h"
#include "SpriteEditor/SSpriteEditorViewportToolbar.h"
#include "SpriteEditor/SpriteDetailsCustomization.h"

#include "SpriteEditor/SSpriteList.h"
#include "Widgets/Docking/SDockTab.h"

#include "ExtractSprites/SPaperExtractSpritesDialog.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////

const FName SpriteEditorAppName = FName(TEXT("SpriteEditorApp"));

//////////////////////////////////////////////////////////////////////////

struct FSpriteEditorTabs
{
	// Tab identifiers
	static const FName DetailsID;
	static const FName ViewportID;
	static const FName SpriteListID;
};

//////////////////////////////////////////////////////////////////////////

const FName FSpriteEditorTabs::DetailsID(TEXT("Details"));
const FName FSpriteEditorTabs::ViewportID(TEXT("Viewport"));
const FName FSpriteEditorTabs::SpriteListID(TEXT("SpriteList"));

//////////////////////////////////////////////////////////////////////////
// SSpriteEditorViewport

class SSpriteEditorViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SSpriteEditorViewport) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FSpriteEditor> InSpriteEditor);

	// SEditorViewport interface
	virtual void BindCommands() override;
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual EVisibility GetTransformToolbarVisibility() const override;
	virtual void OnFocusViewportToSelection() override;
	// End of SEditorViewport interface

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	// Invalidate any references to the sprite being edited; it has changed
	void NotifySpriteBeingEditedHasChanged()
	{
		EditorViewportClient->NotifySpriteBeingEditedHasChanged();
	}

	ESpriteEditorMode::Type GetCurrentMode() const
	{
		return EditorViewportClient->GetCurrentMode();
	}

	void ActivateEditMode()
	{
		EditorViewportClient->ActivateEditMode();
	}

	void ShowExtractSpritesDialog();

private:
	// Pointer back to owning sprite editor instance (the keeper of state)
	TWeakPtr<class FSpriteEditor> SpriteEditorPtr;

	// Viewport client
	TSharedPtr<FSpriteEditorViewportClient> EditorViewportClient;
};

void SSpriteEditorViewport::Construct(const FArguments& InArgs, TSharedPtr<FSpriteEditor> InSpriteEditor)
{
	SpriteEditorPtr = InSpriteEditor;

	SEditorViewport::Construct(SEditorViewport::FArguments());
}

void SSpriteEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FSpriteEditorCommands& Commands = FSpriteEditorCommands::Get();

	TSharedRef<FSpriteEditorViewportClient> EditorViewportClientRef = EditorViewportClient.ToSharedRef();

	// Show toggles
	CommandList->MapAction(
		Commands.SetShowGrid,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FEditorViewportClient::SetShowGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FEditorViewportClient::IsSetShowGridChecked));

	CommandList->MapAction(
		Commands.SetShowSourceTexture,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::ToggleShowSourceTexture),
		FCanExecuteAction::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::CanShowSourceTexture),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsShowSourceTextureChecked));

	CommandList->MapAction(
		Commands.ExtractSprites,
		FExecuteAction::CreateSP(this, &SSpriteEditorViewport::ShowExtractSpritesDialog),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsInSourceRegionEditMode));

	CommandList->MapAction(
		Commands.ToggleShowRelatedSprites,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::ToggleShowRelatedSprites),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsShowRelatedSpritesChecked),
		FIsActionButtonVisible::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsInSourceRegionEditMode));

	CommandList->MapAction(
		Commands.ToggleShowSpriteNames,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::ToggleShowSpriteNames),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsShowSpriteNamesChecked),
		FIsActionButtonVisible::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsInSourceRegionEditMode));

	CommandList->MapAction(
		Commands.SetShowBounds,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FEditorViewportClient::ToggleShowBounds ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FEditorViewportClient::IsSetShowBoundsChecked ) );

	CommandList->MapAction(
		Commands.SetShowCollision,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FEditorViewportClient::SetShowCollision ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FEditorViewportClient::IsSetShowCollisionChecked ) );

	CommandList->MapAction(
		Commands.SetShowMeshEdges,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::ToggleShowMeshEdges),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsShowMeshEdgesChecked));

 	CommandList->MapAction(
 		Commands.SetShowSockets,
 		FExecuteAction::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::ToggleShowSockets ),
 		FCanExecuteAction(),
 		FIsActionChecked::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::IsShowSocketsChecked ) );
 
	CommandList->MapAction(
		Commands.SetShowPivot,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::ToggleShowPivot ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::IsShowPivotChecked ) );

	// Editing modes
	CommandList->MapAction(
		Commands.EnterViewMode,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::EnterViewMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsInViewMode));
	CommandList->MapAction(
		Commands.EnterSourceRegionEditMode,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::EnterSourceRegionEditMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FSpriteEditorViewportClient::IsInSourceRegionEditMode));
	CommandList->MapAction(
		Commands.EnterCollisionEditMode,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::EnterCollisionEditMode ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::IsInCollisionEditMode ) );
	CommandList->MapAction(
		Commands.EnterRenderingEditMode,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::EnterRenderingEditMode ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FSpriteEditorViewportClient::IsInRenderingEditMode ) );
}

TSharedRef<FEditorViewportClient> SSpriteEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable(new FSpriteEditorViewportClient(SpriteEditorPtr, SharedThis(this)));

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SSpriteEditorViewport::BuildViewportToolbar()
{
	return SNew(SSpriteEditorViewportToolbar, SharedThis(this));
}

EVisibility SSpriteEditorViewport::GetTransformToolbarVisibility() const
{
	return EVisibility::Visible;
}

void SSpriteEditorViewport::OnFocusViewportToSelection()
{
	EditorViewportClient->RequestFocusOnSelection(/*bInstant=*/ false);
}

TSharedRef<class SEditorViewport> SSpriteEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SSpriteEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SSpriteEditorViewport::OnFloatingButtonClicked()
{
}

void SSpriteEditorViewport::ShowExtractSpritesDialog()
{
	if (UPaperSprite* Sprite = SpriteEditorPtr.Pin()->GetSpriteBeingEdited())
	{
		if (UTexture2D* SourceTexture = Sprite->GetSourceTexture())
		{
			SPaperExtractSpritesDialog::ShowWindow(SourceTexture);
		}
	}
}


/////////////////////////////////////////////////////
// SSpritePropertiesTabBody

class SSpritePropertiesTabBody : public SSingleObjectDetailsPanel
{
public:
	SLATE_BEGIN_ARGS(SSpritePropertiesTabBody) {}
	SLATE_END_ARGS()

private:
	// Pointer back to owning sprite editor instance (the keeper of state)
	TWeakPtr<class FSpriteEditor> SpriteEditorPtr;
public:
	void Construct(const FArguments& InArgs, TSharedPtr<FSpriteEditor> InSpriteEditor)
	{
		SpriteEditorPtr = InSpriteEditor;

		SSingleObjectDetailsPanel::Construct(SSingleObjectDetailsPanel::FArguments().HostCommandList(InSpriteEditor->GetToolkitCommands()).HostTabManager(InSpriteEditor->GetTabManager()), /*bAutomaticallyObserveViaGetObjectToObserve=*/ true, /*bAllowSearch=*/ true);

		TAttribute<ESpriteEditorMode::Type> SpriteEditorMode = TAttribute<ESpriteEditorMode::Type>::Create(
			TAttribute<ESpriteEditorMode::Type>::FGetter::CreateSP(InSpriteEditor.ToSharedRef(), &FSpriteEditor::GetCurrentMode));
		FOnGetDetailCustomizationInstance CustomizeSpritesForEditor = FOnGetDetailCustomizationInstance::CreateStatic(&FSpriteDetailsCustomization::MakeInstanceForSpriteEditor, SpriteEditorMode);
		PropertyView->RegisterInstancedCustomPropertyLayout(UPaperSprite::StaticClass(), CustomizeSpritesForEditor);
	}

	// SSingleObjectDetailsPanel interface
	virtual UObject* GetObjectToObserve() const override
	{
		return SpriteEditorPtr.Pin()->GetSpriteBeingEdited();
	}

	virtual TSharedRef<SWidget> PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget) override
	{
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				PropertyEditorWidget
			];
	}
	// End of SSingleObjectDetailsPanel interface
};

//////////////////////////////////////////////////////////////////////////
// FSpriteEditor

FSpriteEditor::FSpriteEditor()
	: SpriteBeingEdited(nullptr)
{
}

TSharedRef<SDockTab> FSpriteEditor::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("ViewportTab_Title", "Viewport"))
		[
			SNew(SOverlay)

			// The sprite editor viewport
			+SOverlay::Slot()
			[
				ViewportPtr.ToSharedRef()
			]

			// Bottom-right corner text indicating the preview nature of the sprite editor
			+SOverlay::Slot()
			.Padding(10)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Visibility(EVisibility::HitTestInvisible)
				.TextStyle(FAppStyle::Get(), "Graph.CornerText")
				.Text(this, &FSpriteEditor::GetCurrentModeCornerText)
			]
		];
}

TSharedRef<SDockTab> FSpriteEditor::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<FSpriteEditor> SpriteEditorPtr = SharedThis(this);

	// Spawn the tab
	return SNew(SDockTab)
		.Label(LOCTEXT("DetailsTab_Title", "Details"))
		[
			SNew(SSpritePropertiesTabBody, SpriteEditorPtr)
		];
}

TSharedRef<SDockTab> FSpriteEditor::SpawnTab_SpriteList(const FSpawnTabArgs& Args)
{
	// Spawn the tab
	return SNew(SDockTab)
		.Label(LOCTEXT("SpriteListTab_Title", "Sprite List"))
		[
			SpriteListPtr.ToSharedRef()
		];
}

void FSpriteEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_SpriteEditor", "Sprite Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(FSpriteEditorTabs::ViewportID, FOnSpawnTab::CreateSP(this, &FSpriteEditor::SpawnTab_Viewport))
		.SetDisplayName( LOCTEXT("ViewportTab", "Viewport") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(FSpriteEditorTabs::DetailsID, FOnSpawnTab::CreateSP(this, &FSpriteEditor::SpawnTab_Details))
		.SetDisplayName( LOCTEXT("DetailsTabLabel", "Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FSpriteEditorTabs::SpriteListID, FOnSpawnTab::CreateSP(this, &FSpriteEditor::SpawnTab_SpriteList))
		.SetDisplayName( LOCTEXT("SpriteListTabLabel", "Sprite List") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.ContentBrowser"));
}

void FSpriteEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FSpriteEditorTabs::ViewportID);
	InTabManager->UnregisterTabSpawner(FSpriteEditorTabs::DetailsID);
	InTabManager->UnregisterTabSpawner(FSpriteEditorTabs::SpriteListID);
}

void FSpriteEditor::InitSpriteEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UPaperSprite* InitSprite)
{
	SpriteBeingEdited = InitSprite;

	FSpriteEditorCommands::Register();
	FSpriteGeometryEditCommands::Register();

	BindCommands();

	TSharedPtr<FSpriteEditor> SpriteEditorPtr = SharedThis(this);
	ViewportPtr = SNew(SSpriteEditorViewport, SpriteEditorPtr);
	SpriteListPtr = SNew(SSpriteList, SpriteEditorPtr);

	// Default layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_SpriteEditor_Layout_v7")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell(true)
					->AddTab(FSpriteEditorTabs::ViewportID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.75f)
						->SetHideTabWell(true)
						->AddTab(FSpriteEditorTabs::DetailsID, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->AddTab(FSpriteEditorTabs::SpriteListID, ETabState::OpenedTab)
					)
				)
			)
		);

	// Initialize the asset editor
	InitAssetEditor(Mode, InitToolkitHost, SpriteEditorAppName, StandaloneDefaultLayout, /*bCreateDefaultStandaloneMenu=*/ true, /*bCreateDefaultToolbar=*/ true, InitSprite);

	ViewportPtr->ActivateEditMode();

	// Extend things
	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

void FSpriteEditor::BindCommands()
{
}

FName FSpriteEditor::GetToolkitFName() const
{
	return FName("SpriteEditor");
}

FText FSpriteEditor::GetBaseToolkitName() const
{
	return LOCTEXT("SpriteEditorAppLabel", "Sprite Editor");
}

FText FSpriteEditor::GetToolkitName() const
{
	return FText::FromString(SpriteBeingEdited->GetName());
}

FText FSpriteEditor::GetToolkitToolTipText() const
{
	return FAssetEditorToolkit::GetToolTipTextForObject(SpriteBeingEdited);
}

FString FSpriteEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("SpriteEditor");
}

FString FSpriteEditor::GetDocumentationLink() const
{
	return TEXT("AnimatingObjects/Paper2D/Sprites");
}

void FSpriteEditor::OnToolkitHostingStarted(const TSharedRef<class IToolkit>& Toolkit)
{
	//@TODO: MODETOOLS: Need to be able to register the widget in the toolbox panel with ToolkitHost, so it can instance the ed mode widgets into it
	// 	TSharedPtr<SWidget> InlineContent = Toolkit->GetInlineContent();
// 	if (InlineContent.IsValid())
// 	{
// 		ToolboxPtr->SetContent(InlineContent.ToSharedRef());
// 	}
}

void FSpriteEditor::OnToolkitHostingFinished(const TSharedRef<class IToolkit>& Toolkit)
{
	//ToolboxPtr->SetContent(SNullWidget::NullWidget);
	//@TODO: MODETOOLS: How to handle multiple ed modes at once in a standalone asset editor?
}

FLinearColor FSpriteEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

void FSpriteEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SpriteBeingEdited);
}

UTexture2D* FSpriteEditor::GetSourceTexture() const
{
	return SpriteBeingEdited->GetSourceTexture();
}

void FSpriteEditor::ExtendMenu()
{
}

void FSpriteEditor::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			const FSpriteEditorCommands& SpriteCommands = FSpriteEditorCommands::Get();
			const FSpriteGeometryEditCommands& GeometryCommands = FSpriteGeometryEditCommands::Get();

			ToolbarBuilder.BeginSection("Command");
			{
				ToolbarBuilder.AddToolBarButton(SpriteCommands.SetShowSourceTexture);
				ToolbarBuilder.AddToolBarButton(SpriteCommands.ToggleShowRelatedSprites);
				ToolbarBuilder.AddToolBarButton(SpriteCommands.ToggleShowSpriteNames);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("Tools");
			{
				ToolbarBuilder.AddToolBarButton(SpriteCommands.ExtractSprites);
				ToolbarBuilder.AddToolBarButton(GeometryCommands.AddBoxShape);
				ToolbarBuilder.AddToolBarButton(GeometryCommands.ToggleAddPolygonMode);
				ToolbarBuilder.AddToolBarButton(GeometryCommands.AddCircleShape);
				ToolbarBuilder.AddToolBarButton(GeometryCommands.SnapAllVertices);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		ViewportPtr->GetCommandList(),
		FToolBarExtensionDelegate::CreateStatic( &Local::FillToolbar )
		);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		ViewportPtr->GetCommandList(),
		FToolBarExtensionDelegate::CreateSP(this, &FSpriteEditor::CreateModeToolbarWidgets));

	AddToolbarExtender(ToolbarExtender);

 	IPaper2DEditorModule* Paper2DEditorModule = &FModuleManager::LoadModuleChecked<IPaper2DEditorModule>("Paper2DEditor");
	AddToolbarExtender(Paper2DEditorModule->GetSpriteEditorToolBarExtensibilityManager()->GetAllExtenders());
}

void FSpriteEditor::CreateEditorModeManager()
{
	check(ViewportPtr.IsValid());
	TSharedPtr<FEditorViewportClient> ViewportClient = ViewportPtr->GetViewportClient();
	check(ViewportClient.IsValid());
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewportClient->TakeOwnershipOfModeManager(EditorModeManager);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (FEditorModeTools* const ModeTools = ViewportClient->GetModeTools())
	{
		// Opt out of new ITF gizmos and viewport inputs for now
		ModeTools->SetSupportsViewportITF(false);
	}
}

void FSpriteEditor::SetSpriteBeingEdited(UPaperSprite* NewSprite)
{
	if ((NewSprite != SpriteBeingEdited) && (NewSprite != nullptr))
	{
		UPaperSprite* OldSprite = SpriteBeingEdited;
		SpriteBeingEdited = NewSprite;
		
		// Let the viewport know that we are editing something different
		ViewportPtr->NotifySpriteBeingEditedHasChanged();

		// Let the editor know that are editing something different
		RemoveEditingObject(OldSprite);
		AddEditingObject(NewSprite);

		// Update the asset picker to select the new active sprite
		SpriteListPtr->SelectAsset(NewSprite);
	}
}

ESpriteEditorMode::Type FSpriteEditor::GetCurrentMode() const
{
	return ViewportPtr->GetCurrentMode();
}

void FSpriteEditor::CreateModeToolbarWidgets(FToolBarBuilder& IgnoredBuilder)
{
	FSlimHorizontalToolBarBuilder ToolbarBuilder(ViewportPtr->GetCommandList(), FMultiBoxCustomization::None);
	ToolbarBuilder.AddToolBarButton(FSpriteEditorCommands::Get().EnterViewMode);
	ToolbarBuilder.AddToolBarButton(FSpriteEditorCommands::Get().EnterSourceRegionEditMode);
	ToolbarBuilder.AddToolBarButton(FSpriteEditorCommands::Get().EnterCollisionEditMode);
	ToolbarBuilder.AddToolBarButton(FSpriteEditorCommands::Get().EnterRenderingEditMode);
	AddToolbarWidget(ToolbarBuilder.MakeWidget());
}

FText FSpriteEditor::GetCurrentModeCornerText() const
{
	switch (GetCurrentMode())
	{
	case ESpriteEditorMode::EditCollisionMode:
		return LOCTEXT("EditCollisionGeometry_CornerText", "Edit Collision");
	case ESpriteEditorMode::EditRenderingGeomMode:
		return LOCTEXT("EditRenderGeometry_CornerText", "Edit Render Geometry");
	case ESpriteEditorMode::EditSourceRegionMode:
		return LOCTEXT("EditSourceRegion_CornerText", "Edit Source Region");
	default:
		return FText::GetEmpty();
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
