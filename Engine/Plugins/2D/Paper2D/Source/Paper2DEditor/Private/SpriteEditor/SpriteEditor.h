// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class FSpawnTabArgs;

class FToolBarBuilder;
class SSpriteEditorViewport;
class SSpriteList;
class UPaperSprite;
class UTexture2D;

//////////////////////////////////////////////////////////////////////////
// 

namespace ESpriteEditorMode
{
	enum Type
	{
		ViewMode,
		EditSourceRegionMode,
		EditCollisionMode,
		EditRenderingGeomMode
	};
}

//////////////////////////////////////////////////////////////////////////
// FSpriteEditor

class FSpriteEditor : public FAssetEditorToolkit, public FGCObject
{
public:
	FSpriteEditor();

	// IToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	// End of IToolkit interface

	// FAssetEditorToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FString GetDocumentationLink() const override;
	virtual void OnToolkitHostingStarted(const TSharedRef<class IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<class IToolkit>& Toolkit) override;
	// End of FAssetEditorToolkit

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSpriteEditor");
	}
	// End of FSerializableObject interface

	// Get the source texture for the current sprite being edited
	UTexture2D* GetSourceTexture() const;
public:
	void InitSpriteEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UPaperSprite* InitSprite);

	UPaperSprite* GetSpriteBeingEdited() const { return SpriteBeingEdited; }
	void SetSpriteBeingEdited(UPaperSprite* NewSprite);

	ESpriteEditorMode::Type GetCurrentMode() const;

protected:
	TObjectPtr<UPaperSprite> SpriteBeingEdited;
	TSharedPtr<SSpriteEditorViewport> ViewportPtr;
	TSharedPtr<SSpriteList> SpriteListPtr;

protected:
	void BindCommands();
	void ExtendMenu();
	void ExtendToolbar();
	virtual void CreateEditorModeManager() override;

	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SpriteList(const FSpawnTabArgs& Args);

	void CreateModeToolbarWidgets(FToolBarBuilder& ToolbarBuilder);

	FText GetCurrentModeCornerText() const;
};
