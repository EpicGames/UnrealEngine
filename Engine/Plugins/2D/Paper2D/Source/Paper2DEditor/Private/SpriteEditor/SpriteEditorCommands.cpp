// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SpriteEditorCommands.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////
// FSpriteEditorCommands

void FSpriteEditorCommands::RegisterCommands()
{
	// Show toggles
	UI_COMMAND(SetShowGrid, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowSourceTexture, "Src Tex", "Toggles display of the source texture (useful when it is an atlas).", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowBounds, "Bounds", "Toggles display of the bounds of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowCollision, "Collision", "Toggles display of the simplified collision mesh of the static mesh, if one has been assigned.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::C, EModifierKey::Alt));

	UI_COMMAND(SetShowSockets, "Sockets", "Displays the sprite sockets.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(SetShowPivot, "Pivot", "Display the pivot location of the sprite.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowMeshEdges, "Mesh Edges", "Overlays the mesh edges on top of the view.", EUserInterfaceActionType::ToggleButton, FInputChord());

	// Source region edit mode
	UI_COMMAND(ExtractSprites, "Extract Sprites", "Triggers the Extract Sprites dialog on the source texture.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleShowRelatedSprites, "Other Sprites", "Toggles display of the source region bounds for other sprites that share the same source texture.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowSpriteNames, "Sprite Names", "Toggles display of the name of each sprite that shares the same source texture.", EUserInterfaceActionType::ToggleButton, FInputChord());

	// Editing modes
	UI_COMMAND(EnterViewMode, "View", "View the sprite.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterSourceRegionEditMode, "Edit Source Region", "Edit the sprite source region.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterCollisionEditMode, "Edit Collision", "Edit the collision geometry.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnterRenderingEditMode, "Edit RenderGeom", "Edit the rendering geometry (useful to reduce overdraw).", EUserInterfaceActionType::ToggleButton, FInputChord());
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
