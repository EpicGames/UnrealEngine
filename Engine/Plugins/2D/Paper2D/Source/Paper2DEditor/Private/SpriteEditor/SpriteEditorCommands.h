// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "PaperStyle.h"
#include "Styling/ISlateStyle.h"

class FSpriteEditorCommands : public TCommands<FSpriteEditorCommands>
{
public:
	FSpriteEditorCommands()
		: TCommands<FSpriteEditorCommands>(
			TEXT("SpriteEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "SpriteEditor", "Sprite Editor"), // Localized context name for displaying
			NAME_None, // Parent
			FPaperStyle::Get()->GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:
	// Show toggles
	TSharedPtr<FUICommandInfo> SetShowGrid;
	TSharedPtr<FUICommandInfo> SetShowSourceTexture;
	TSharedPtr<FUICommandInfo> SetShowBounds;
	TSharedPtr<FUICommandInfo> SetShowCollision;
	
	TSharedPtr<FUICommandInfo> SetShowSockets;

	TSharedPtr<FUICommandInfo> SetShowPivot;
	TSharedPtr<FUICommandInfo> SetShowMeshEdges;

	// Source region edit mode
	TSharedPtr<FUICommandInfo> ExtractSprites;
	TSharedPtr<FUICommandInfo> ToggleShowRelatedSprites;
	TSharedPtr<FUICommandInfo> ToggleShowSpriteNames;

	// Editing modes
	TSharedPtr<FUICommandInfo> EnterViewMode;
	TSharedPtr<FUICommandInfo> EnterSourceRegionEditMode;
	TSharedPtr<FUICommandInfo> EnterCollisionEditMode;
	TSharedPtr<FUICommandInfo> EnterRenderingEditMode;
};
