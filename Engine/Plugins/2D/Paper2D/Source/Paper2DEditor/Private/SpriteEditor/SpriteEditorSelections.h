// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Paper2DModule.h"
#include "PaperEditorShared/AssetEditorSelectedItem.h"
#include "PaperSprite.h"

//////////////////////////////////////////////////////////////////////////
// FSelectionTypes

class FSelectionTypes
{
public:
	static const FName GeometryShape;
	static const FName Vertex;
	static const FName Edge;
	static const FName Pivot;
	static const FName SourceRegion;
private:
	FSelectionTypes() {}
};

//////////////////////////////////////////////////////////////////////////
// ISpriteSelectionContext

class ISpriteSelectionContext
{
public:
	virtual FVector2D SelectedItemConvertWorldSpaceDeltaToLocalSpace(const FVector& WorldSpaceDelta) const = 0;
	virtual FVector2D WorldSpaceToTextureSpace(const FVector& SourcePoint) const = 0;
	virtual FVector TextureSpaceToWorldSpace(const FVector2D& SourcePoint) const = 0;
	virtual float SelectedItemGetUnitsPerPixel() const = 0;
	virtual void BeginTransaction(const FText& SessionName) = 0;
	virtual void MarkTransactionAsDirty() = 0;
	virtual void EndTransaction() = 0;
	virtual void InvalidateViewportAndHitProxies() = 0;
};

//////////////////////////////////////////////////////////////////////////
// FSpriteSelectedSourceRegion

class FSpriteSelectedSourceRegion : public FSelectedItem
{
public:
	int32 VertexIndex;
	TWeakObjectPtr<UPaperSprite> SpritePtr;

public:
	FSpriteSelectedSourceRegion()
		: FSelectedItem(FSelectionTypes::SourceRegion)
		, VertexIndex(0)
	{
	}

	virtual uint32 GetTypeHash() const override
	{
		return VertexIndex;
	}

	virtual bool Equals(const FSelectedItem& OtherItem) const override
	{
		if (OtherItem.IsA(FSelectionTypes::SourceRegion))
		{
			const FSpriteSelectedSourceRegion& V1 = *this;
			const FSpriteSelectedSourceRegion& V2 = *(FSpriteSelectedSourceRegion*)(&OtherItem);

			return (V1.VertexIndex == V2.VertexIndex) && (V1.SpritePtr == V2.SpritePtr);
		}
		else
		{
			return false;
		}
	}

	virtual void ApplyDeltaIndexed(const FVector2D& WorldSpaceDelta, int32 TargetVertexIndex)
	{
		if (UPaperSprite* Sprite = SpritePtr.Get())
		{
			UTexture2D* SourceTexture = Sprite->GetSourceTexture();
			const FVector2D SourceDims = (SourceTexture != nullptr) ? FVector2D(SourceTexture->GetSurfaceWidth(), SourceTexture->GetSurfaceHeight()) : FVector2D::ZeroVector;

			int32 XAxis = 0; // -1 = min, 0 = none, 1 = max
            int32 YAxis = 0; // ditto
            
			switch (VertexIndex)
			{
            case 0: // Top left
                XAxis = -1;
                YAxis = -1;
                break;
            case 1: // Top right
                XAxis = 1;
                YAxis = -1;
                break;
            case 2: // Bottom right
                XAxis = 1;
                YAxis = 1;
                break;
            case 3: // Bottom left
                XAxis = -1;
                YAxis = 1;
                break;
            case 4: // Top
                YAxis = -1;
                break;
            case 5: // Right
                XAxis = 1;
                break;
            case 6: // Bottom
                YAxis = 1;
                break;
            case 7: // Left
                XAxis = -1;
                break;
			default:
				break;
			}

			const FVector2D TextureSpaceDelta = Sprite->ConvertWorldSpaceDeltaToTextureSpace(PaperAxisX * WorldSpaceDelta.X + PaperAxisY * WorldSpaceDelta.Y, /*bIgnoreRotation=*/ true);

            if (XAxis == -1)
            {
                const float AllowedDelta = FMath::Clamp(TextureSpaceDelta.X, -Sprite->SourceUV.X, Sprite->SourceDimension.X - 1);
                Sprite->SourceUV.X += AllowedDelta;
                Sprite->SourceDimension.X -= AllowedDelta;
            }
            else if (XAxis == 1)
            {
				Sprite->SourceDimension.X = FMath::Clamp(Sprite->SourceDimension.X + TextureSpaceDelta.X, 1.0f, SourceDims.X - Sprite->SourceUV.X);
            }
            
            if (YAxis == -1)
            {
				const float AllowedDelta = FMath::Clamp(TextureSpaceDelta.Y, -Sprite->SourceUV.Y, Sprite->SourceDimension.Y - 1);
				Sprite->SourceUV.Y += AllowedDelta;
				Sprite->SourceDimension.Y -= AllowedDelta;
            }
            else if (YAxis == 1)
            {
				Sprite->SourceDimension.Y = FMath::Clamp(Sprite->SourceDimension.Y + TextureSpaceDelta.Y, 1.0f, SourceDims.Y - Sprite->SourceUV.Y);
            }
		}
	}

	FVector GetWorldPosIndexed(int32 TargetVertexIndex) const
	{
		FVector Result = FVector::ZeroVector;

		if (UPaperSprite* Sprite = SpritePtr.Get())
		{
			UTexture2D* SourceTexture = Sprite->GetSourceTexture();
			const FVector2D SourceDims = (SourceTexture != nullptr) ? FVector2D(SourceTexture->GetSurfaceWidth(), SourceTexture->GetSurfaceHeight()) : FVector2D::ZeroVector;
			FVector2D BoundsVertex = Sprite->SourceUV;
			switch (VertexIndex)
			{
            case 0:
                break;
            case 1:
                BoundsVertex.X += Sprite->SourceDimension.X;
                break;
            case 2:
                BoundsVertex.X += Sprite->SourceDimension.X;
                BoundsVertex.Y += Sprite->SourceDimension.Y;
                break;
            case 3:
                BoundsVertex.Y += Sprite->SourceDimension.Y;
                break;
			case 4:
				BoundsVertex.X += Sprite->SourceDimension.X * 0.5f;
				break;
			case 5:
				BoundsVertex.X += Sprite->SourceDimension.X;
				BoundsVertex.Y += Sprite->SourceDimension.Y * 0.5f;
				break;
			case 6:
				BoundsVertex.X += Sprite->SourceDimension.X * 0.5f;
				BoundsVertex.Y += Sprite->SourceDimension.Y;
				break;
			case 7:
				BoundsVertex.Y += Sprite->SourceDimension.Y * 0.5f;
				break;
			default:
				break;
			}

			const FVector PixelSpaceResult = BoundsVertex.X * PaperAxisX + (SourceDims.Y - BoundsVertex.Y) * PaperAxisY;
			Result = PixelSpaceResult * Sprite->GetUnrealUnitsPerPixel();
		}

		return Result;
	}

	virtual void ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode) override
	{
		if (MoveMode == UE::Widget::WM_Translate)
		{
			ApplyDeltaIndexed(Delta, VertexIndex);
		}
	}

	FVector GetWorldPos() const override
	{
		return GetWorldPosIndexed(VertexIndex);
	}
};


//////////////////////////////////////////////////////////////////////////
// FSpriteSelectedShape

class FSpriteSelectedShape : public FSelectedItem
{
public:
	// The editor context
	ISpriteSelectionContext* EditorContext;

	// The geometry that this shape belongs to
	FSpriteGeometryCollection& Geometry;

	// The index of this shape in the geometry above
	const int32 ShapeIndex;

	// Is this a background object that should have lower priority?
	const bool bIsBackground;

	TWeakObjectPtr<UPaperSprite> SpritePtr;
public:
	FSpriteSelectedShape(ISpriteSelectionContext* InEditorContext, FSpriteGeometryCollection& InGeometry, int32 InShapeIndex, bool bInIsBackground = false);

	// FSelectedItem interface
	virtual uint32 GetTypeHash() const override;
	virtual EMouseCursor::Type GetMouseCursor() const override;
	virtual bool Equals(const FSelectedItem& OtherItem) const override;
	virtual bool IsBackgroundObject() const override;
	virtual void ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode) override;
	virtual FVector GetWorldPos() const override;
	// End of FSelectedItem interface
};

//////////////////////////////////////////////////////////////////////////
// FSpriteSelectedVertex

class FSpriteSelectedVertex : public FSelectedItem
{
public:
	// The editor context
	ISpriteSelectionContext* EditorContext;

	// The geometry that this shape belongs to
	FSpriteGeometryCollection& Geometry;

	const int32 ShapeIndex;
	const int32 VertexIndex;

public:
	FSpriteSelectedVertex(ISpriteSelectionContext* InEditorContext, FSpriteGeometryCollection& InGeometry, int32 InShapeIndex, int32 InVertexIndex)
		: FSelectedItem(FSelectionTypes::Vertex)
		, EditorContext(InEditorContext)
		, Geometry(InGeometry)
		, ShapeIndex(InShapeIndex)
		, VertexIndex(InVertexIndex)
	{
	}

	virtual uint32 GetTypeHash() const override
	{
		return VertexIndex + (ShapeIndex * 311);
	}

	virtual bool Equals(const FSelectedItem& OtherItem) const override
	{
		if (OtherItem.IsA(FSelectionTypes::Vertex))
		{
			const FSpriteSelectedVertex& V1 = *this;
			const FSpriteSelectedVertex& V2 = *(FSpriteSelectedVertex*)(&OtherItem);

			return (V1.VertexIndex == V2.VertexIndex) && (V1.ShapeIndex == V2.ShapeIndex) && (&(V1.Geometry) == &(V2.Geometry));
		}
		else
		{
			return false;
		}
	}

	virtual void ApplyDeltaIndexed(const FVector2D& Delta, int32 TargetVertexIndex)
	{
		if (Geometry.Shapes.IsValidIndex(ShapeIndex))
		{
			FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];
			TArray<FVector2D>& Vertices = Shape.Vertices;
			TargetVertexIndex = (Vertices.Num() > 0) ? (TargetVertexIndex % Vertices.Num()) : 0;
			if (Vertices.IsValidIndex(TargetVertexIndex))
			{
				FVector2D& ShapeSpaceVertex = Vertices[TargetVertexIndex];

				const FVector WorldSpaceDelta = PaperAxisX * Delta.X + PaperAxisY * Delta.Y;
				const FVector2D TextureSpaceDelta = EditorContext->SelectedItemConvertWorldSpaceDeltaToLocalSpace(WorldSpaceDelta);
				const FVector2D NewTextureSpacePos = Shape.ConvertShapeSpaceToTextureSpace(ShapeSpaceVertex) + TextureSpaceDelta;
				ShapeSpaceVertex = Shape.ConvertTextureSpaceToShapeSpace(NewTextureSpacePos);

				Geometry.GeometryType = ESpritePolygonMode::FullyCustom;
				Shape.ShapeType = ESpriteShapeType::Polygon;
			}
		}
	}

	FVector GetWorldPosIndexed(int32 TargetVertexIndex) const
	{
		FVector Result = FVector::ZeroVector;

		if (Geometry.Shapes.IsValidIndex(ShapeIndex))
		{
			const FSpriteGeometryShape& Shape = Geometry.Shapes[ShapeIndex];
			const TArray<FVector2D>& Vertices = Shape.Vertices;
			TargetVertexIndex = (Vertices.Num() > 0) ? (TargetVertexIndex % Vertices.Num()) : 0;
			if (Vertices.IsValidIndex(TargetVertexIndex))
			{
				const FVector2D& ShapeSpacePos = Vertices[TargetVertexIndex];
				const FVector2D TextureSpacePos = Shape.ConvertShapeSpaceToTextureSpace(ShapeSpacePos);
				Result = EditorContext->TextureSpaceToWorldSpace(TextureSpacePos);
			}
		}

		return Result;
	}

	virtual void ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode) override
	{
		if (MoveMode == UE::Widget::WM_Translate)
		{
			ApplyDeltaIndexed(Delta, VertexIndex);
		}
	}

	FVector GetWorldPos() const override
	{
		return GetWorldPosIndexed(VertexIndex);
	}
};


//////////////////////////////////////////////////////////////////////////
// FSpriteSelectedEdge

// Note: Defined based on a vertex index; this is the edge between the vertex and the next one.
class FSpriteSelectedEdge : public FSpriteSelectedVertex
{
public:
	FSpriteSelectedEdge(ISpriteSelectionContext* InEditorContext, FSpriteGeometryCollection& InGeometry, int32 InShapeIndex, int32 InVertexIndex)
		: FSpriteSelectedVertex(InEditorContext, InGeometry, InShapeIndex, InVertexIndex)
	{
		TypeName = FSelectionTypes::Edge;
	}

	virtual bool IsA(FName TestType) const
	{
		return (TestType == TypeName) || (TestType == FSelectionTypes::Vertex);
	}

	virtual bool Equals(const FSelectedItem& OtherItem) const override
	{
		if (OtherItem.IsA(FSelectionTypes::Edge))
		{
			const FSpriteSelectedEdge& E1 = *this;
			const FSpriteSelectedEdge& E2 = *(FSpriteSelectedEdge*)(&OtherItem);

			return (E1.VertexIndex == E2.VertexIndex) && (E1.ShapeIndex == E2.ShapeIndex) && (&(E1.Geometry) == &(E2.Geometry));
		}
		else
		{
			return false;
		}
	}

	virtual void ApplyDelta(const FVector2D& Delta, const FRotator& Rotation, const FVector& Scale3D, UE::Widget::EWidgetMode MoveMode) override
	{
		ApplyDeltaIndexed(Delta, VertexIndex);
		ApplyDeltaIndexed(Delta, VertexIndex+1);
	}

	FVector GetWorldPos() const override
	{
		const FVector Pos1 = GetWorldPosIndexed(VertexIndex);
		const FVector Pos2 = GetWorldPosIndexed(VertexIndex+1);

		return (Pos1 + Pos2) * 0.5f;
	}
};
