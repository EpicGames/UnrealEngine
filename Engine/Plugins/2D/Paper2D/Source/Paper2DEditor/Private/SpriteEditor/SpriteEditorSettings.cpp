// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SpriteEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpriteEditorSettings)

//////////////////////////////////////////////////////////////////////////
// USpriteEditorSettings

USpriteEditorSettings::USpriteEditorSettings()
	: BackgroundColor(55, 55, 55)
	, bShowGridByDefault(false)
{
}

