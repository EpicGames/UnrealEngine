// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SpriteEditorViewportClient.h"
#include "InputKeyEventArgs.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "CanvasItem.h"
#include "Misc/PackageName.h"
#include "Utils.h"
#include "PaperSpriteComponent.h"
#include "AssetEditorModeManager.h"
#include "SEditorViewport.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "GlobalRenderResources.h"
#include "ComponentReregisterContext.h"

#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CanvasTypes.h"
#include "PaperEditorShared/SocketEditing.h"

#include "PaperEditorShared/SpriteGeometryEditMode.h"


#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "SpriteEditor/SpriteEditorSettings.h"
#include "PaperSpriteFactory.h"
#include "UnrealWidget.h"

#define LOCTEXT_NAMESPACE "SpriteEditor"

//////////////////////////////////////////////////////////////////////////
// FSelectionTypes

const FName FSelectionTypes::GeometryShape(TEXT("GeometryShape"));
const FName FSelectionTypes::Vertex(TEXT("Vertex"));
const FName FSelectionTypes::Edge(TEXT("Edge"));
const FName FSelectionTypes::Pivot(TEXT("Pivot"));
const FName FSelectionTypes::SourceRegion(TEXT("SourceRegion"));


//////////////////////////////////////////////////////////////////////////
// Sprite editing constants

namespace SpriteEditingConstants
{
	// Tint the source texture darker to help distinguish it from the sprite being edited
	const FLinearColor SourceTextureDarkTintColor(0.05f, 0.05f, 0.05f, 1.0f);

	// Note: MinMouseRadius must be greater than MinArrowLength
	const FLinearColor BakedCollisionLineRenderColor(1.0f, 1.0f, 0.0f, 0.25f);
	const FLinearColor BakedCollisionRenderColor(1.0f, 1.0f, 0.0f, 0.5f);
	const float BakedCollisionVertexSize = 3.0f;

	const FLinearColor SourceRegionBoundsColor(1.0f, 1.0f, 1.0f, 0.8f);
	const FLinearColor SourceRegionRelatedBoundsColor(0.3f, 0.3f, 0.3f, 0.8f);
	const FLinearColor SourceRegionRelatedSpriteNameColor(0.6f, 0.6f, 0.6f, 0.8f);

	const FLinearColor CollisionShapeColor(0.0f, 0.7f, 1.0f, 1.0f);
	const FLinearColor RenderShapeColor(1.0f, 0.2f, 0.0f, 1.0f);
	const FLinearColor SubtractiveRenderShapeColor(0.0f, 0.2f, 1.0f, 1.0f);
}

//////////////////////////////////////////////////////////////////////////
// FSpriteEditorViewportClient

FSpriteEditorViewportClient::FSpriteEditorViewportClient(TWeakPtr<FSpriteEditor> InSpriteEditor, TWeakPtr<class SEditorViewport> InSpriteEditorViewportPtr)
	: CurrentMode(ESpriteEditorMode::ViewMode)
	, SpriteEditorPtr(InSpriteEditor)
	, SpriteEditorViewportPtr(InSpriteEditorViewportPtr)
{
	check(SpriteEditorPtr.IsValid() && SpriteEditorViewportPtr.IsValid());

	// The tile map editor fully supports mode tools and isn't doing any incompatible stuff with the Widget
	Widget->SetUsesEditorModeTools(ModeTools.Get());

	PreviewScene = &OwnedPreviewScene;
	((FAssetEditorModeManager*)ModeTools.Get())->SetPreviewScene(PreviewScene);

	SetRealtime(true);

	bManipulating = false;
	bManipulationDirtiedSomething = false;
	ScopedTransaction = nullptr;

	bShowSourceTexture = false;
	bShowSockets = true;
	bShowPivot = true;
	bShowRelatedSprites = true;
	bShowNamesForSprites = true;

	DrawHelper.bDrawGrid = GetDefault<USpriteEditorSettings>()->bShowGridByDefault;

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetCompositeEditorPrimitives(true);

	// Create a render component for the sprite being edited
	{
		RenderSpriteComponent = NewObject<UPaperSpriteComponent>();
		UPaperSprite* Sprite = GetSpriteBeingEdited();
		RenderSpriteComponent->SetSprite(Sprite);

		PreviewScene->AddComponent(RenderSpriteComponent, FTransform::Identity);
	}

	// Create a sprite and render component for the source texture view
	{
		UPaperSprite* DummySprite = NewObject<UPaperSprite>();
		DummySprite->SpriteCollisionDomain = ESpriteCollisionMode::None;
		DummySprite->PivotMode = ESpritePivotMode::Bottom_Left;
		DummySprite->CollisionGeometry.GeometryType = ESpritePolygonMode::SourceBoundingBox;
		DummySprite->RenderGeometry.GeometryType = ESpritePolygonMode::SourceBoundingBox;

		SourceTextureViewComponent = NewObject<UPaperSpriteComponent>();
		SourceTextureViewComponent->SetSprite(DummySprite);
		UpdateSourceTextureSpriteFromSprite(GetSpriteBeingEdited());

		// Nudge the source texture view back a bit so it doesn't occlude sprites
		const FTransform Transform(-1.0f * PaperAxisZ);
		SourceTextureViewComponent->SetVisibleFlag(false);
		PreviewScene->AddComponent(SourceTextureViewComponent, Transform);
	}
}

void FSpriteEditorViewportClient::ActivateEditMode()
{
	// Activate the sprite geometry edit mode
	ModeTools->SetDefaultMode(FSpriteGeometryEditMode::EM_SpriteGeometry);
	ModeTools->ActivateDefaultMode();

	FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
	check(GeometryEditMode);
	GeometryEditMode->SetEditorContext(this);
	GeometryEditMode->BindCommands(SpriteEditorViewportPtr.Pin()->GetCommandList());
	ModeTools->SetWidgetMode(UE::Widget::WM_Translate);
}

void FSpriteEditorViewportClient::UpdateSourceTextureSpriteFromSprite(UPaperSprite* SourceSprite)
{
	UPaperSprite* TargetSprite = SourceTextureViewComponent->GetSprite();
	check(TargetSprite);

	if (SourceSprite != nullptr)
	{
		if ((SourceSprite->GetSourceTexture() != TargetSprite->GetSourceTexture()) || (TargetSprite->PixelsPerUnrealUnit != SourceSprite->PixelsPerUnrealUnit))
		{
			FComponentReregisterContext ReregisterSprite(SourceTextureViewComponent);

			FSpriteAssetInitParameters SpriteReinitParams;

			SpriteReinitParams.SetTextureAndFill(SourceSprite->GetSourceTexture());
			SpriteReinitParams.DefaultMaterialOverride = SourceSprite->DefaultMaterial;
			SpriteReinitParams.AlternateMaterialOverride = SourceSprite->AlternateMaterial;
			SpriteReinitParams.SetPixelsPerUnrealUnit(SourceSprite->PixelsPerUnrealUnit);
			TargetSprite->InitializeSprite(SpriteReinitParams);

			RequestFocusOnSelection(/*bInstant=*/ true);
		}


		// Position the sprite for the mode its meant to be in
		FVector2D CurrentPivotPosition;
		ESpritePivotMode::Type CurrentPivotMode = TargetSprite->GetPivotMode(/*out*/CurrentPivotPosition);

		FVector Translation(1.0f * PaperAxisZ);
		if (IsInSourceRegionEditMode())
		{
			if (CurrentPivotMode != ESpritePivotMode::Bottom_Left)
			{
				TargetSprite->SetPivotMode(ESpritePivotMode::Bottom_Left, FVector2D::ZeroVector);
				TargetSprite->PostEditChange();
			}
			SourceTextureViewComponent->SetSpriteColor(FLinearColor::White);
			SourceTextureViewComponent->SetWorldTransform(FTransform(Translation));
		}
		else
		{
			const FVector2D PivotPosition = SourceSprite->GetPivotPosition();
			if (CurrentPivotMode != ESpritePivotMode::Custom || CurrentPivotPosition != PivotPosition)
			{
				TargetSprite->SetPivotMode(ESpritePivotMode::Custom, PivotPosition);
				TargetSprite->PostEditChange();
			}

			// Tint the source texture darker to help distinguish the two
			SourceTextureViewComponent->SetSpriteColor(SpriteEditingConstants::SourceTextureDarkTintColor);

			const bool bRotated = SourceSprite->IsRotatedInSourceImage();
			if (bRotated)
			{
				FQuat Rotation(PaperAxisZ, FMath::DegreesToRadians(90.0f));
				SourceTextureViewComponent->SetWorldTransform(FTransform(Rotation, Translation));
			}
			else
			{
				SourceTextureViewComponent->SetWorldTransform(FTransform(Translation));
			}
		}
	}
	else
	{
		// No source sprite, so don't draw the target either
		TargetSprite->SourceTexture = nullptr;
	}
}

FVector2D FSpriteEditorViewportClient::SourceTextureSpaceToScreenSpace(const FSceneView& View, const FVector2D& SourcePoint) const
{
	const FVector WorldSpacePoint = SourceTextureSpaceToWorldSpace(SourcePoint);

	FVector2D PixelLocation;
	View.WorldToPixel(WorldSpacePoint, /*out*/ PixelLocation);
	return PixelLocation;
}

FVector FSpriteEditorViewportClient::SourceTextureSpaceToWorldSpace(const FVector2D& SourcePoint) const
{
	UPaperSprite* Sprite = SourceTextureViewComponent->GetSprite();
	return Sprite->ConvertTextureSpaceToWorldSpace(SourcePoint);
}

void FSpriteEditorViewportClient::DrawRelatedSprites(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, const FLinearColor& BoundsColor, const FLinearColor& NameColor)
{
	const FLinearColor ShadowColor = FLinearColor::Black;

	for (int32 SpriteIndex = 0; SpriteIndex < RelatedSprites.Num(); ++SpriteIndex)
	{
		FRelatedSprite& RelatedSprite = RelatedSprites[SpriteIndex];
		const FVector2D& SourceUV = RelatedSprite.SourceUV;
		const FVector2D& SourceDimension = RelatedSprite.SourceDimension;

		if (bShowNamesForSprites)
		{
			const FVector2D TextPos = SourceTextureSpaceToScreenSpace(View, SourceUV + SourceDimension*0.5f);

			const FText AssetNameText = FText::AsCultureInvariant(RelatedSprite.AssetData.AssetName.ToString());
			FCanvasTextItem TextItem(TextPos, AssetNameText, GEngine->GetSmallFont(), NameColor);
			TextItem.EnableShadow(ShadowColor);
			TextItem.bCentreX = true;
			TextItem.bCentreY = true;

			TextItem.Draw(&Canvas);
		}

		if (bShowRelatedSprites)
		{
			FVector2D BoundsVertices[4];
			BoundsVertices[0] = SourceTextureSpaceToScreenSpace(View, SourceUV);
			BoundsVertices[1] = SourceTextureSpaceToScreenSpace(View, SourceUV + FVector2D(SourceDimension.X, 0));
			BoundsVertices[2] = SourceTextureSpaceToScreenSpace(View, SourceUV + FVector2D(SourceDimension.X, SourceDimension.Y));
			BoundsVertices[3] = SourceTextureSpaceToScreenSpace(View, SourceUV + FVector2D(0, SourceDimension.Y));

			for (int32 VertexIndex = 0; VertexIndex < 4; ++VertexIndex)
			{
				const int32 NextVertexIndex = (VertexIndex + 1) % 4;

				FCanvasLineItem LineItem(BoundsVertices[VertexIndex], BoundsVertices[NextVertexIndex]);
				LineItem.SetColor(BoundsColor);
				Canvas.DrawItem(LineItem);
			}
		}
	}
}

void FSpriteEditorViewportClient::DrawSourceRegion(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, const FLinearColor& GeometryVertexColor)
{
	const bool bIsHitTesting = Canvas.IsHitTesting();
	UPaperSprite* Sprite = GetSpriteBeingEdited();

    const float CornerCollisionVertexSize = 8.0f;
	const float EdgeCollisionVertexSize = 6.0f;

	const FLinearColor GeometryLineColor(GeometryVertexColor.R, GeometryVertexColor.G, GeometryVertexColor.B, 0.5f * GeometryVertexColor.A);
    
    const bool bDrawEdgeHitProxies = true;
    const bool bDrawCornerHitProxies = true;

	FVector2D BoundsVertices[4];
	BoundsVertices[0] = SourceTextureSpaceToScreenSpace(View, Sprite->SourceUV);
	BoundsVertices[1] = SourceTextureSpaceToScreenSpace(View, Sprite->SourceUV + FVector2D(Sprite->SourceDimension.X, 0));
	BoundsVertices[2] = SourceTextureSpaceToScreenSpace(View, Sprite->SourceUV + FVector2D(Sprite->SourceDimension.X, Sprite->SourceDimension.Y));
	BoundsVertices[3] = SourceTextureSpaceToScreenSpace(View, Sprite->SourceUV + FVector2D(0, Sprite->SourceDimension.Y));

	if (bShowNamesForSprites)
	{
		const FVector2D TextPos = SourceTextureSpaceToScreenSpace(View, Sprite->SourceUV + FVector2D(Sprite->SourceDimension.X*0.5f, Sprite->SourceDimension.Y*0.5f));

		const FText AssetNameText = FText::AsCultureInvariant(Sprite->GetName());
		FCanvasTextItem TextItem(TextPos, AssetNameText, GEngine->GetSmallFont(), FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.bCentreX = true;
		TextItem.bCentreY = true;

		TextItem.Draw(&Canvas);
	}

	for (int32 VertexIndex = 0; VertexIndex < 4; ++VertexIndex)
	{
		const int32 NextVertexIndex = (VertexIndex + 1) % 4;

		// Draw the edge
		if (bIsHitTesting)
		{
			TSharedPtr<FSpriteSelectedSourceRegion> Data = MakeShareable(new FSpriteSelectedSourceRegion());
			Data->SpritePtr = Sprite;
			Data->VertexIndex = 4 + VertexIndex;
			Canvas.SetHitProxy(new HSpriteSelectableObjectHitProxy(Data));
		}

		FCanvasLineItem LineItem(BoundsVertices[VertexIndex], BoundsVertices[NextVertexIndex]);
		LineItem.SetColor(GeometryVertexColor);
		Canvas.DrawItem(LineItem);

        // Add edge hit proxy
        if (bDrawEdgeHitProxies)
        {
			const FVector2D MidPoint = (BoundsVertices[VertexIndex] + BoundsVertices[NextVertexIndex]) * 0.5f;
			Canvas.DrawTile(MidPoint.X - EdgeCollisionVertexSize*0.5f, MidPoint.Y - EdgeCollisionVertexSize*0.5f, EdgeCollisionVertexSize, EdgeCollisionVertexSize, 0.f, 0.f, 1.f, 1.f, GeometryVertexColor, GWhiteTexture);
        }

		if (bIsHitTesting)
		{
			Canvas.SetHitProxy(nullptr);
		}

        // Add corner hit proxy
        if (bDrawCornerHitProxies)
        {
			const FVector2D CornerPoint = BoundsVertices[VertexIndex];
			
			if (bIsHitTesting)
            {
                TSharedPtr<FSpriteSelectedSourceRegion> Data = MakeShareable(new FSpriteSelectedSourceRegion());
                Data->SpritePtr = Sprite;
                Data->VertexIndex = VertexIndex;
                Canvas.SetHitProxy(new HSpriteSelectableObjectHitProxy(Data));
            }
            
            Canvas.DrawTile(CornerPoint.X - CornerCollisionVertexSize * 0.5f, CornerPoint.Y - CornerCollisionVertexSize * 0.5f, CornerCollisionVertexSize, CornerCollisionVertexSize, 0.f, 0.f, 1.f, 1.f, GeometryVertexColor, GWhiteTexture);

            if (bIsHitTesting)
            {
                Canvas.SetHitProxy(nullptr);
            }
        }
	}
}

void FSpriteEditorViewportClient::AnalyzeSpriteMaterialType(UPaperSprite* Sprite, int32& OutNumOpaque, int32& OutNumMasked, int32& OutNumTranslucent)
{
	struct Local
	{
		static void AttributeTrianglesByMaterialType(int32 NumTriangles, UMaterialInterface* Material, int32& NumOpaqueTriangles, int32& NumMaskedTriangles, int32& NumTranslucentTriangles)
		{
			if (Material != nullptr)
			{
				if (IsOpaqueBlendMode(*Material))
				{
					NumOpaqueTriangles += NumTriangles;
				}
				else if (IsMaskedBlendMode(*Material))
				{
					NumMaskedTriangles += NumTriangles;
				}
				else
				{
					NumTranslucentTriangles += NumTriangles;
				}
			}
		}
	};

	OutNumOpaque = 0;
	OutNumMasked = 0;
	OutNumTranslucent = 0;

	int32 NumVerts = Sprite->BakedRenderData.Num();
	int32 DefaultTriangles = 0;
	int32 AlternateTriangles = 0;
	if (Sprite->AlternateMaterialSplitIndex != INDEX_NONE)
	{
		DefaultTriangles = Sprite->AlternateMaterialSplitIndex / 3;
		AlternateTriangles = (NumVerts - Sprite->AlternateMaterialSplitIndex) / 3;
	}
	else
	{
		DefaultTriangles = NumVerts / 3;
	}

	Local::AttributeTrianglesByMaterialType(DefaultTriangles, Sprite->GetDefaultMaterial(), /*inout*/ OutNumOpaque, /*inout*/ OutNumMasked, /*inout*/ OutNumTranslucent);
	Local::AttributeTrianglesByMaterialType(AlternateTriangles, Sprite->GetAlternateMaterial(), /*inout*/ OutNumOpaque, /*inout*/ OutNumMasked, /*inout*/ OutNumTranslucent);
}

void FSpriteEditorViewportClient::DrawRenderStats(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, class UPaperSprite* Sprite, int32& YPos)
{
	FCanvasTextItem TextItem(FVector2D(6, YPos), LOCTEXT("RenderGeomBaked", "Render Geometry (baked)"), GEngine->GetSmallFont(), FLinearColor::White);
	TextItem.EnableShadow(FLinearColor::Black);

	TextItem.Draw(&Canvas);
	TextItem.Position += FVector2D(6.0f, 18.0f);

	int32 NumOpaqueTriangles = 0;
	int32 NumMaskedTriangles = 0;
	int32 NumTranslucentTriangles = 0;
	AnalyzeSpriteMaterialType(Sprite, /*out*/ NumOpaqueTriangles, /*out*/ NumMaskedTriangles, /*out*/ NumTranslucentTriangles);

	int32 NumSections = (Sprite->AlternateMaterialSplitIndex != INDEX_NONE) ? 2 : 1;
	if (NumSections > 1)
	{
		TextItem.Text = FText::Format(LOCTEXT("SectionCount", "Sections: {0}"), FText::AsNumber(NumSections));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	// Draw the number of triangles
	if (NumOpaqueTriangles > 0)
	{
		TextItem.Text = FText::Format(LOCTEXT("OpaqueTriangleCount", "Triangles: {0} (opaque)"), FText::AsNumber(NumOpaqueTriangles));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	if (NumMaskedTriangles > 0)
	{
		TextItem.Text = FText::Format(LOCTEXT("MaskedTriangleCount", "Triangles: {0} (masked)"), FText::AsNumber(NumMaskedTriangles));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	if (NumTranslucentTriangles > 0)
	{
		TextItem.Text = FText::Format(LOCTEXT("TranslucentTriangleCount", "Triangles: {0} (translucent)"), FText::AsNumber(NumTranslucentTriangles));
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	if ((NumOpaqueTriangles + NumMaskedTriangles + NumTranslucentTriangles) == 0)
	{
		static const FText NoShapesPrompt = LOCTEXT("NoRenderDataWarning", "Warning: No rendering triangles (create a new shape using the toolbar)");
		TextItem.Text = NoShapesPrompt;
		TextItem.SetColor(FLinearColor::Yellow);
		TextItem.Draw(&Canvas);
		TextItem.Position.Y += 18.0f;
	}

	YPos = (int32)TextItem.Position.Y;
}

void FSpriteEditorViewportClient::DrawBoundsAsText(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, int32& YPos)
{
	FNumberFormattingOptions NoDigitGroupingFormat;
	NoDigitGroupingFormat.UseGrouping = false;

	UPaperSprite* Sprite = GetSpriteBeingEdited();
	FBoxSphereBounds Bounds = Sprite->GetRenderBounds();

	const FText DisplaySizeText = FText::Format(LOCTEXT("BoundsSize", "Approx. Size: {0}x{1}x{2}"),
		FText::AsNumber((int32)(Bounds.BoxExtent.X * 2.0f), &NoDigitGroupingFormat),
		FText::AsNumber((int32)(Bounds.BoxExtent.Y * 2.0f), &NoDigitGroupingFormat),
		FText::AsNumber((int32)(Bounds.BoxExtent.Z * 2.0f), &NoDigitGroupingFormat));

	Canvas.DrawShadowedString(
		6,
		YPos,
		*DisplaySizeText.ToString(),
		GEngine->GetSmallFont(),
		FLinearColor::White);
	YPos += 18;
}

void FSpriteEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	const bool bIsHitTesting = Canvas.IsHitTesting();
	if (!bIsHitTesting)
	{
		Canvas.SetHitProxy(nullptr);
	}

	if (!SpriteEditorPtr.IsValid())
	{
		return;
	}

	UPaperSprite* Sprite = GetSpriteBeingEdited();

	int32 YPos = 42;

	static const FText SourceRegionHelpStr = LOCTEXT("SourceRegionHelp", "Drag handles to adjust source region\nDouble-click on an image region to select all connected pixels (Ctrl creates a new sprite)\nHold down Ctrl and drag a rectangle to create a new sprite at that position\nClick on other sprite rectangles to change the active sprite");

	switch (CurrentMode)
	{
	case ESpriteEditorMode::ViewMode:
	default:

		// Display the pivot
		{
			FNumberFormattingOptions NoDigitGroupingFormat;
			NoDigitGroupingFormat.UseGrouping = false;
			const FText PivotStr = FText::Format(LOCTEXT("PivotPosition", "Pivot: ({0}, {1})"), FText::AsNumber(Sprite->CustomPivotPoint.X, &NoDigitGroupingFormat), FText::AsNumber(Sprite->CustomPivotPoint.Y, &NoDigitGroupingFormat));
			FCanvasTextItem TextItem(FVector2D(6, YPos), PivotStr, GEngine->GetSmallFont(), FLinearColor::White);
			TextItem.EnableShadow(FLinearColor::Black);
			TextItem.Draw(&Canvas);
			YPos += 18;
		}

		// Baked collision data
		if (Sprite->BodySetup != nullptr)
		{
			FSpriteGeometryEditMode::DrawCollisionStats(InViewport, View, Canvas, Sprite->BodySetup, /*inout*/ YPos);
		}

		// Baked render data
		DrawRenderStats(InViewport, View, Canvas, Sprite, /*inout*/ YPos);

		// And bounds
		DrawBoundsAsText(InViewport, View, Canvas, /*inout*/ YPos);

		break;
	case ESpriteEditorMode::EditCollisionMode:
		{
			// Draw the collision geometry stats
			YPos += 60; //@TODO: Need a better way to determine this from the editor mode
			if (Sprite->BodySetup != nullptr)
			{
				FSpriteGeometryEditMode::DrawGeometryStats(InViewport, View, Canvas, Sprite->CollisionGeometry, false, /*inout*/ YPos);
				FSpriteGeometryEditMode::DrawCollisionStats(InViewport, View, Canvas, Sprite->BodySetup, /*inout*/ YPos);
			}
			else
			{
				FCanvasTextItem TextItem(FVector2D(6, YPos), LOCTEXT("NoCollisionDataMainScreen", "No collision data"), GEngine->GetSmallFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Draw(&Canvas);
			}
		}
		break;
	case ESpriteEditorMode::EditRenderingGeomMode:
		{
			// Draw the render geometry stats
			YPos += 60; //@TODO: Need a better way to determine this from the editor mode
			FSpriteGeometryEditMode::DrawGeometryStats(InViewport, View, Canvas, Sprite->RenderGeometry, true, /*inout*/ YPos);
			DrawRenderStats(InViewport, View, Canvas, Sprite, /*inout*/ YPos);

			// And bounds
			DrawBoundsAsText(InViewport, View, Canvas, /*inout*/ YPos);
		}
		break;
	case ESpriteEditorMode::EditSourceRegionMode:
		{
			// Display tool help
			{
				FCanvasTextItem TextItem(FVector2D(6, YPos), SourceRegionHelpStr, GEngine->GetSmallFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Draw(&Canvas);
				YPos += 18;
			}

			if (bShowRelatedSprites)
			{
				DrawRelatedSprites(InViewport, View, Canvas, SpriteEditingConstants::SourceRegionRelatedBoundsColor, SpriteEditingConstants::SourceRegionRelatedSpriteNameColor);
			}

			DrawSourceRegion(InViewport, View, Canvas, SpriteEditingConstants::SourceRegionBoundsColor);
		}
		break;
	}

	if (bShowSockets && !IsInSourceRegionEditMode())
	{
		FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
		FSocketEditingHelper::DrawSocketNames(GeometryEditMode, RenderSpriteComponent, InViewport, View, Canvas);
	}

	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);
}

void FSpriteEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);

	// We don't draw the pivot when showing the source region
	// The pivot may be outside the actual texture bounds there
	if (bShowPivot && !bShowSourceTexture && !IsInSourceRegionEditMode())
	{
		const bool bCanSelectPivot = false;
		const bool bHitTestingForPivot = PDI->IsHitTesting() && bCanSelectPivot;
		FUnrealEdUtils::DrawWidget(View, PDI, RenderSpriteComponent->GetComponentTransform().ToMatrixWithScale(), 0, 0, EAxisList::XZ, EWidgetMovementMode::WMM_Translate, bHitTestingForPivot);
	}

	if (bShowSockets && !IsInSourceRegionEditMode())
	{
		FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
		FSocketEditingHelper::DrawSockets(GeometryEditMode, RenderSpriteComponent, View, PDI);
	}
}

FBox FSpriteEditorViewportClient::GetDesiredFocusBounds() const
{
	UPaperSpriteComponent* ComponentToFocusOn = SourceTextureViewComponent->IsVisible() ? SourceTextureViewComponent : RenderSpriteComponent;
	return ComponentToFocusOn->Bounds.GetBox();
}

void FSpriteEditorViewportClient::Tick(float DeltaSeconds)
{
	if (UPaperSprite* Sprite = GetSpriteBeingEdited())
	{
		//@TODO: Doesn't need to happen every frame, only when properties are updated

		// Update the source texture view sprite (in case the texture has changed)
		UpdateSourceTextureSpriteFromSprite(Sprite);

		// Reposition the sprite (to be at the correct relative location to it's parent, undoing the pivot behavior)
		const FVector2D PivotInTextureSpace = Sprite->ConvertPivotSpaceToTextureSpace(FVector2D::ZeroVector);
		const FVector PivotInWorldSpace = TextureSpaceToWorldSpace(PivotInTextureSpace);
		RenderSpriteComponent->SetRelativeLocation(PivotInWorldSpace);

		bool bSourceTextureViewComponentVisibility = bShowSourceTexture || IsInSourceRegionEditMode();
		if (bSourceTextureViewComponentVisibility != SourceTextureViewComponent->IsVisible())
		{
			RequestFocusOnSelection(/*bInstant=*/ true);
			SourceTextureViewComponent->SetVisibility(bSourceTextureViewComponentVisibility);
		}

		bool bRenderTextureViewComponentVisibility = !IsInSourceRegionEditMode();
		if (bRenderTextureViewComponentVisibility != RenderSpriteComponent->IsVisible())
		{
			RequestFocusOnSelection(/*bInstant=*/ true);
			RenderSpriteComponent->SetVisibility(bRenderTextureViewComponentVisibility);
		}

		const FVector2D BoxSize(Sprite->GetSourceSize());
		const FVector2D BoxLocation(Sprite->GetSourceUV());
		FBox2D SpriteBounds(ForceInitToZero);
		SpriteBounds.Min = BoxLocation;
		SpriteBounds.Max = BoxLocation + BoxSize;

		if (FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry))
		{
			GeometryEditMode->SetNewGeometryPreferredBounds(SpriteBounds);
		}
	}

	FPaperEditorViewportClient::Tick(DeltaSeconds);

	OwnedPreviewScene.GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

void FSpriteEditorViewportClient::ToggleShowSourceTexture()
{
	bShowSourceTexture = !bShowSourceTexture;
	SourceTextureViewComponent->SetVisibility(bShowSourceTexture);
	
	Invalidate();
}

void FSpriteEditorViewportClient::ToggleShowRelatedSprites()
{
	bShowRelatedSprites = !bShowRelatedSprites;
	Invalidate();
}

void FSpriteEditorViewportClient::ToggleShowSpriteNames()
{
	bShowNamesForSprites = !bShowNamesForSprites;
	Invalidate();
}

void FSpriteEditorViewportClient::ToggleShowMeshEdges()
{
	EngineShowFlags.SetMeshEdges(!EngineShowFlags.MeshEdges);
	Invalidate(); 
}

bool FSpriteEditorViewportClient::IsShowMeshEdgesChecked() const
{
	return EngineShowFlags.MeshEdges;
}

// Find all related sprites (not including self)
void FSpriteEditorViewportClient::UpdateRelatedSpritesList()
{
	UPaperSprite* Sprite = GetSpriteBeingEdited();
	UTexture2D* Texture = Sprite->GetSourceTexture();
	if (Texture != nullptr)
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(UPaperSprite::StaticClass()->GetClassPathName());
		const FName SourceTexturePropName(TEXT("SourceTexture"));
		Filter.TagsAndValues.Add(SourceTexturePropName, TSoftObjectPtr<UTexture2D>(Texture).ToString());
		// Legacy format for files that haven't been resaved
		Filter.TagsAndValues.Add(SourceTexturePropName, FAssetData(Texture).GetExportTextName());

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		TArray<FAssetData> SpriteAssetData;
		AssetRegistryModule.Get().GetAssets(Filter, SpriteAssetData);

		FAssetData CurrentAssetData(Sprite);

		RelatedSprites.Empty();
		for (int32 i = 0; i < SpriteAssetData.Num(); ++i)
		{
			FAssetData& SpriteAsset = SpriteAssetData[i];
			if (SpriteAsset == Sprite)
			{
				continue;
			}

			const FString SourceUVString = SpriteAsset.GetTagValueRef<FString>("SourceUV");
			const FString SourceDimensionString = SpriteAsset.GetTagValueRef<FString>("SourceDimension");
			if (!SourceUVString.IsEmpty() && !SourceDimensionString.IsEmpty())
			{
				FVector2D SourceUV = FVector2D::ZeroVector;
				FVector2D SourceDimension = FVector2D::ZeroVector;
				if (SourceUV.InitFromString(*SourceUVString) && SourceDimension.InitFromString(*SourceDimensionString))
				{
					FRelatedSprite RelatedSprite;
					RelatedSprite.AssetData = SpriteAsset;
					RelatedSprite.SourceUV = SourceUV;
					RelatedSprite.SourceDimension = SourceDimension;
					
					RelatedSprites.Add(RelatedSprite);
				}
			}
		}
	}
}

void FSpriteEditorViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	const FViewportClick Click(&View, this, Key, Event, HitX, HitY);
	const bool bIsCtrlKeyDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	const bool bIsShiftKeyDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
	const bool bIsAltKeyDown = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);
	bool bHandled = false;

	HSpriteSelectableObjectHitProxy* SelectedItemProxy = HitProxyCast<HSpriteSelectableObjectHitProxy>(HitProxy);

	if (IsInSourceRegionEditMode())
	{
		if ((Event == EInputEvent::IE_DoubleClick) && (Key == EKeys::LeftMouseButton))
		{
			FVector4 WorldPoint = View.PixelToWorld(HitX, HitY, 0);
			UPaperSprite* Sprite = GetSpriteBeingEdited();
			FVector2D TexturePoint = SourceTextureViewComponent->GetSprite()->ConvertWorldSpaceToTextureSpace(WorldPoint);
			if (bIsCtrlKeyDown)
			{
				const FVector2D StartingUV = Sprite->GetSourceUV();
				const FVector2D StartingSize = Sprite->GetSourceSize();

				if (UPaperSprite* NewSprite = CreateNewSprite(FIntPoint((int32)StartingUV.X, (int32)StartingUV.Y), FIntPoint((int32)StartingSize.X, (int32)StartingSize.Y)))
				{
					NewSprite->ExtractSourceRegionFromTexturePoint(TexturePoint);
					bHandled = true;
				}
			}
			else
			{
				Sprite->ExtractSourceRegionFromTexturePoint(TexturePoint);
				bHandled = true;
			}
		}
		else if ((Event == EInputEvent::IE_Released) && (Key == EKeys::LeftMouseButton))
		{
			FVector4 WorldPoint = View.PixelToWorld(HitX, HitY, 0);
			FVector2D TexturePoint = SourceTextureViewComponent->GetSprite()->ConvertWorldSpaceToTextureSpace(WorldPoint);
			for (int32 RelatedSpriteIndex = 0; RelatedSpriteIndex < RelatedSprites.Num(); ++RelatedSpriteIndex)
			{
				FRelatedSprite& RelatedSprite = RelatedSprites[RelatedSpriteIndex];
				if ((TexturePoint.X >= RelatedSprite.SourceUV.X) && (TexturePoint.Y >= RelatedSprite.SourceUV.Y) &&
					(TexturePoint.X < (RelatedSprite.SourceUV.X + RelatedSprite.SourceDimension.X)) &&
					(TexturePoint.Y < (RelatedSprite.SourceUV.Y + RelatedSprite.SourceDimension.Y)))
				{
					bHandled = true;

					// Select this sprite
					if (UPaperSprite* LoadedSprite = Cast<UPaperSprite>(RelatedSprite.AssetData.GetAsset()))
					{
						if (SpriteEditorPtr.IsValid())
						{
							SpriteEditorPtr.Pin()->SetSpriteBeingEdited(LoadedSprite);
							break;
						}
					}
				}
			}
		}
	}

	if (!bHandled)
	{
		FPaperEditorViewportClient::ProcessClick(View, HitProxy, Key, Event, HitX, HitY);
	}
}

// Create a new sprite and return this sprite. The sprite editor will now be editing this new sprite
// Returns nullptr if failed
UPaperSprite* FSpriteEditorViewportClient::CreateNewSprite(const FIntPoint& TopLeft, const FIntPoint& Dimensions)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	UPaperSprite* CurrentSprite = GetSpriteBeingEdited();
	UPaperSprite* CreatedSprite = nullptr;

	// Create the factory used to generate the sprite
	UPaperSpriteFactory* SpriteFactory = NewObject<UPaperSpriteFactory>();
	SpriteFactory->InitialTexture = CurrentSprite->GetSourceTexture();
	SpriteFactory->bUseSourceRegion = true;
	SpriteFactory->InitialSourceUV = TopLeft;
	SpriteFactory->InitialSourceDimension = Dimensions;


	// Get a unique name for the sprite
	FString Name, PackageName;
	AssetToolsModule.Get().CreateUniqueAssetName(CurrentSprite->GetOutermost()->GetName(), TEXT(""), /*out*/ PackageName, /*out*/ Name);
	const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
	if (UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, PackagePath, UPaperSprite::StaticClass(), SpriteFactory))
	{
		TArray<UObject*> Objects;
		Objects.Add(NewAsset);
		ContentBrowserModule.Get().SyncBrowserToAssets(Objects);

		UPaperSprite* NewSprite = Cast<UPaperSprite>(NewAsset);
		if (SpriteEditorPtr.IsValid() && (NewSprite != nullptr))
		{
			SpriteEditorPtr.Pin()->SetSpriteBeingEdited(NewSprite);
		}

		CreatedSprite = NewSprite;
	}

	return CreatedSprite;
}

bool FSpriteEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = false;
	FInputEventState InputState(EventArgs.Viewport, EventArgs.Key, EventArgs.Event);

	// Handle marquee tracking in source region edit mode
	if (IsInSourceRegionEditMode())
	{
		FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
		check(GeometryEditMode);

		const bool bMarqueeStartModifier = InputState.IsCtrlButtonPressed();
		if (GeometryEditMode->ProcessMarquee(EventArgs.Viewport, EventArgs.Key, EventArgs.Event, bMarqueeStartModifier))
		{
			FIntPoint TextureSpaceStartPos;
			FIntPoint TextureSpaceDimensions;
			if (ConvertMarqueeToSourceTextureSpace(/*out*/TextureSpaceStartPos, /*out*/TextureSpaceDimensions))
			{
				//@TODO: Warn if overlapping with another sprite
				CreateNewSprite(TextureSpaceStartPos, TextureSpaceDimensions);
			}
		}
	}
	
	// Pass keys to standard controls, if we didn't consume input
	return (bHandled) ? true : FEditorViewportClient::InputKey(EventArgs);
}

void FSpriteEditorViewportClient::TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge)
{
	//@TODO: Should push this into FEditorViewportClient
	// Begin transacting.  Give the current editor mode an opportunity to do the transacting.
	const bool bTrackingHandledExternally = ModeTools->StartTracking(this, Viewport);

	if (!bManipulating && bIsDragging && !bTrackingHandledExternally)
	{
		BeginTransaction(LOCTEXT("ModificationInViewportTransaction", "Modification in Viewport"));
		bManipulating = true;
		bManipulationDirtiedSomething = false;
	}
}

void FSpriteEditorViewportClient::TrackingStopped() 
{
	// Stop transacting.  Give the current editor mode an opportunity to do the transacting.
	const bool bTransactingHandledByEditorMode = ModeTools->EndTracking(this, Viewport);

	if (bManipulating && !bTransactingHandledByEditorMode)
	{
		EndTransaction();
		bManipulating = false;
	}
}

FLinearColor FSpriteEditorViewportClient::GetBackgroundColor() const
{
	return GetDefault<USpriteEditorSettings>()->BackgroundColor;
}

FVector2D FSpriteEditorViewportClient::SelectedItemConvertWorldSpaceDeltaToLocalSpace(const FVector& WorldSpaceDelta) const
{
	UPaperSprite* Sprite = GetSpriteBeingEdited();
	return Sprite->ConvertWorldSpaceDeltaToTextureSpace(WorldSpaceDelta);
}

FVector2D FSpriteEditorViewportClient::WorldSpaceToTextureSpace(const FVector& SourcePoint) const
{
	UPaperSprite* Sprite = GetSpriteBeingEdited();
	return Sprite->ConvertWorldSpaceToTextureSpace(SourcePoint);
}

FVector FSpriteEditorViewportClient::TextureSpaceToWorldSpace(const FVector2D& SourcePoint) const
{
	UPaperSprite* Sprite = GetSpriteBeingEdited();
	return Sprite->ConvertTextureSpaceToWorldSpace(SourcePoint);
}

float FSpriteEditorViewportClient::SelectedItemGetUnitsPerPixel() const
{
	UPaperSprite* Sprite = GetSpriteBeingEdited();
	return Sprite->GetUnrealUnitsPerPixel();
}

void FSpriteEditorViewportClient::BeginTransaction(const FText& SessionName)
{
	if (ScopedTransaction == nullptr)
	{
		ScopedTransaction = new FScopedTransaction(SessionName);

		UPaperSprite* Sprite = GetSpriteBeingEdited();
		Sprite->Modify();
	}
}

void FSpriteEditorViewportClient::MarkTransactionAsDirty()
{
	bManipulationDirtiedSomething = true;
	Invalidate();
	//@TODO: Can add a call to Sprite->PostEditChange here if we want to update the baked sprite data during a drag operation
	// (maybe passing in Interactive - if so, the EndTransaction PostEditChange needs to be a ValueSet)
}

void FSpriteEditorViewportClient::EndTransaction()
{
	if (bManipulationDirtiedSomething)
	{
		UPaperSprite* Sprite = GetSpriteBeingEdited();

		if (IsInSourceRegionEditMode())
		{
			// Snap to pixel grid at the end of the drag
			Sprite->SourceUV.X = FMath::Max(FMath::RoundToFloat(Sprite->SourceUV.X), 0.0f);
			Sprite->SourceUV.Y = FMath::Max(FMath::RoundToFloat(Sprite->SourceUV.Y), 0.0f);
			Sprite->SourceDimension.X = FMath::Max(FMath::RoundToFloat(Sprite->SourceDimension.X), 0.0f);
			Sprite->SourceDimension.Y = FMath::Max(FMath::RoundToFloat(Sprite->SourceDimension.Y), 0.0f);
		}

		Sprite->PostEditChange();
	}
	
	bManipulationDirtiedSomething = false;

	if (ScopedTransaction != nullptr)
	{
		delete ScopedTransaction;
		ScopedTransaction = nullptr;
	}
}

void FSpriteEditorViewportClient::InvalidateViewportAndHitProxies()
{
	Viewport->Invalidate();
}

void FSpriteEditorViewportClient::NotifySpriteBeingEditedHasChanged()
{
	//@TODO: Ideally we do this before switching
	EndTransaction();

	// Refresh the viewport in case we were not in realtime mode
	Invalidate();

	// Update components to know about the new sprite being edited
	UPaperSprite* Sprite = GetSpriteBeingEdited();

	RenderSpriteComponent->SetSprite(Sprite);
	UpdateSourceTextureSpriteFromSprite(Sprite);

	InternalActivateNewMode(CurrentMode);

	//@TODO: Only do this if the sprite isn't visible (may consider doing a flashing pulse around the source region rect?)
	RequestFocusOnSelection(/*bInstant=*/ true);

	if (Sprite != nullptr)
	{
		// Create and display a notification about the new sprite being edited
		const FText NotificationErrorText = FText::Format(LOCTEXT("SwitchingToSprite", "Editing {0}"), FText::AsCultureInvariant(Sprite->GetName()));
		FNotificationInfo Info(NotificationErrorText);
		Info.ExpireDuration = 2.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void FSpriteEditorViewportClient::InternalActivateNewMode(ESpriteEditorMode::Type NewMode)
{
	CurrentMode = NewMode;
	Viewport->InvalidateHitProxy();

	UPaperSprite* Sprite = GetSpriteBeingEdited();

	FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
	check(GeometryEditMode);

	// Note: This has side effects (clearing the selection set, ensuring the geometry is correct if the sprite being edited changed, etc...).
	// Do not skip even if the mode is not really changing.
	GeometryEditMode->SetGeometryBeingEdited(nullptr, /*bAllowCircles=*/ false, /*bAllowSubtractivePolygons=*/ false);

	switch (CurrentMode)
	{
	case ESpriteEditorMode::ViewMode:
		break;
	case ESpriteEditorMode::EditSourceRegionMode:
		UpdateRelatedSpritesList();
		break;
	case ESpriteEditorMode::EditCollisionMode:
		GeometryEditMode->SetGeometryColors(SpriteEditingConstants::CollisionShapeColor, FLinearColor::White);
		if (Sprite != nullptr)
		{
			GeometryEditMode->SetGeometryBeingEdited(&(Sprite->CollisionGeometry), /*bAllowCircles=*/ true, /*bAllowSubtractivePolygons=*/ false);
		}
		break;
	case ESpriteEditorMode::EditRenderingGeomMode:
		GeometryEditMode->SetGeometryColors(SpriteEditingConstants::RenderShapeColor, SpriteEditingConstants::SubtractiveRenderShapeColor);
		if (Sprite != nullptr)
		{
			GeometryEditMode->SetGeometryBeingEdited(&(Sprite->RenderGeometry), /*bAllowCircles=*/ false, /*bAllowSubtractivePolygons=*/ true);
		}
		break;
	}
}

bool FSpriteEditorViewportClient::ConvertMarqueeToSourceTextureSpace(/*out*/ FIntPoint& OutStartPos, /*out*/ FIntPoint& OutDimension)
{
	FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
	check(GeometryEditMode);
	const FVector2D MarqueeStartPos = GeometryEditMode->GetMarqueeStartPos();
	const FVector2D MarqueeEndPos = GeometryEditMode->GetMarqueeEndPos();

	bool bSuccessful = false;
	UPaperSprite* Sprite = SourceTextureViewComponent->GetSprite();
	UTexture2D* SpriteSourceTexture = Sprite->GetSourceTexture();
	if (SpriteSourceTexture != nullptr)
	{
		// Calculate world space positions
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(Viewport, GetScene(), EngineShowFlags));
		FSceneView* View = CalcSceneView(&ViewFamily);
		const FVector StartPos = View->PixelToWorld(MarqueeStartPos.X, MarqueeStartPos.Y, 0);
		const FVector EndPos = View->PixelToWorld(MarqueeEndPos.X, MarqueeEndPos.Y, 0);

		// Convert to source texture space to work out the pixels dragged
		FVector2D TextureSpaceStartPos = Sprite->ConvertWorldSpaceToTextureSpace(StartPos);
		FVector2D TextureSpaceEndPos = Sprite->ConvertWorldSpaceToTextureSpace(EndPos);

		if (TextureSpaceStartPos.X > TextureSpaceEndPos.X)
		{
			Swap(TextureSpaceStartPos.X, TextureSpaceEndPos.X);
		}
		if (TextureSpaceStartPos.Y > TextureSpaceEndPos.Y)
		{
			Swap(TextureSpaceStartPos.Y, TextureSpaceEndPos.Y);
		}

		const FIntPoint SourceTextureSize(SpriteSourceTexture->GetImportedSize());
		const int32 SourceTextureWidth = SourceTextureSize.X;
		const int32 SourceTextureHeight = SourceTextureSize.Y;
		
		FIntPoint TSStartPos;
		TSStartPos.X = FMath::Clamp<int32>((int32)TextureSpaceStartPos.X, 0, SourceTextureWidth - 1);
		TSStartPos.Y = FMath::Clamp<int32>((int32)TextureSpaceStartPos.Y, 0, SourceTextureHeight - 1);

		FIntPoint TSEndPos;
		TSEndPos.X = FMath::Clamp<int32>((int32)TextureSpaceEndPos.X, 0, SourceTextureWidth - 1);
		TSEndPos.Y = FMath::Clamp<int32>((int32)TextureSpaceEndPos.Y, 0, SourceTextureHeight - 1);

		const FIntPoint TextureSpaceDimensions = TSEndPos - TSStartPos;
		if ((TextureSpaceDimensions.X > 0) || (TextureSpaceDimensions.Y > 0))
		{
			OutStartPos = TSStartPos;
			OutDimension = TextureSpaceDimensions;
			bSuccessful = true;
		}
	}

	return bSuccessful;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
