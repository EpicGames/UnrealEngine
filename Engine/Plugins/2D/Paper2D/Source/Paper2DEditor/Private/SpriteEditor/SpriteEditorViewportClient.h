// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreviewScene.h"
#include "SpriteEditor/SpriteEditor.h"
#include "SpriteEditor/SpriteEditorSelections.h"
#include "PaperEditorViewportClient.h"

class FCanvas;
class UPaperSpriteComponent;

//////////////////////////////////////////////////////////////////////////
// FRelatedSprite

struct FRelatedSprite
{
	FAssetData AssetData;
	FVector2D SourceUV;
	FVector2D SourceDimension;
};

//////////////////////////////////////////////////////////////////////////
// FSpriteEditorViewportClient

class FSpriteEditorViewportClient : public FPaperEditorViewportClient, public ISpriteSelectionContext
{
public:
	/** Constructor */
	FSpriteEditorViewportClient(TWeakPtr<FSpriteEditor> InSpriteEditor, TWeakPtr<class SEditorViewport> InSpriteEditorViewportPtr);

	// FViewportClient interface
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual void Tick(float DeltaSeconds) override;
	// End of FViewportClient interface

	// FEditorViewportClient interface
	virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual void TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
	virtual void TrackingStopped() override;
	virtual FLinearColor GetBackgroundColor() const override;
	// End of FEditorViewportClient interface

	// ISpriteSelectionContext interface
	virtual FVector2D SelectedItemConvertWorldSpaceDeltaToLocalSpace(const FVector& WorldSpaceDelta) const override;
	virtual FVector2D WorldSpaceToTextureSpace(const FVector& SourcePoint) const override;
	virtual FVector TextureSpaceToWorldSpace(const FVector2D& SourcePoint) const override;
	virtual float SelectedItemGetUnitsPerPixel() const override;
	virtual void BeginTransaction(const FText& SessionName) override;
	virtual void MarkTransactionAsDirty() override;
	virtual void EndTransaction() override;
	virtual void InvalidateViewportAndHitProxies() override;
	// End of ISpriteSelectionContext interface

	void ActivateEditMode();

	void ToggleShowSockets() { bShowSockets = !bShowSockets; Invalidate(); }
	bool IsShowSocketsChecked() const { return bShowSockets; }

	void ToggleShowPivot() { bShowPivot = !bShowPivot; Invalidate(); }
	bool IsShowPivotChecked() const { return bShowPivot; }

	void ToggleShowMeshEdges();
	bool IsShowMeshEdgesChecked() const;

	void EnterViewMode() { InternalActivateNewMode(ESpriteEditorMode::ViewMode); }
	void EnterSourceRegionEditMode() { InternalActivateNewMode(ESpriteEditorMode::EditSourceRegionMode); }
	void EnterCollisionEditMode() { InternalActivateNewMode(ESpriteEditorMode::EditCollisionMode); }
	void EnterRenderingEditMode() { InternalActivateNewMode(ESpriteEditorMode::EditRenderingGeomMode); }

	bool IsInViewMode() const { return CurrentMode == ESpriteEditorMode::ViewMode; }
	bool IsInSourceRegionEditMode() const { return CurrentMode == ESpriteEditorMode::EditSourceRegionMode; }
	bool IsInCollisionEditMode() const { return CurrentMode == ESpriteEditorMode::EditCollisionMode; }
	bool IsInRenderingEditMode() const { return CurrentMode == ESpriteEditorMode::EditRenderingGeomMode; }

	//
	bool IsEditingGeometry() const { return IsInCollisionEditMode() || IsInRenderingEditMode(); }

	void ToggleShowSourceTexture();
	bool IsShowSourceTextureChecked() const { return bShowSourceTexture; }
	bool CanShowSourceTexture() const { return !IsInSourceRegionEditMode(); }

	void ToggleShowRelatedSprites();
	bool IsShowRelatedSpritesChecked() const { return bShowRelatedSprites; }

	void ToggleShowSpriteNames();
	bool IsShowSpriteNamesChecked() const { return bShowNamesForSprites; }

	// Invalidate any references to the sprite being edited; it has changed
	void NotifySpriteBeingEditedHasChanged();

	void UpdateRelatedSpritesList();

	UPaperSprite* CreateNewSprite(const FIntPoint& TopLeft, const FIntPoint& Dimensions);

	ESpriteEditorMode::Type GetCurrentMode() const
	{
		return CurrentMode;
	}

	static void AnalyzeSpriteMaterialType(UPaperSprite* Sprite, int32& OutNumOpaque, int32& OutNumMasked, int32& OutNumTranslucent);

protected:
	// FPaperEditorViewportClient interface
	virtual FBox GetDesiredFocusBounds() const override;
	// End of FPaperEditorViewportClient interface

private:
	// Editor mode
	ESpriteEditorMode::Type CurrentMode;

	// The preview scene
	FPreviewScene OwnedPreviewScene;

	// Sprite editor that owns this viewport
	TWeakPtr<FSpriteEditor> SpriteEditorPtr;

	// Render component for the source texture view
	UPaperSpriteComponent* SourceTextureViewComponent;

	// Render component for the sprite being edited
	UPaperSpriteComponent* RenderSpriteComponent;

	// Widget mode
// 	UE::Widget::EWidgetMode DesiredWidgetMode;

	// Are we currently manipulating something?
	bool bManipulating;

	// Did we dirty something during manipulation?
	bool bManipulationDirtiedSomething;

	// Pointer back to the sprite editor viewport control that owns us
	TWeakPtr<class SEditorViewport> SpriteEditorViewportPtr;

	// The current transaction for undo/redo
	class FScopedTransaction* ScopedTransaction;

	// Should we show the source texture?
	bool bShowSourceTexture;

	// Should we show sockets?
	bool bShowSockets;

	// Should we show the sprite pivot?
	bool bShowPivot;

	// Should we show related sprites in the source texture?
	bool bShowRelatedSprites;

	// Should we show names for sprites in the source region edit mode?
	bool bShowNamesForSprites;

	// Other sprites that share the same source texture
	TArray<FRelatedSprite> RelatedSprites;

private:
	UPaperSprite* GetSpriteBeingEdited() const
	{
		return SpriteEditorPtr.Pin()->GetSpriteBeingEdited();
	}

	// Position relative to source texture (ignoring rotation and other transformations applied to extract the sprite)
	FVector2D SourceTextureSpaceToScreenSpace(const FSceneView& View, const FVector2D& SourcePoint) const;
	FVector SourceTextureSpaceToWorldSpace(const FVector2D& SourcePoint) const;

	void DrawBoundsAsText(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, int32& YPos);

	void DrawRenderStats(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, class UPaperSprite* Sprite, int32& YPos);
	void DrawSourceRegion(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, const FLinearColor& GeometryVertexColor);
	void DrawRelatedSprites(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, const FLinearColor& BoundsColor, const FLinearColor& NameColor);

	void UpdateSourceTextureSpriteFromSprite(UPaperSprite* SourceSprite);
	
	bool ConvertMarqueeToSourceTextureSpace(/*out*/ FIntPoint& OutStartPos, /*out*/ FIntPoint& OutDimension);

	// Activates a new mode, clearing selection set, etc...
	void InternalActivateNewMode(ESpriteEditorMode::Type NewMode);
};
