// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriteEditor/SpritePolygonCollectionCustomization.h"
#include "PropertyHandle.h"

//////////////////////////////////////////////////////////////////////////
// FSpritePolygonCollectionCustomization

TSharedRef<IPropertyTypeCustomization> FSpritePolygonCollectionCustomization::MakeInstance()
{
	return MakeShareable(new FSpritePolygonCollectionCustomization);
}

void FSpritePolygonCollectionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FSpritePolygonCollectionCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}
#if 0
void FSpritePolygonCollectionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Make sure sprite properties are near the top
	DetailLayout.EditCategory("Sprite", TEXT(""), ECategoryPriority::TypeSpecific);

	TSharedRef<IPropertyHandle> RenderGeometry = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, RenderGeometry));

	IDetailCategoryBuilder& RenderingCategory = DetailLayout.EditCategory("Rendering");
	IDetailPropertyRow& RenderGeometryProperty = RenderingCategory.AddProperty(RenderGeometry);

	// Build the collision category
	IDetailCategoryBuilder& CollisionCategory = DetailLayout.EditCategory("Collision");
	
	TSharedPtr<IPropertyHandle> SpriteCollisionDomainProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, SpriteCollisionDomain));
	TAttribute<EVisibility> ParticipatesInPhysics = TAttribute<EVisibility>::Create( TAttribute<EVisibility>::FGetter::CreateSP( this, &FSpriteDetailsCustomization::AnyPhysicsMode, SpriteCollisionDomainProperty) ) ;
	TAttribute<EVisibility> ParticipatesInPhysics3D = TAttribute<EVisibility>::Create( TAttribute<EVisibility>::FGetter::CreateSP( this, &FSpriteDetailsCustomization::PhysicsModeMatches, SpriteCollisionDomainProperty, ESpriteCollisionMode::Use3DPhysics) ) ;

	CollisionCategory.AddProperty(SpriteCollisionDomainProperty);
	CollisionCategory.AddProperty( DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, CollisionGeometry)) )
		.Visibility(ParticipatesInPhysics);
	CollisionCategory.AddProperty( DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperSprite, CollisionThickness)) )
		.Visibility(ParticipatesInPhysics3D);
}
#endif

EVisibility FSpritePolygonCollectionCustomization::PolygonModeMatches(TSharedPtr<IPropertyHandle> Property, ESpritePolygonMode::Type DesiredMode) const
{
	if (Property.IsValid())
	{
		uint8 ValueAsByte;
		FPropertyAccess::Result Result = Property->GetValue(/*out*/ ValueAsByte);

		if (Result == FPropertyAccess::Success)
		{
			return (((ESpritePolygonMode::Type)ValueAsByte) == DesiredMode) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	// If there are multiple values, show all properties
	return EVisibility::Visible;
}
