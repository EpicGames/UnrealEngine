// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace ESpritePolygonMode { enum Type : int; }
struct EVisibility;

//////////////////////////////////////////////////////////////////////////
// FSpritePolygonCollectionCustomization

class FSpritePolygonCollectionCustomization : public IPropertyTypeCustomization
{
public:
	// Makes a new instance of this customization
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End of IPropertyTypeCustomization interface

protected:
	EVisibility PolygonModeMatches(TSharedPtr<IPropertyHandle> Property, ESpritePolygonMode::Type DesiredMode) const;
};
