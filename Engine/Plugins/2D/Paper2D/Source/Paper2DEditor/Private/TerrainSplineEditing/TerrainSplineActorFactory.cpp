// Copyright Epic Games, Inc. All Rights Reserved.

#include "TerrainSplineActorFactory.h"
#include "AssetRegistry/AssetData.h"
#include "PaperTerrainActor.h"
#include "PaperTerrainMaterial.h"
#include "PaperTerrainComponent.h"
#include "PaperRuntimeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TerrainSplineActorFactory)

//////////////////////////////////////////////////////////////////////////
// UTerrainSplineActorFactory

UTerrainSplineActorFactory::UTerrainSplineActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = NSLOCTEXT("Paper2D", "TerrainSplineFactoryDisplayName", "Paper2D Terrain Spline");
	NewActorClass = APaperTerrainActor::StaticClass();
}

void UTerrainSplineActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	if (UPaperTerrainMaterial* TerrainMaterial = Cast<UPaperTerrainMaterial>(Asset))
	{
		APaperTerrainActor* TypedActor = CastChecked<APaperTerrainActor>(NewActor);
		UPaperTerrainComponent* RenderComponent = TypedActor->GetRenderComponent();
		check(RenderComponent);

		RenderComponent->UnregisterComponent();
		RenderComponent->TerrainMaterial = TerrainMaterial;
		RenderComponent->RegisterComponent();
	}
}

bool UTerrainSplineActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (GetDefault<UPaperRuntimeSettings>()->bEnableTerrainSplineEditing)
	{
		if (AssetData.IsValid() && AssetData.IsInstanceOf(UPaperTerrainMaterial::StaticClass()))
		{
			return true;
		}
		else
		{
			return Super::CanCreateActorFrom(AssetData, OutErrorMsg);
		}
	}
	else
	{
		return false;
	}
}

