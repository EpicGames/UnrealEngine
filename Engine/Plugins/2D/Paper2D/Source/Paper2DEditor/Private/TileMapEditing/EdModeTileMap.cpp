// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/EdModeTileMap.h"
#include "EditorViewportClient.h"
#include "NavigationSystem.h"
#include "Framework/Application/SlateApplication.h"
#include "EditorModeManager.h"
#include "Paper2DModule.h"
#include "PaperTileLayer.h"
#include "PaperTileMapComponent.h"
#include "PaperTileSet.h"
#include "TileMapEditing/TileMapEdModeToolkit.h"
#include "Selection.h"
#include "Toolkits/ToolkitManager.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "CanvasTypes.h"
#include "PaperTileMap.h"

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// Editor constants

namespace TileMapEditorConstants
{
	const float IntervalBetweenNavMeshRebuilds = 2.0f;
	const bool bFlushEntireComponentWhenNavMeshIsDirty = true;
};

//////////////////////////////////////////////////////////////////////////
// FTileMapDirtyRegion

FTileMapDirtyRegion::FTileMapDirtyRegion(UPaperTileMapComponent* InComponent, const FBox& DirtyRegionInTileSpace)
	: ComponentPtr(InComponent)
	, DirtyRegionInWorldSpace(ForceInitToZero)
{
	if (DirtyRegionInTileSpace.IsValid)
	{
		if (UPaperTileMap* TileMap = InComponent->TileMap)
		{
			const FTransform ComponentToWorld = InComponent->GetComponentTransform();
			const FVector MinCoordLS = TileMap->GetTilePositionInLocalSpace(DirtyRegionInTileSpace.Min.X, DirtyRegionInTileSpace.Min.Y, (int32)DirtyRegionInTileSpace.Min.Z);
			const FVector MaxCoordLS = TileMap->GetTilePositionInLocalSpace(DirtyRegionInTileSpace.Max.X + 1.0f, DirtyRegionInTileSpace.Max.Y + 1.0f, (int32)DirtyRegionInTileSpace.Max.Z);
			
			DirtyRegionInWorldSpace += ComponentToWorld.TransformPosition(MinCoordLS - TileMap->GetCollisionThickness() * PaperAxisZ);
			DirtyRegionInWorldSpace += ComponentToWorld.TransformPosition(MaxCoordLS + TileMap->GetCollisionThickness() * PaperAxisZ);

			//DrawDebugBox(InComponent->GetWorld(), DirtyRegionInWorldSpace.GetCenter(), DirtyRegionInWorldSpace.GetExtent(), FQuat::Identity, FColor::White, true, 1.0f);
		}
	}
}

void FTileMapDirtyRegion::PushToNavSystem() const
{
	if (UPaperTileMapComponent* Component = ComponentPtr.Get())
	{
		if (Component->IsNavigationRelevant())
		{
			if (UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Component))
			{
				NavSys->AddDirtyArea(DirtyRegionInWorldSpace, ENavigationDirtyFlag::All);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FHorizontalSpan - used for flood filling

struct FHorizontalSpan
{
	int32 X0;
	int32 X1;
	int32 Y;

	FHorizontalSpan(int32 InX, int32 InY)
		: X0(InX)
		, X1(InX)
		, Y(InY)
	{
	}

	// Indexes a bit in the reachability array
	static FBitReference Reach(UPaperTileLayer* Layer, TBitArray<>& Reachability, int32 X, int32 Y)
	{
		const int32 Index = (Layer->GetLayerWidth() * Y) + X;
		return Reachability[Index];
	}

	// Grows a span horizontally until it reaches something that doesn't match
	void GrowSpan(const FPaperTileInfo& RequiredInk, UPaperTileLayer* Layer, TBitArray<>& Reachability)
	{
		// Go left
		for (int32 TestX = X0 - 1; TestX >= 0; --TestX)
		{
			const FPaperTileInfo ExistingCell = Layer->GetCell(TestX, Y);
			const bool bCellMatches = (ExistingCell == RequiredInk) || (!ExistingCell.IsValid() && !RequiredInk.IsValid());
			if (bCellMatches && !Reach(Layer, Reachability, TestX, Y))
			{
				X0 = TestX;
			}
			else
			{
				break;
			}
		}

		// Go right
		for (int32 TestX = X1 + 1; TestX < Layer->GetLayerWidth(); ++TestX)
		{
			const FPaperTileInfo ExistingCell = Layer->GetCell(TestX, Y);
			const bool bCellMatches = (ExistingCell == RequiredInk) || (!ExistingCell.IsValid() && !RequiredInk.IsValid());
			if (bCellMatches && !Reach(Layer, Reachability, TestX, Y))
			{
				X1 = TestX;
			}
			else
			{
				break;
			}
		}

		// Commit the span to the reachability array
		for (int32 X = X0; X <= X1; ++X)
		{
			Reach(Layer, Reachability, X, Y) = true;
		}
	}
};

//////////////////////////////////////////////////////////////////////////
// FEdModeTileMap

const FEditorModeID FEdModeTileMap::EM_TileMap(TEXT("EM_TileMap"));

FEdModeTileMap::FEdModeTileMap()
	: bWasPainting(false)
	, bIsPainting(false)
	, bHasValidInkSource(false)
	, bWasHoldingSelectWhenPaintingStarted(false)
	, bIsLastCursorValid(false)
	, DrawPreviewDimensionsLS(0.0f, 0.0f, 0.0f)
	, EraseBrushSize(1)
	, TimeUntilNavMeshRebuild(TileMapEditorConstants::IntervalBetweenNavMeshRebuilds)
	, ActiveTool(ETileMapEditorTool::Paintbrush)
{
	bDrawPivot = false;
	bDrawGrid = false;
}

FEdModeTileMap::~FEdModeTileMap()
{
}

void FEdModeTileMap::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdMode::AddReferencedObjects(Collector);
}

bool FEdModeTileMap::UsesToolkits() const
{
	return true;
}

void FEdModeTileMap::Enter()
{
	FEdMode::Enter();

	UWorld* World = GetWorld();

	CursorPreviewComponent = NewObject<UPaperTileMapComponent>();
	CursorPreviewComponent->TileMap->InitializeNewEmptyTileMap();
	CursorPreviewComponent->TranslucencySortPriority = 99999;
	CursorPreviewComponent->bShowPerTileGridWhenSelected = false;
	CursorPreviewComponent->bShowPerTileGridWhenUnselected = false;
	CursorPreviewComponent->bShowPerLayerGridWhenSelected = false;
	CursorPreviewComponent->bShowPerLayerGridWhenUnselected = false;
	CursorPreviewComponent->bShowOutlineWhenUnselected = false;
	CursorPreviewComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	CursorPreviewComponent->UpdateBounds();
	CursorPreviewComponent->AddToRoot();
	CursorPreviewComponent->RegisterComponentWithWorld(World);
	CursorPreviewComponent->SetMobility(EComponentMobility::Static);

	SetActiveTool(ETileMapEditorTool::Paintbrush);

	if (!Toolkit.IsValid())
	{
		Toolkit = MakeShareable(new FTileMapEdModeToolkit(this));
		Toolkit->Init(Owner->GetToolkitHost());
	}
}

void FEdModeTileMap::Exit()
{
	FlushPendingDirtyRegions();

	if (Toolkit.IsValid())
	{
		FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
		Toolkit.Reset();
	}

	CursorPreviewComponent->RemoveFromRoot();
	CursorPreviewComponent->UnregisterComponent();
	CursorPreviewComponent = nullptr;

	// Call base Exit method to ensure proper cleanup
	FEdMode::Exit();
}

void FEdModeTileMap::FlushPendingDirtyRegions()
{
	TSet<UPaperTileMapComponent*> ComponentsToInvalidate;

	for (const FTileMapDirtyRegion& DirtyRegion : PendingDirtyRegions)
	{
		if (UPaperTileMapComponent* Component = DirtyRegion.GetComponent())
		{
			if (Component->IsNavigationRelevant())
			{
				if (TileMapEditorConstants::bFlushEntireComponentWhenNavMeshIsDirty)
				{
					ComponentsToInvalidate.Add(Component);
				}
				else
				{
					DirtyRegion.PushToNavSystem();
				}
			}
		}
	}

	for (UPaperTileMapComponent* Component : ComponentsToInvalidate)
	{
		if (Component)
		{
			FNavigationSystem::UpdateComponentData(*Component);
		}		
	}

	PendingDirtyRegions.Empty();
}

void FEdModeTileMap::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);

	if (PendingDirtyRegions.Num() > 0)
	{
		TimeUntilNavMeshRebuild -= DeltaTime;
		if (TimeUntilNavMeshRebuild <= 0.0f)
		{
			TimeUntilNavMeshRebuild = TileMapEditorConstants::IntervalBetweenNavMeshRebuilds;
			FlushPendingDirtyRegions();
		}
	}
}

void FEdModeTileMap::ActorSelectionChangeNotify()
{
	if (FindSelectedComponent() == nullptr)
	{
		Owner->DeactivateMode(FEdModeTileMap::EM_TileMap);
	}
}

bool FEdModeTileMap::MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
	if (ViewportClient->EngineShowFlags.ModeWidgets)
	{
		const FViewportCursorLocation Ray = CalculateViewRay(ViewportClient, Viewport);
		UpdatePreviewCursor(Ray);
	}

	RefreshBrushSize();

	return FEdMode::MouseEnter(ViewportClient, Viewport, x, y);
}

bool FEdModeTileMap::MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	DrawPreviewDimensionsLS = FVector::ZeroVector;
	bIsLastCursorValid = false;
	LastCursorTileMap.Reset();

	CursorPreviewComponent->SetVisibility(false);
	return FEdMode::MouseLeave(ViewportClient, Viewport);
}

bool FEdModeTileMap::MouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 x, int32 y)
{
	if (InViewportClient->EngineShowFlags.ModeWidgets)
	{
		const FViewportCursorLocation Ray = CalculateViewRay(InViewportClient, InViewport);
		UpdatePreviewCursor(Ray);
	}

	// Overridden to prevent the default behavior
	return false;
}

bool FEdModeTileMap::CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY)
{
	if (InViewportClient->EngineShowFlags.ModeWidgets)
	{
		const FViewportCursorLocation Ray = CalculateViewRay(InViewportClient, InViewport);
		
		UpdatePreviewCursor(Ray);

		if (bIsPainting)
		{
			UseActiveToolAtLocation(Ray);
			return true;
		}
		else
		{
			return false;
		}
	}

	return false;
}

bool FEdModeTileMap::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return true;
}

bool FEdModeTileMap::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return true;
}

bool FEdModeTileMap::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent)
{
	bool bHandled = false;

	const bool bIsLeftButtonDown = ( InKey == EKeys::LeftMouseButton && InEvent != IE_Released ) || InViewport->KeyState( EKeys::LeftMouseButton );
	const bool bIsCtrlDown = ( ( InKey == EKeys::LeftControl || InKey == EKeys::RightControl ) && InEvent != IE_Released ) || InViewport->KeyState( EKeys::LeftControl ) || InViewport->KeyState( EKeys::RightControl );
	const bool bIsShiftDown = ( ( InKey == EKeys::LeftShift || InKey == EKeys::RightShift ) && InEvent != IE_Released ) || InViewport->KeyState( EKeys::LeftShift ) || InViewport->KeyState( EKeys::RightShift );

	//@TODO: Don't need to do this always, but any time Shift is pressed or released
	RefreshBrushSize();

	if (InViewportClient->EngineShowFlags.ModeWidgets)
	{
		// Does the user want to paint right now?
		bWasPainting = bIsPainting;
		const bool bUserWantsPaint = bIsLeftButtonDown;
		bool bAnyPaintAbleActorsUnderCursor = false;
		bIsPainting = bUserWantsPaint;

		if (!bWasPainting && bIsPainting)
		{
			// Starting to paint, record if Shift was down which indicates a select instead of the regular tool
			bWasHoldingSelectWhenPaintingStarted = bIsShiftDown;
		}
		else if (bWasPainting && !bIsPainting)
		{
			// Stopping painting
			InViewportClient->Viewport->SetPreCaptureMousePosFromSlateCursor();
		}

		const FViewportCursorLocation Ray = CalculateViewRay(InViewportClient, InViewport);

		UpdatePreviewCursor(Ray);

		if (bIsPainting)
		{
			bHandled = true;
			bAnyPaintAbleActorsUnderCursor = UseActiveToolAtLocation(Ray);
		}
		bWasPainting = bIsPainting;
	}

	if (!bHandled)
	{
		bHandled = FEdMode::InputKey(InViewportClient, InViewport, InKey, InEvent);
	}

	return bHandled;
}

bool FEdModeTileMap::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	return false;
}

void FEdModeTileMap::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);

	//@TODO: Need the force-realtime hack

	// If this viewport does not support Mode widgets we will not draw it here.
	FEditorViewportClient* ViewportClient = (FEditorViewportClient*)Viewport->GetClient();
	if ((ViewportClient != nullptr) && !ViewportClient->EngineShowFlags.ModeWidgets)
	{
		return;
	}

	// Determine if the active tool is in a valid state
	const bool bToolIsReadyToDraw = IsToolReadyToBeUsed();

	// Draw the preview cursor
	if (bIsLastCursorValid)
	{
		if (UPaperTileMap* TileMap = LastCursorTileMap.Get())
		{
			// Slight depth bias so that the wireframe grid overlay doesn't z-fight with the tiles themselves
			const float DepthBias = 0.0001f;
			const FLinearColor CursorWireColor = bToolIsReadyToDraw ? FLinearColor::White : FLinearColor::Red;

			const int32 CursorWidth = GetCursorWidth();
			const int32 CursorHeight = GetCursorHeight();

			FIntRect CursorRange(LastCursorTileX, LastCursorTileY, LastCursorTileX + CursorWidth, LastCursorTileY + CursorHeight);

			if ((GetActiveTool() == ETileMapEditorTool::EyeDropper) && bIsPainting)
			{
				CursorRange = LastEyeDropperBounds;
			}


			TArray<FVector> TilePolygon;
			TilePolygon.Empty(6);
			for (int32 CY = CursorRange.Min.Y; CY < CursorRange.Max.Y; ++CY)
			{
				for (int32 CX = CursorRange.Min.X; CX < CursorRange.Max.X; ++CX)
				{
					TilePolygon.Reset();
					TileMap->GetTilePolygon(CX, CY, LastCursorTileZ, /*out*/ TilePolygon);

					FVector LastPositionWS = ComponentToWorld.TransformPosition(TilePolygon[TilePolygon.Num()-1]);
					for (int32 VertexIndex = 0; VertexIndex < TilePolygon.Num(); ++VertexIndex)
					{
						const FVector ThisPositionWS = ComponentToWorld.TransformPosition(TilePolygon[VertexIndex]);
						PDI->DrawLine(LastPositionWS, ThisPositionWS, CursorWireColor, SDPG_Foreground, 0.0f, DepthBias);
						LastPositionWS = ThisPositionWS;
					}
				}
			}
		}
	}
}

void FEdModeTileMap::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	const FIntRect CanvasRect = Canvas->GetViewRect();

	// Display a help message to exit the editing mode (but only when in the world, not in individual asset editors)
	if (GetModeManager() == &GLevelEditorModeTools())
	{
		static const FText EdModeHelp = LOCTEXT("TileMapEditorModeHelp", "Editing a tile map, press Escape to exit this mode");
		static const FString EdModeHelpAsString = EdModeHelp.ToString();

		int32 XL;
		int32 YL;
		StringSize(GEngine->GetLargeFont(), XL, YL, *EdModeHelpAsString);

		const float DrawX = FMath::FloorToFloat(CanvasRect.Min.X + (CanvasRect.Width() - XL) * 0.5f);
		const float DrawY = 30.0f;
		Canvas->DrawShadowedString(DrawX, DrawY, *EdModeHelpAsString, GEngine->GetLargeFont(), FLinearColor::White);
	}

	bool bDrawToolDescription = false;

	static const FText UnknownTool = LOCTEXT("NoTool", "No tool selected");
	static const FText NoTilesForTool = LOCTEXT("NoInkToolDesc", "No tile selected");

	FText ToolDescription = UnknownTool;
	switch (GetActiveTool())
	{
	case ETileMapEditorTool::Eraser:
		ToolDescription = LOCTEXT("EraserTool", "Erase");
		bDrawToolDescription = true;
		break;
	case ETileMapEditorTool::Paintbrush:
		ToolDescription = bHasValidInkSource ? LOCTEXT("BrushTool", "Paint") : NoTilesForTool;
		bDrawToolDescription = true;
		break;
	case ETileMapEditorTool::PaintBucket:
		ToolDescription = bHasValidInkSource ? LOCTEXT("PaintBucketTool", "Fill") : NoTilesForTool;
		bDrawToolDescription = true;
		break;
	case ETileMapEditorTool::EyeDropper:
		ToolDescription = LOCTEXT("EyeDropperTool", "Select");
		bDrawToolDescription = true;
		break;
	case ETileMapEditorTool::TerrainBrush:
		ToolDescription = LOCTEXT("TerrainTool", "Terrain"); //@TODO: TileMapTerrain: Show the current terrain name?
		bDrawToolDescription = true;
		break;
	}

	if (bDrawToolDescription && !DrawPreviewDimensionsLS.IsNearlyZero())
	{
		const FString ToolDescriptionString = FString::Printf(TEXT("(%d, %d) %s"), LastCursorTileX, LastCursorTileY, *ToolDescription.ToString());

		FVector2D ScreenSpacePreviewLocation;
		if (View->WorldToPixel(DrawPreviewTopLeft, /*out*/ ScreenSpacePreviewLocation))
		{
			const bool bToolIsReadyToDraw = IsToolReadyToBeUsed();
			const FLinearColor ToolPromptColor = bToolIsReadyToDraw ? FLinearColor::White : FLinearColor::Red;

			int32 XL;
			int32 YL;
			StringSize(GEngine->GetLargeFont(), XL, YL, *ToolDescriptionString);
			const float DrawX = FMath::FloorToFloat(ScreenSpacePreviewLocation.X);
			const float DrawY = FMath::FloorToFloat(ScreenSpacePreviewLocation.Y - YL);
			Canvas->DrawShadowedString(DrawX, DrawY, *ToolDescriptionString, GEngine->GetLargeFont(), ToolPromptColor);
		}
	}


	// Draw the 'status tray' information
	if (UPaperTileMap* LastMap = LastCursorTileMap.Get())
	{
		if (bIsLastCursorValid && LastMap->TileLayers.IsValidIndex(LastCursorTileZ))
		{
			UPaperTileLayer* LastLayer = LastMap->TileLayers[LastCursorTileZ];

			FNumberFormattingOptions NoCommas;
			NoCommas.UseGrouping = false;

			const FPaperTileInfo Cell = LastLayer->GetCell(LastCursorTileX, LastCursorTileY);
			const bool bInBounds = LastLayer->InBounds(LastCursorTileX, LastCursorTileY);

			FText TileIndexDescription;
			if (!bInBounds)
			{
				TileIndexDescription = LOCTEXT("OutOfBoundsCell", "(outside map)");
			}
			else if (Cell.IsValid())
			{
				TileIndexDescription = FText::AsCultureInvariant(FString::Printf(TEXT("%s #%d %c%c%c"),
					*Cell.TileSet->GetName(),
					Cell.GetTileIndex(),
					Cell.HasFlag(EPaperTileFlags::FlipHorizontal) ? TEXT('H') : TEXT('_'),
					Cell.HasFlag(EPaperTileFlags::FlipVertical) ? TEXT('V') : TEXT('_'),
					Cell.HasFlag(EPaperTileFlags::FlipDiagonal) ? TEXT('D') : TEXT('_')));
			}
			else
			{
				TileIndexDescription = LOCTEXT("EmptyCell", "(empty)");
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("X"), FText::AsNumber(LastCursorTileX, &NoCommas));
			Args.Add(TEXT("Y"), FText::AsNumber(LastCursorTileY, &NoCommas));
			Args.Add(TEXT("TileIndex"), TileIndexDescription);
			Args.Add(TEXT("LayerName"), LastLayer->LayerName);

			const static FText FormatString = LOCTEXT("TileCursorStatusMessage", "({X}, {Y}) [{TileIndex}]   Current Layer: {LayerName}");
			const FText CursorDescriptionText = FText::Format(FormatString, Args);
			const FString CursorDescriptionString = CursorDescriptionText.ToString();

			int32 XL;
			int32 YL;
			StringSize(GEngine->GetLargeFont(), XL, YL, *CursorDescriptionString);

			const float DrawX = FMath::FloorToFloat(CanvasRect.Min.X + (CanvasRect.Width() - XL) * 0.5f);
			const float DrawY = FMath::FloorToFloat(CanvasRect.Max.Y - 10.0f - YL);
			Canvas->DrawShadowedString(DrawX, DrawY, *CursorDescriptionString, GEngine->GetLargeFont(), FLinearColor::White);
		}
	}
}

bool FEdModeTileMap::AllowWidgetMove()
{
	return false;
}

bool FEdModeTileMap::ShouldDrawWidget() const
{
	return false;
}

bool FEdModeTileMap::UsesTransformWidget() const
{
	return false;
}

UPaperTileMapComponent* FEdModeTileMap::FindSelectedComponent() const
{
	UPaperTileMapComponent* TileMapComponent = nullptr;

	USelection* SelectedActors = Owner->GetSelectedActors();
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = CastChecked<AActor>(*Iter);

		TileMapComponent = Actor->FindComponentByClass<UPaperTileMapComponent>();
		if (TileMapComponent != nullptr)
		{
			break;
		}
	}

	if (TileMapComponent == nullptr)
	{
		USelection* SelectedObjects = Owner->GetSelectedObjects();
		for (FSelectionIterator Iter(*SelectedObjects); Iter; ++Iter)
		{
			UObject* Foo = *Iter;
			TileMapComponent = Cast<UPaperTileMapComponent>(Foo);

			if (TileMapComponent != nullptr)
			{
				break;
			}
		}
	}

	return TileMapComponent;
}

UPaperTileLayer* FEdModeTileMap::GetSourceInkLayer() const
{
	return CursorPreviewComponent->TileMap->TileLayers[0];
}

UPaperTileLayer* FEdModeTileMap::GetSelectedLayerUnderCursor(const FViewportCursorLocation& Ray, int32& OutTileX, int32& OutTileY, bool bAllowOutOfBounds) const
{
	const FVector TraceStart = Ray.GetOrigin();
	const FVector TraceDir = Ray.GetDirection();
	const int32 BrushWidth = GetBrushWidth();
	const int32 BrushHeight = GetBrushHeight();

	UPaperTileMapComponent* TileMapComponent = FindSelectedComponent();

	if (TileMapComponent != nullptr)
	{
		if (UPaperTileMap* TileMap = TileMapComponent->TileMap)
		{
			// Find the selected layer
			int32 LayerIndex = TileMap->SelectedLayerIndex;

			// If there was a selected layer, pick it
			if (TileMap->TileLayers.IsValidIndex(LayerIndex))
			{
				UPaperTileLayer* Layer = TileMap->TileLayers[LayerIndex];

				ComponentToWorld = TileMapComponent->GetComponentTransform();
				const FVector LocalStart = ComponentToWorld.InverseTransformPosition(TraceStart);
				const FVector LocalDirection = ComponentToWorld.InverseTransformVector(TraceDir);
				const FVector LocalEnd = LocalStart + (LocalDirection * HALF_WORLD_MAX);
					 
				const FVector LSPlaneCorner = PaperAxisZ * TileMap->SeparationPerLayer;

				const FPlane LayerPlane(LSPlaneCorner + PaperAxisX, LSPlaneCorner, LSPlaneCorner + PaperAxisY);

				FVector Intersection;
				if (FMath::SegmentPlaneIntersection(LocalStart, LocalEnd, LayerPlane, /*out*/ Intersection))
				{
					TileMap->GetTileCoordinatesFromLocalSpacePosition(Intersection, /*out*/ OutTileX, /*out*/ OutTileY);

					const bool bInBounds = (OutTileX > -BrushWidth) && (OutTileX < TileMap->MapWidth) && (OutTileY > -BrushHeight) && (OutTileY < TileMap->MapHeight);
					if (bInBounds || bAllowOutOfBounds)
					{
						return Layer;
					}
				}
			}
		}
	}

	OutTileX = 0;
	OutTileY = 0;
	return nullptr;
}


bool FEdModeTileMap::UseActiveToolAtLocation(const FViewportCursorLocation& Ray)
{
	switch (GetActiveTool())
	{
	case ETileMapEditorTool::EyeDropper:
		return SelectTiles(Ray);
	case ETileMapEditorTool::Paintbrush:
		return PaintTiles(Ray);
	case ETileMapEditorTool::Eraser:
		return EraseTiles(Ray);
	case ETileMapEditorTool::PaintBucket:
		return FloodFillTiles(Ray);
	case ETileMapEditorTool::TerrainBrush:
		return PaintTilesWithTerrain(Ray);
	default:
		check(false);
		return false;
	};
}

bool FEdModeTileMap::BlitLayer(UPaperTileLayer* SourceLayer, UPaperTileLayer* TargetLayer, FBox& OutDirtyRect, int32 OffsetX, int32 OffsetY, bool bBlitEmptyTiles)
{
	FScopedTransaction Transaction(LOCTEXT("TileMapPaintActionTransaction", "Tile Painting"));

	const int32 LayerCoord = TargetLayer->GetLayerIndex();

	bool bPaintedOnSomething = false;
	bool bChangedSomething = false;

	for (int32 SourceY = 0; SourceY < SourceLayer->GetLayerHeight(); ++SourceY)
	{
		const int32 TargetY = OffsetY + SourceY;

		if ((TargetY < 0) || (TargetY >= TargetLayer->GetLayerHeight()))
		{
			continue;
		}

		for (int32 SourceX = 0; SourceX < SourceLayer->GetLayerWidth(); ++SourceX)
		{
			const int32 TargetX = OffsetX + SourceX;

			if ((TargetX < 0) || (TargetX >= TargetLayer->GetLayerWidth()))
			{
				continue;
			}

			const FPaperTileInfo Ink = SourceLayer->GetCell(SourceX, SourceY);

			if ((Ink.IsValid() || bBlitEmptyTiles) && (TargetLayer->GetCell(TargetX, TargetY) != Ink))
			{
				if (!bChangedSomething)
				{
					TargetLayer->SetFlags(RF_Transactional);
					TargetLayer->Modify();
					bChangedSomething = true;
				}

				OutDirtyRect += FVector(TargetX, TargetY, LayerCoord);
				TargetLayer->SetCell(TargetX, TargetY, Ink);
			}

			bPaintedOnSomething = true;
		}
	}

	if (bChangedSomething)
	{
		TargetLayer->GetTileMap()->PostEditChange();
	}

	if (!bChangedSomething)
	{
		Transaction.Cancel();
	}

	return bPaintedOnSomething;
}

bool FEdModeTileMap::SelectTiles(const FViewportCursorLocation& Ray)
{
	bool bPaintedOnSomething = false;

	int32 DestTileX;
	int32 DestTileY;

	if (UPaperTileLayer* TargetLayer = GetSelectedLayerUnderCursor(Ray, /*out*/ DestTileX, /*out*/ DestTileY))
	{
		FIntPoint EyeDropperEnd = FIntPoint(DestTileX, DestTileY);
		if (!bWasPainting)
		{
			EyeDropperStart = EyeDropperEnd;
		}

		FIntRect SelectionBounds(EyeDropperStart, EyeDropperStart);
		SelectionBounds.Include(EyeDropperEnd);
		SelectionBounds.Max.X += 1;
		SelectionBounds.Max.Y += 1;

		if (!bWasPainting || (SelectionBounds != LastEyeDropperBounds))
		{
			SetActivePaintFromLayer(TargetLayer, SelectionBounds.Min, SelectionBounds.Size());
		}

		LastEyeDropperBounds = SelectionBounds;
		bPaintedOnSomething = true;
	}

	return bPaintedOnSomething;
}

bool FEdModeTileMap::PaintTiles(const FViewportCursorLocation& Ray)
{
	bool bPaintedOnSomething = false;

	// If we are using an ink source, validate that it exists
	if (!HasValidSelection())
	{
		return false;
	}

	int32 DestTileX;
	int32 DestTileY;

	if (UPaperTileLayer* TargetLayer = GetSelectedLayerUnderCursor(Ray, /*out*/ DestTileX, /*out*/ DestTileY))
	{
		FBox DirtyRect(ForceInitToZero);
		bPaintedOnSomething = BlitLayer(GetSourceInkLayer(), TargetLayer, /*out*/ DirtyRect, DestTileX, DestTileY);

		if (DirtyRect.IsValid)
		{
			new (PendingDirtyRegions) FTileMapDirtyRegion(FindSelectedComponent(), DirtyRect);
		}
	}

	return bPaintedOnSomething;
}

bool FEdModeTileMap::EraseTiles(const FViewportCursorLocation& Ray)
{
	bool bPaintedOnSomething = false;
	bool bChangedSomething = false;
	FBox DirtyRect(ForceInitToZero);

	const int32 BrushWidth = GetBrushWidth();
	const int32 BrushHeight = GetBrushHeight();

	const FPaperTileInfo EmptyCellValue;

	int32 DestTileX;
	int32 DestTileY;

	if (UPaperTileLayer* Layer = GetSelectedLayerUnderCursor(Ray, /*out*/ DestTileX, /*out*/ DestTileY))
	{
		UPaperTileMap* TileMap = Layer->GetTileMap();
		const int32 LayerCoord = Layer->GetLayerIndex();

		FScopedTransaction Transaction( LOCTEXT("TileMapEraseActionTransaction", "Tile Erasing") );

		for (int32 Y = 0; Y < BrushWidth; ++Y)
		{
			const int DY = DestTileY + Y;

			if ((DY < 0) || (DY >= TileMap->MapHeight))
			{
				continue;
			}

			for (int32 X = 0; X < BrushHeight; ++X)
			{
				const int DX = DestTileX + X;

				if ((DX < 0) || (DX >= TileMap->MapWidth))
				{
					continue;
				}

				if (Layer->GetCell(DX, DY).IsValid())
				{
					if (!bChangedSomething)
					{
						Layer->SetFlags(RF_Transactional);
						Layer->Modify();
						bChangedSomething = true;
					}
					Layer->SetCell(DX, DY, EmptyCellValue);
					DirtyRect += FVector(DX, DY, LayerCoord);
				}

				bPaintedOnSomething = true;
			}
		}

		if (bChangedSomething)
		{
			if (DirtyRect.IsValid)
			{
				UPaperTileMapComponent* TileMapComponent = FindSelectedComponent();
				new (PendingDirtyRegions) FTileMapDirtyRegion(TileMapComponent, DirtyRect);
			}

			TileMap->PostEditChange();
		}

		if (!bChangedSomething)
		{
			Transaction.Cancel();
		}
	}

	return bPaintedOnSomething;
}

bool FEdModeTileMap::FloodFillTiles(const FViewportCursorLocation& Ray)
{
	bool bPaintedOnSomething = false;
	bool bChangedSomething = false;

	// Validate that the tool we're using can be used right now
	if (!HasValidSelection())
	{
		return false;
	}

	int DestTileX;
	int DestTileY;

	if (UPaperTileLayer* TargetLayer = GetSelectedLayerUnderCursor(Ray, /*out*/ DestTileX, /*out*/ DestTileY))
	{
		//@TODO: Should we allow off-canvas flood filling too?
		if ((DestTileX < 0) || (DestTileY < 0))
		{
			return false;
		}

		FBox DirtyRect(ForceInitToZero);

		// The kind of ink we'll replace, starting at the seed point
		const FPaperTileInfo RequiredInk = TargetLayer->GetCell(DestTileX, DestTileY);
		const int32 LayerIndex = TargetLayer->GetLayerIndex();

		UPaperTileMap* TileMap = TargetLayer->GetTileMap();

		//@TODO: Unoptimized first-pass approach
		const int32 NumTiles = TileMap->MapWidth * TileMap->MapHeight;

		// Flag for all tiles indicating if they are reachable from the seed paint point
		TBitArray<> TileReachability;
		TileReachability.Init(false, NumTiles);

		// List of horizontal spans that still need to be checked for adjacent colors above and below
		TArray<FHorizontalSpan> OutstandingSpans;

		// Start off at the seed point
		FHorizontalSpan& InitialSpan = *(new (OutstandingSpans) FHorizontalSpan(DestTileX, DestTileY));
		InitialSpan.GrowSpan(RequiredInk, TargetLayer, TileReachability);

		// Process the list of outstanding spans until it is empty
		while (OutstandingSpans.Num())
		{
			FHorizontalSpan Span = OutstandingSpans.Pop(EAllowShrinking::No);

			// Create spans below and above
			for (int32 DY = -1; DY <= 1; DY += 2)
			{
				const int32 Y = Span.Y + DY;
				if ((Y < 0) || (Y >= TargetLayer->GetLayerHeight()))
				{
					continue;
				}
					
				for (int32 X = Span.X0; X <= Span.X1; ++X)
				{
					// If it is the right color and not already visited, create a span there
					FPaperTileInfo ExistingCell = TargetLayer->GetCell(X, Y);
					const bool bCellMatches = (ExistingCell == RequiredInk) || (!ExistingCell.IsValid() && !RequiredInk.IsValid());

					if (bCellMatches && !FHorizontalSpan::Reach(TargetLayer, TileReachability, X, Y))
					{
						FHorizontalSpan& NewSpan = *(new (OutstandingSpans) FHorizontalSpan(X, Y));
						NewSpan.GrowSpan(RequiredInk, TargetLayer, TileReachability);
					}
				}
			}
		}
		
		// Now the reachability map should be populated, so we can use it to flood fill
		FScopedTransaction Transaction( LOCTEXT("TileMapFloodFillActionTransaction", "Tile Paint Bucket") );

		// Figure out where the top left square of the map starts in the pattern, based on the seed point
		UPaperTileLayer* SourceLayer = GetSourceInkLayer();
		const int32 BrushWidth = SourceLayer->GetLayerWidth();
		const int32 BrushHeight = SourceLayer->GetLayerHeight();

		int32 BrushPatternOffsetX = (BrushWidth - ((DestTileX + BrushWidth) % BrushWidth));
		int32 BrushPatternOffsetY = (BrushHeight - ((DestTileY + BrushHeight) % BrushHeight));
		int32 ReachIndex = 0;
		for (int32 DY = 0; DY < TileMap->MapHeight; ++DY)
		{
			const int32 InsideBrushY = (DY + BrushPatternOffsetY) % BrushHeight;
	
			for (int32 DX = 0; DX < TileMap->MapWidth; ++DX)
			{
				if (TileReachability[ReachIndex++])
				{
					const int32 InsideBrushX = (DX + BrushPatternOffsetX) % BrushWidth;

					FPaperTileInfo NewInk = SourceLayer->GetCell(InsideBrushX, InsideBrushY);

					if (TargetLayer->GetCell(DX, DY) != NewInk)
					{
						if (!bChangedSomething)
						{
							TargetLayer->SetFlags(RF_Transactional);
							TargetLayer->Modify();
							bChangedSomething = true;
						}

						DirtyRect += FVector(DX, DY, LayerIndex);

						TargetLayer->SetCell(DX, DY, NewInk);
					}

					bPaintedOnSomething = true;
				}
			}
		}

		if (bChangedSomething)
		{
			if (DirtyRect.IsValid)
			{
				new (PendingDirtyRegions) FTileMapDirtyRegion(FindSelectedComponent(), DirtyRect);
			}

			TileMap->PostEditChange();
		}

		if (!bChangedSomething)
		{
			Transaction.Cancel();
		}
	}

	return bPaintedOnSomething;
}

bool FEdModeTileMap::PaintTilesWithTerrain(const FViewportCursorLocation& Ray)
{
	bool bPaintedOnSomething = false;
	bool bChangedSomething = false;

	// Validate that the tool we're using can be used right now
	if (!HasValidSelection())
	{
		return false;
	}

	int DestTileX;
	int DestTileY;

	if (UPaperTileLayer* TargetLayer = GetSelectedLayerUnderCursor(Ray, /*out*/ DestTileX, /*out*/ DestTileY))
	{
		UPaperTileMap* TileMap = TargetLayer->GetTileMap();
		const int32 LayerIndex = TargetLayer->GetLayerIndex();

		FBox DirtyRect(ForceInitToZero);

		if ((DestTileX >= 0) && (DestTileY >= 0) && (DestTileX < TileMap->MapWidth) & (DestTileY < TileMap->MapHeight))
		{
			FScopedTransaction Transaction(LOCTEXT("TileMapTerrainBrushActionTransaction", "Terrain Brush"));

			for (int32 OY = -1; OY <= 1; ++OY)
			{
				for (int32 OX = -1; OX <= 1; ++OX)
				{
					const int32 DX = DestTileX + OX;
					const int32 DY = DestTileY + OY;
					FPaperTileInfo PreviousTileInfo = TargetLayer->GetCell(DX, DY);

					//@TODO: TileMapTerrain: Implement this
					FPaperTileInfo NewInk = PreviousTileInfo;

					if (PreviousTileInfo != NewInk)
					{
						if (!bChangedSomething)
						{
							TargetLayer->SetFlags(RF_Transactional);
							TargetLayer->Modify();
							bChangedSomething = true;
						}

						DirtyRect += FVector(DX, DY, LayerIndex);

						TargetLayer->SetCell(DX, DY, NewInk);
					}
				}
			}

			if (bChangedSomething)
			{
				if (DirtyRect.IsValid)
				{
					new (PendingDirtyRegions) FTileMapDirtyRegion(FindSelectedComponent(), DirtyRect);
				}

				TileMap->PostEditChange();
			}

			if (!bChangedSomething)
			{
				Transaction.Cancel();
			}
		}
	}

	return bPaintedOnSomething;
}

void FEdModeTileMap::DestructiveResizePreviewComponent(int32 NewWidth, int32 NewHeight)
{
	UPaperTileMap* PreviewMap = CursorPreviewComponent->TileMap;
	PreviewMap->MapWidth = FMath::Max<int32>(NewWidth, 1);
	PreviewMap->MapHeight = FMath::Max<int32>(NewHeight, 1);
	FPropertyChangedEvent EditedMapSizeEvent(UPaperTileMap::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPaperTileMap, MapWidth)));
	PreviewMap->PostEditChangeProperty(EditedMapSizeEvent);

	CursorPreviewComponent->MarkRenderStateDirty();
}

void FEdModeTileMap::SetActivePaint(UPaperTileSet* TileSet, FIntPoint TopLeft, FIntPoint Dimensions)
{
	if ((TileSet == nullptr) || (Dimensions.X == 0) || (Dimensions.Y == 0))
	{
		bHasValidInkSource = false;
	}
	else
	{
		bHasValidInkSource = true;
	}

	FIntPoint Scale(1, 1);
	if (TileSet != nullptr)
	{
		Scale.X = FMath::Max(1, TileSet->GetTileSize().X / CursorPreviewComponent->TileMap->TileWidth);
		Scale.Y = FMath::Max(1, TileSet->GetTileSize().Y / CursorPreviewComponent->TileMap->TileHeight);
	}

	DestructiveResizePreviewComponent(Dimensions.X * Scale.X, Dimensions.Y * Scale.Y);

	UPaperTileMap* PreviewMap = CursorPreviewComponent->TileMap;
	UPaperTileLayer* PreviewLayer = GetSourceInkLayer();

	// Empty the preview layer (when selecting tiles from a tile set with a larger tile size than the map, there will be gaps in what is setup below)
	FPaperTileInfo EmptyCell;
	for (int32 Y = 0; Y < PreviewLayer->GetLayerHeight(); ++Y)
	{
		for (int32 X = 0; X < PreviewLayer->GetLayerWidth(); ++X)
		{
			PreviewLayer->SetCell(X, Y, EmptyCell);
		}
	}

	// Copy the selection into the preview ink layer
	for (int32 Y = 0; Y < FMath::Max(1, Dimensions.Y); ++Y)
	{
		for (int32 X = 0; X < FMath::Max(1, Dimensions.X); ++X)
		{
			FPaperTileInfo TileInfo;

			const int32 SourceX = X + TopLeft.X;
			const int32 SourceY = Y + TopLeft.Y;

			if ((TileSet != nullptr) && (SourceX < TileSet->GetTileCountX()) && (SourceY < TileSet->GetTileCountY()))
			{
				TileInfo.PackedTileIndex = SourceX + (SourceY * TileSet->GetTileCountX());
				TileInfo.TileSet = TileSet;
			}

			PreviewLayer->SetCell(X * Scale.X, Y * Scale.Y + Scale.Y - 1, TileInfo);
		}
	}

	CursorPreviewComponent->MarkRenderStateDirty();

	RefreshBrushSize();
}

void FEdModeTileMap::SetActivePaintFromLayer(UPaperTileLayer* SourceLayer, FIntPoint TopLeft, FIntPoint Dimensions)
{
	if ((Dimensions.X == 0) || (Dimensions.Y == 0))
	{
		bHasValidInkSource = false;
	}
	else
	{
		bHasValidInkSource = true;
	}

	DestructiveResizePreviewComponent(Dimensions.X, Dimensions.Y);

	UPaperTileMap* PreviewMap = CursorPreviewComponent->TileMap;
	UPaperTileLayer* PreviewLayer = GetSourceInkLayer();
	for (int32 Y = 0; Y < PreviewMap->MapHeight; ++Y)
	{
		for (int32 X = 0; X < PreviewMap->MapWidth; ++X)
		{
			const int32 SourceX = X + TopLeft.X;
			const int32 SourceY = Y + TopLeft.Y;

			FPaperTileInfo TileInfo = SourceLayer->GetCell(SourceX, SourceY);

			PreviewLayer->SetCell(X, Y, TileInfo);
		}
	}

	CursorPreviewComponent->MarkRenderStateDirty();

	RefreshBrushSize();
}

void FEdModeTileMap::FlipSelectionHorizontally()
{
	UPaperTileMap* PreviewMap = CursorPreviewComponent->TileMap;
	UPaperTileLayer* PreviewLayer = GetSourceInkLayer();
	for (int32 Y = 0; Y < PreviewMap->MapHeight; ++Y)
	{
		// Flip the tiles within individual cells
		for (int32 X = 0; X < PreviewMap->MapWidth; ++X)
		{
			FPaperTileInfo Cell = PreviewLayer->GetCell(X, Y);
			if (Cell.IsValid())
			{
				Cell.ToggleFlag(EPaperTileFlags::FlipHorizontal);
			}
			PreviewLayer->SetCell(X, Y, Cell);
		}

		// Flip the selection as a whole
		for (int32 X = 0; X < PreviewMap->MapWidth / 2; ++X)
		{
			const int32 MirrorX = PreviewMap->MapWidth - 1 - X;
			FPaperTileInfo LeftCell = PreviewLayer->GetCell(X, Y);
			FPaperTileInfo RightCell = PreviewLayer->GetCell(MirrorX, Y);
			PreviewLayer->SetCell(X, Y, RightCell);
			PreviewLayer->SetCell(MirrorX, Y, LeftCell);
		}
	}

	CursorPreviewComponent->MarkRenderStateDirty();
}

void FEdModeTileMap::FlipSelectionVertically()
{
	UPaperTileMap* PreviewMap = CursorPreviewComponent->TileMap;
	UPaperTileLayer* PreviewLayer = GetSourceInkLayer();
	for (int32 X = 0; X < PreviewMap->MapWidth; ++X)
	{
		// Flip the tiles within individual cells
		for (int32 Y = 0; Y < PreviewMap->MapHeight; ++Y)
		{
			FPaperTileInfo Cell = PreviewLayer->GetCell(X, Y);
			if (Cell.IsValid())
			{
				Cell.ToggleFlag(EPaperTileFlags::FlipVertical);
			}
			PreviewLayer->SetCell(X, Y, Cell);
		}

		// Flip the selection as a whole
		for (int32 Y = 0; Y < PreviewMap->MapHeight / 2; ++Y)
		{
			const int32 MirrorY = PreviewMap->MapHeight - 1 - Y;
			FPaperTileInfo TopCell = PreviewLayer->GetCell(X, Y);
			FPaperTileInfo BottomCell = PreviewLayer->GetCell(X, MirrorY);
			PreviewLayer->SetCell(X, Y, BottomCell);
			PreviewLayer->SetCell(X, MirrorY, TopCell);
		}
	}

	CursorPreviewComponent->MarkRenderStateDirty();
}

void FEdModeTileMap::RotateTilesInSelection(bool bIsClockwise)
{
	UPaperTileLayer* PreviewLayer = GetSourceInkLayer();

	static const uint8 ClockwiseRotationMap[8] = { 5, 4, 1, 0, 7, 6, 3, 2 };
	static const uint8 CounterclockwiseRotationMap[8] = { 3, 2, 7, 6, 1, 0, 5, 4 };
	const uint8* RotationTable = bIsClockwise ? ClockwiseRotationMap : CounterclockwiseRotationMap;

	const int32 OldWidth = PreviewLayer->GetLayerWidth();
	const int32 OldHeight = PreviewLayer->GetLayerHeight();

	// Copy off the tiles and rotate within each tile
	TArray<FPaperTileInfo> OldTiles;
	OldTiles.Empty(PreviewLayer->GetLayerWidth() * PreviewLayer->GetLayerHeight());
	for (int32 Y = 0; Y < PreviewLayer->GetLayerHeight(); ++Y)
	{
		for (int32 X = 0; X < PreviewLayer->GetLayerWidth(); ++X)
		{
			FPaperTileInfo Cell = PreviewLayer->GetCell(X, Y);
			if (Cell.IsValid())
			{
				uint8 NewFlags = RotationTable[Cell.GetFlagsAsIndex()];
				Cell.SetFlagsAsIndex(NewFlags);
			}
			OldTiles.Add(Cell);
		}
	}

	// Resize, transposing width and height
	DestructiveResizePreviewComponent(PreviewLayer->GetLayerHeight(), PreviewLayer->GetLayerWidth());

	// Place the tiles back in the rotated layout
	for (int32 NewY = 0; NewY < PreviewLayer->GetLayerHeight(); ++NewY)
	{
		for (int32 NewX = 0; NewX < PreviewLayer->GetLayerWidth(); ++NewX)
		{
			const int32 OldX = bIsClockwise ? NewY : (OldWidth - 1 - NewY);
			const int32 OldY = bIsClockwise ? (OldHeight - 1 - NewX) : NewX;

			FPaperTileInfo Cell = OldTiles[OldY*OldWidth + OldX];

			PreviewLayer->SetCell(NewX, NewY, Cell);
		}
	}
}

bool FEdModeTileMap::IsToolReadyToBeUsed() const
{
	bool bToolIsReadyToDraw = false;
	switch (GetActiveTool())
	{
	case ETileMapEditorTool::EyeDropper:
		bToolIsReadyToDraw = true;
		break;
	case ETileMapEditorTool::Paintbrush:
		bToolIsReadyToDraw = bHasValidInkSource;
		break;
	case ETileMapEditorTool::Eraser:
		bToolIsReadyToDraw = true;
		break;
	case ETileMapEditorTool::PaintBucket:
		bToolIsReadyToDraw = bHasValidInkSource;
		break;
	case ETileMapEditorTool::TerrainBrush:
		bToolIsReadyToDraw = bHasValidInkSource; //@TODO: TileMapTerrain: What to do here...
		break;
	default:
		check(false);
		break;
	}

	return bToolIsReadyToDraw;
}

void FEdModeTileMap::RotateSelectionCW()
{
	RotateTilesInSelection(/*bIsClockwise=*/ true);
}

void FEdModeTileMap::RotateSelectionCCW()
{
	RotateTilesInSelection(/*bIsClockwise=*/ false);
}

bool FEdModeTileMap::HasValidSelection() const
{
	UPaperTileLayer* PreviewLayer = GetSourceInkLayer();
	return (PreviewLayer->GetLayerWidth() > 0) && (PreviewLayer->GetLayerHeight() > 0) && bHasValidInkSource;
}

void FEdModeTileMap::SynchronizePreviewWithTileMap(UPaperTileMap* NewTileMap)
{
	UPaperTileMap* PreviewTileMap = CursorPreviewComponent->TileMap;

	bool bPreviewComponentDirty = false;

#define UE_CHANGE_IF_DIFFERENT(PropertyName) \
	if (PreviewTileMap->PropertyName != NewTileMap->PropertyName) \
	{ \
		PreviewTileMap->PropertyName = NewTileMap->PropertyName; \
		bPreviewComponentDirty = true; \
	}

	UE_CHANGE_IF_DIFFERENT(TileWidth);
	UE_CHANGE_IF_DIFFERENT(TileHeight);
	UE_CHANGE_IF_DIFFERENT(PixelsPerUnrealUnit);
	UE_CHANGE_IF_DIFFERENT(SeparationPerTileX);
	UE_CHANGE_IF_DIFFERENT(SeparationPerTileY);
	UE_CHANGE_IF_DIFFERENT(SeparationPerLayer);
	UE_CHANGE_IF_DIFFERENT(Material);
	UE_CHANGE_IF_DIFFERENT(ProjectionMode);

#undef UE_CHANGE_IF_DIFFERENT

	if (bPreviewComponentDirty)
	{
		CursorPreviewComponent->MarkRenderStateDirty();
	}
}

void FEdModeTileMap::UpdatePreviewCursor(const FViewportCursorLocation& Ray)
{
	DrawPreviewDimensionsLS = FVector::ZeroVector;
	bIsLastCursorValid = false;
	LastCursorTileMap.Reset();

	// See if we should draw the preview
	{
		int32 LocalTileX0;
		int32 LocalTileY0;
		if (UPaperTileLayer* TileLayer = GetSelectedLayerUnderCursor(Ray, /*out*/ LocalTileX0, /*out*/ LocalTileY0, /*bAllowOutOfBounds=*/ true))
		{
			UPaperTileMap* TileMap = TileLayer->GetTileMap();
			int32 LayerIndex;
			ensure(TileMap->TileLayers.Find(TileLayer, LayerIndex));

			LastCursorTileX = LocalTileX0;
			LastCursorTileY = LocalTileY0;
			LastCursorTileZ = LayerIndex;
			bIsLastCursorValid = true;
			LastCursorTileMap = TileMap;

			const int32 CursorWidth = GetCursorWidth();
			const int32 CursorHeight = GetCursorHeight();

			const int32 LocalTileX1 = LocalTileX0 + CursorWidth;
			const int32 LocalTileY1 = LocalTileY0 + CursorHeight;

			DrawPreviewTopLeft = ComponentToWorld.TransformPosition(TileMap->GetTilePositionInLocalSpace(LocalTileX0, LocalTileY0, LayerIndex));
			const FVector WorldPosition = DrawPreviewTopLeft;
			const FVector WorldPositionBR = ComponentToWorld.TransformPosition(TileMap->GetTilePositionInLocalSpace(LocalTileX1, LocalTileY1, LayerIndex));

			DrawPreviewSpace = ComponentToWorld;
			DrawPreviewLocation = (WorldPosition + WorldPositionBR) * 0.5f;

			DrawPreviewDimensionsLS = 0.5f*((PaperAxisX * CursorWidth * TileMap->TileWidth) + (PaperAxisY * -CursorHeight * TileMap->TileHeight));

			// Figure out how far to nudge out the tile map (we want a decent size (especially if the layer separation is small), but should never be a full layer out)
			const float AbsoluteSeparation = FMath::Abs(TileMap->SeparationPerLayer);
			const float DepthBiasNudge = -FMath::Min(FMath::Max(1.0f, AbsoluteSeparation * 0.05f), AbsoluteSeparation * 0.5f);

			const FVector ComponentPreviewLocationNoNudge = ComponentToWorld.TransformPosition(TileMap->GetTileCenterInLocalSpace(LocalTileX0, LocalTileY0, LayerIndex));
			const FVector ComponentPreviewLocation = ComponentPreviewLocationNoNudge + (PaperAxisZ * DepthBiasNudge);

			CursorPreviewComponent->SetWorldLocation(ComponentPreviewLocation);
			CursorPreviewComponent->SetWorldRotation(FRotator(ComponentToWorld.GetRotation()));
			CursorPreviewComponent->SetWorldScale3D(ComponentToWorld.GetScale3D());
			SynchronizePreviewWithTileMap(TileMap);
		}
	}
}

FViewportCursorLocation FEdModeTileMap::CalculateViewRay(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( 
		InViewportClient->Viewport, 
		InViewportClient->GetScene(),
		InViewportClient->EngineShowFlags )
		.SetRealtimeUpdate( InViewportClient->IsRealtime() ));

	FSceneView* View = InViewportClient->CalcSceneView( &ViewFamily );
	FViewportCursorLocation MouseViewportRay( View, InViewportClient, InViewport->GetMouseX(), InViewport->GetMouseY() );

	return MouseViewportRay;
}

void FEdModeTileMap::SetActiveTool(ETileMapEditorTool::Type NewTool)
{
	ActiveTool = NewTool;
	RefreshBrushSize();
}

ETileMapEditorTool::Type FEdModeTileMap::GetActiveTool() const
{
	// Force the eyedropper active when Shift is held (or if it was held when painting started, even if it was released later)
	const bool bHoldingShift = !bIsPainting && FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	const bool bWasHoldingShift = bIsPainting && bWasHoldingSelectWhenPaintingStarted;
	
	return (bHoldingShift || bWasHoldingShift) ? ETileMapEditorTool::EyeDropper : ActiveTool;
}

int32 FEdModeTileMap::GetBrushWidth() const
{
	int32 BrushWidth = 1;

	switch (GetActiveTool())
	{
	case ETileMapEditorTool::EyeDropper:
		BrushWidth = FMath::Max<int32>(LastEyeDropperBounds.Width(), 1);
		break;
	case ETileMapEditorTool::Paintbrush:
		BrushWidth = GetSourceInkLayer()->GetLayerWidth();
		break;
	case ETileMapEditorTool::Eraser:
		BrushWidth = EraseBrushSize;
		break;
	case ETileMapEditorTool::PaintBucket:
		BrushWidth = GetSourceInkLayer()->GetLayerWidth();
		break;
	case ETileMapEditorTool::TerrainBrush:
		BrushWidth = 1;
		break;
	default:
		check(false);
		break;
	}

	return BrushWidth;
}

int32 FEdModeTileMap::GetBrushHeight() const
{
	int32 BrushHeight = 1;
	
	switch (GetActiveTool())
	{
	case ETileMapEditorTool::EyeDropper:
		BrushHeight = FMath::Max<int32>(LastEyeDropperBounds.Height(), 1);
		break;
	case ETileMapEditorTool::Paintbrush:
		BrushHeight = GetSourceInkLayer()->GetLayerHeight();
		break;
	case ETileMapEditorTool::Eraser:
		BrushHeight = EraseBrushSize;
		break;
	case ETileMapEditorTool::PaintBucket:
		BrushHeight = GetSourceInkLayer()->GetLayerHeight();
		break;
	case ETileMapEditorTool::TerrainBrush:
		BrushHeight = 1;
		break;
	default:
		check(false);
		break;
	}

	return BrushHeight;
}

int32 FEdModeTileMap::GetCursorWidth() const
{
	const int32 CursorWidth = ((GetActiveTool() == ETileMapEditorTool::PaintBucket) || !bHasValidInkSource) ? 1 : GetBrushWidth();
	return CursorWidth;
}

int32 FEdModeTileMap::GetCursorHeight() const
{
	const int32 CursorHeight = ((GetActiveTool() == ETileMapEditorTool::PaintBucket) || !bHasValidInkSource) ? 1 : GetBrushHeight();
	return CursorHeight;
}

void FEdModeTileMap::RefreshBrushSize()
{
	const bool bShowPreviewDesired = !DrawPreviewDimensionsLS.IsNearlyZero();

	switch (GetActiveTool())
	{
	case ETileMapEditorTool::EyeDropper:
		CursorPreviewComponent->SetVisibility(!bIsPainting);
		break;
	case ETileMapEditorTool::Paintbrush:
		CursorPreviewComponent->SetVisibility(bShowPreviewDesired);
		break;
	case ETileMapEditorTool::Eraser:
		CursorPreviewComponent->SetVisibility(false);
		break;
	case ETileMapEditorTool::PaintBucket:
		CursorPreviewComponent->SetVisibility(false);
		break;
	case ETileMapEditorTool::TerrainBrush:
		CursorPreviewComponent->SetVisibility(bShowPreviewDesired); //@TODO: TileMapTerrain
		break;
	default:
		check(false);
		break;
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
