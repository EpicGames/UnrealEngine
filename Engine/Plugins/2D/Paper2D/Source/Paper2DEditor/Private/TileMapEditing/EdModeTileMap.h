// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdMode.h"
#include "Math/IntRect.h"

class FExtender;
struct FViewportCursorLocation;

class FCanvas;
class UPaperTileLayer;
class UPaperTileMap;
class UPaperTileMapComponent;
class UPaperTileSet;

namespace ETileMapEditorTool
{
	enum Type
	{
		Paintbrush,
		Eraser,
		PaintBucket,
		EyeDropper,
		TerrainBrush
	};
}

//////////////////////////////////////////////////////////////////////////
// FTileMapDirtyRegion

struct FTileMapDirtyRegion
{
public:
	TWeakObjectPtr<UPaperTileMapComponent> ComponentPtr;
	FBox DirtyRegionInWorldSpace;

public:
	FTileMapDirtyRegion()
	{
	}

	FTileMapDirtyRegion(UPaperTileMapComponent* InComponent, const FBox& DirtyRegionInTileSpace);

	void PushToNavSystem() const;
	
	UPaperTileMapComponent* GetComponent() const
	{
		return ComponentPtr.Get();
	}
};

//////////////////////////////////////////////////////////////////////////
// FEdModeTileMap

class FEdModeTileMap : public FEdMode
{
public:
	static const FEditorModeID EM_TileMap;

public:
	FEdModeTileMap();
	virtual ~FEdModeTileMap();

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FSerializableObject

	// FEdMode interface
	virtual bool UsesToolkits() const override;
	virtual void Enter() override;
	virtual void Exit() override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual bool MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;
	virtual bool MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;
	virtual bool CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual void ActorSelectionChangeNotify() override;
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool UsesTransformWidget() const override;
	// End of FEdMode interface

	void SetActiveTool(ETileMapEditorTool::Type NewTool);
	ETileMapEditorTool::Type GetActiveTool() const;

	void SetActivePaint(UPaperTileSet* TileSet, FIntPoint TopLeft, FIntPoint Dimensions);
	void SetActivePaintFromLayer(UPaperTileLayer* SourceLayer, FIntPoint TopLeft, FIntPoint Dimensions);

	void DestructiveResizePreviewComponent(int32 NewWidth, int32 NewHeight);

	void FlipSelectionHorizontally();
	void FlipSelectionVertically();
	void RotateSelectionCW();
	void RotateSelectionCCW();
	bool HasValidSelection() const;

	void RefreshBrushSize();

protected:
	bool UseActiveToolAtLocation(const FViewportCursorLocation& Ray);

	bool SelectTiles(const FViewportCursorLocation& Ray);
	bool PaintTiles(const FViewportCursorLocation& Ray);
	bool EraseTiles(const FViewportCursorLocation& Ray);
	bool FloodFillTiles(const FViewportCursorLocation& Ray);
	bool PaintTilesWithTerrain(const FViewportCursorLocation& Ray);


	void UpdatePreviewCursor(const FViewportCursorLocation& Ray);

	// Returns the selected layer under the cursor, and the intersection tile coordinates
	// Note: The tile coordinates can be negative if the brush is off the top or left of the tile map, but still overlaps the map!!!
	UPaperTileLayer* GetSelectedLayerUnderCursor(const FViewportCursorLocation& Ray, int32& OutTileX, int32& OutTileY, bool bAllowOutOfBounds = false) const;

	// Compute a world space ray from the screen space mouse coordinates
	FViewportCursorLocation CalculateViewRay(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	
	TSharedRef<FExtender> AddCreationModeExtender(const TSharedRef<FUICommandList> InCommandList);

	void EnableTileMapEditMode();
	bool IsTileMapEditModeActive() const;

	void SynchronizePreviewWithTileMap(UPaperTileMap* NewTileMap);

	void RotateTilesInSelection(bool bIsClockwise);

	bool IsToolReadyToBeUsed() const;

	bool BlitLayer(UPaperTileLayer* SourceLayer, UPaperTileLayer* TargetLayer, FBox& OutDirtyRect, int32 OffsetX = 0, int32 OffsetY = 0, bool bBlitEmptyTiles = false);

	void FlushPendingDirtyRegions();
public:
	UPaperTileMapComponent* FindSelectedComponent() const;

	UPaperTileLayer* GetSourceInkLayer() const;

	int32 GetBrushWidth() const;
	int32 GetBrushHeight() const;

	int32 GetCursorWidth() const;
	int32 GetCursorHeight() const;
protected:
	// Were we previously painting?
	bool bWasPainting;

	// Are we currently painting?
	bool bIsPainting;

	// Ink source
	bool bHasValidInkSource;
	
	// State for eyedropper
	bool bWasHoldingSelectWhenPaintingStarted;
	FIntPoint EyeDropperStart;
	FIntRect LastEyeDropperBounds;

	//
	FTransform DrawPreviewSpace;

	// Center of preview rectangle
	FVector DrawPreviewLocation;

	// Size of rectangle
	int32 LastCursorTileX;
	int32 LastCursorTileY;
	int32 LastCursorTileZ;
	bool bIsLastCursorValid;
	TWeakObjectPtr<UPaperTileMap> LastCursorTileMap;

	FVector DrawPreviewDimensionsLS;

	// Top left of the component bounds
	FVector DrawPreviewTopLeft;

	int32 EraseBrushSize;

	UPaperTileMapComponent* CursorPreviewComponent;

	// Nav mesh rebuilding
	float TimeUntilNavMeshRebuild;
	TArray<FTileMapDirtyRegion> PendingDirtyRegions;

	//
	ETileMapEditorTool::Type ActiveTool;
	mutable FTransform ComponentToWorld;
};

