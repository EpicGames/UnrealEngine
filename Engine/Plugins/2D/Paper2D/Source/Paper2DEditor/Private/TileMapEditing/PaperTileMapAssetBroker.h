// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperTileMap.h"
#include "ComponentAssetBroker.h"
#include "PaperTileMapComponent.h"

//////////////////////////////////////////////////////////////////////////
// FPaperTileMapAssetBroker

class FPaperTileMapAssetBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UPaperTileMap::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UPaperTileMapComponent* RenderComp = Cast<UPaperTileMapComponent>(InComponent))
		{
			UPaperTileMap* TileMap = Cast<UPaperTileMap>(InAsset);

			if ((TileMap != nullptr) || (InAsset == nullptr))
			{
				RenderComp->TileMap = TileMap;
				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UPaperTileMapComponent* RenderComp = Cast<UPaperTileMapComponent>(InComponent))
		{
			if ((RenderComp->TileMap != nullptr) && (RenderComp->TileMap->IsAsset()))
			{
				return RenderComp->TileMap;
			}
		}

		return nullptr;
	}
};

