// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/PaperTileMapDetailsCustomization.h"
#include "DetailsViewArgs.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Commands/UICommandList.h"
#include "PaperTileLayer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "PaperTileMap.h"
#include "Widgets/Text/STextBlock.h"
#include "PaperTileMapComponent.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "EditorModeManager.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PaperTileSet.h"
#include "TileMapEditing/EdModeTileMap.h"

#include "IDetailsView.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "PaperTileMapPromotionFactory.h"
#include "PaperImporterSettings.h"

#include "TileMapEditing/STileLayerList.h"
#include "ScopedTransaction.h"
#include "IPropertyUtilities.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// FPaperTileMapDetailsCustomization

TSharedRef<IDetailCustomization> FPaperTileMapDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FPaperTileMapDetailsCustomization);
}

void FPaperTileMapDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout.GetSelectedObjects();
	MyDetailLayout = nullptr;
	
	FNotifyHook* NotifyHook = DetailLayout.GetPropertyUtilities()->GetNotifyHook();

	bool bEditingActor = false;

	UPaperTileMap* TileMap = nullptr;
	UPaperTileMapComponent* TileComponent = nullptr;
	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		UObject* TestObject = SelectedObjects[ObjectIndex].Get();
		if (AActor* CurrentActor = Cast<AActor>(TestObject))
		{
			if (UPaperTileMapComponent* CurrentComponent = CurrentActor->FindComponentByClass<UPaperTileMapComponent>())
			{
				bEditingActor = true;
				TileComponent = CurrentComponent;
				TileMap = CurrentComponent->TileMap;
				break;
			}
		}
		else if (UPaperTileMapComponent* TestComponent = Cast<UPaperTileMapComponent>(TestObject))
		{
			TileComponent = TestComponent;
			TileMap = TestComponent->TileMap;
			break;
		}
		else if (UPaperTileMap* TestTileMap = Cast<UPaperTileMap>(TestObject))
		{
			TileMap = TestTileMap;
			break;
		}
	}
	TileMapPtr = TileMap;
	TileMapComponentPtr = TileComponent;

	// Make sure the Tile Map category is right below the Transform
	IDetailCategoryBuilder& TileMapCategory = DetailLayout.EditCategory("Tile Map", LOCTEXT("TileMapDetailCategory", "Tile Map"), ECategoryPriority::Important);

	// Add the 'instanced' versus 'asset' indicator to the tile map header
	TileMapCategory.HeaderContent
	(
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(5.0f, 0.0f))
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("TinyText"))
				.Text_Lambda([this] { return IsInstanced() ? LOCTEXT("Instanced", "Instanced") : LOCTEXT("Asset", "Asset"); })
				.ToolTipText(LOCTEXT("InstancedVersusAssetTooltip", "Tile map components can either own a unique tile map instance, or reference a shareable tile map asset"))
			]
		]
	);


	TAttribute<EVisibility> InternalInstanceVis = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FPaperTileMapDetailsCustomization::GetVisibilityForInstancedOnlyProperties));

	TSharedRef<SWrapBox> ButtonBox = SNew(SWrapBox).UseAllottedSize(true);

	const float MinButtonSize = 120.0f;
	const FMargin ButtonPadding(0.0f, 2.0f, 2.0f, 0.0f);

	// Edit tile map button
	ButtonBox->AddSlot()
	.Padding(ButtonPadding)
	[
		SNew(SBox)
		.MinDesiredWidth(MinButtonSize)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.OnClicked(this, &FPaperTileMapDetailsCustomization::EnterTileMapEditingMode)
			.Visibility(this, &FPaperTileMapDetailsCustomization::GetNonEditModeVisibility)
			.IsEnabled(this, &FPaperTileMapDetailsCustomization::GetIsEditModeEnabled)
			.Text(LOCTEXT("EditAsset", "Edit Map"))
			.ToolTipText(LOCTEXT("EditAssetToolTip", "Edit this tile map"))
		]
	];

	// Create new tile map button
	ButtonBox->AddSlot()
	.Padding(ButtonPadding)
	[
		SNew(SBox)
		.MinDesiredWidth(MinButtonSize)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.OnClicked(this, &FPaperTileMapDetailsCustomization::OnNewButtonClicked)
			.Visibility(this, &FPaperTileMapDetailsCustomization::GetNewButtonVisiblity)
			.Text(LOCTEXT("CreateNewInstancedMap", "New Empty Map"))
			.ToolTipText(LOCTEXT("CreateNewInstancedMapToolTip", "Create a new (instanced) tile map"))
		]
	];

	// Promote to asset button
	ButtonBox->AddSlot()
	.Padding(ButtonPadding)
	[
		SNew(SBox)
		.MinDesiredWidth(MinButtonSize)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.OnClicked(this, &FPaperTileMapDetailsCustomization::OnPromoteToAssetButtonClicked)
			.Visibility(InternalInstanceVis)
			.Text(LOCTEXT("PromoteToAsset", "Promote To Asset"))
			.ToolTipText(LOCTEXT("PromoteToAssetToolTip", "Save this tile map as a reusable asset"))
		]
	];

	// Convert to instance button
	ButtonBox->AddSlot()
	.Padding(ButtonPadding)
	[
		SNew(SBox)
		.MinDesiredWidth(MinButtonSize)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.OnClicked(this, &FPaperTileMapDetailsCustomization::OnMakeInstanceFromAssetButtonClicked)
 			.Visibility(this, &FPaperTileMapDetailsCustomization::GetVisibilityForMakeIntoInstance)
			.Text(LOCTEXT("ConvertToInstance", "Convert To Instance"))
			.ToolTipText(LOCTEXT("ConvertToInstanceToolTip", "Copy the asset referenced by this tile map component into a unique instance that can be locally edited"))
		]
	];

	if (TileComponent != nullptr)
	{
		TileMapCategory
		.AddCustomRow(LOCTEXT( "TileMapInstancingControlsSearchText", "Edit Map New Empty Map Promote Asset"))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				ButtonBox
			]
		];

		TileMapCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UPaperTileMapComponent, TileMap));
	}

	// Try to get the hosting command list from the details view
	TSharedPtr<FUICommandList> CommandList;
	if (TSharedPtr<IDetailsView> DetailsView = DetailLayout.GetDetailsViewSharedPtr())
	{
		CommandList = DetailsView->GetHostCommandList();
	}
	if (!CommandList.IsValid())
	{
		CommandList = MakeShareable(new FUICommandList);
	}

	// Add the layer browser
	if (TileMap != nullptr)
	{
		TAttribute<EVisibility> LayerBrowserVis;
		LayerBrowserVis.Set(EVisibility::Visible);
		if (TileComponent != nullptr)
		{
			LayerBrowserVis = InternalInstanceVis;
		}

		FText TileLayerListText = LOCTEXT("TileLayerList", "Tile layer list");
		TileMapCategory.AddCustomRow(TileLayerListText)
		.Visibility(LayerBrowserVis)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Font(DetailLayout.GetDetailFont())
				.Text(TileLayerListText)
			]
			+SVerticalBox::Slot()
			[
				SNew(STileLayerList, TileMap, NotifyHook, CommandList)
				.OnSelectedLayerChanged(this, &FPaperTileMapDetailsCustomization::OnSelectedLayerChanged)
			]
		];
	}

	// Hide the layers since they'll get visualized directly
	TSharedRef<IPropertyHandle> TileLayersProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UPaperTileMap, TileLayers));
	DetailLayout.HideProperty(TileLayersProperty);

	// Add properties for the currently selected layer
	if ((TileMap != nullptr) && ((TileComponent == nullptr) || (TileComponent->OwnsTileMap())))
	{
		if (TileMap->TileLayers.IsValidIndex(TileMap->SelectedLayerIndex))
		{
			UPaperTileLayer* SelectedLayer = TileMap->TileLayers[TileMap->SelectedLayerIndex];

			const FText LayerCategoryDisplayName = LOCTEXT("LayerCategoryHeading", "Selected Layer");
			IDetailCategoryBuilder& LayerCategory = DetailLayout.EditCategory(TEXT("SelectedLayer"), LayerCategoryDisplayName, ECategoryPriority::Important);

			LayerCategory.HeaderContent
			(
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(FMargin(5.0f, 0.0f))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("TinyText"))
						.Text(this, &FPaperTileMapDetailsCustomization::GetLayerSettingsHeadingText)
						.ToolTipText(LOCTEXT("LayerSettingsTooltip", "Properties specific to the currently selected layer"))
					]
				]
			);

			TArray<UObject*> ListOfSelectedLayers;
			ListOfSelectedLayers.Add(SelectedLayer);

			for (const FProperty* TestProperty : TFieldRange<FProperty>(SelectedLayer->GetClass()))
			{
				if (TestProperty->HasAnyPropertyFlags(CPF_Edit))
				{
					const bool bAdvancedDisplay = TestProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay);
					const EPropertyLocation::Type PropertyLocation = bAdvancedDisplay ? EPropertyLocation::Advanced : EPropertyLocation::Common;

					LayerCategory.AddExternalObjectProperty(ListOfSelectedLayers, TestProperty->GetFName(), PropertyLocation);
				}
			}
		}
	}

	// Make sure the setup category is near the top (just below the layer browser and layer-specific stuff)
	IDetailCategoryBuilder& SetupCategory = DetailLayout.EditCategory("Setup", FText::GetEmpty(), ECategoryPriority::Important);

	// Add all of the properties from the inline tilemap
	if ((TileComponent != nullptr) && (TileComponent->OwnsTileMap()))
	{
		TArray<UObject*> ListOfTileMaps;
		ListOfTileMaps.Add(TileMap);

		for (const FProperty* TestProperty : TFieldRange<FProperty>(TileMap->GetClass()))
		{
			if (TestProperty->HasAnyPropertyFlags(CPF_Edit))
			{
				const bool bAdvancedDisplay = TestProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay);
				const EPropertyLocation::Type PropertyLocation = bAdvancedDisplay ? EPropertyLocation::Advanced : EPropertyLocation::Common;

				const FName CategoryName(*TestProperty->GetMetaData(TEXT("Category")));
				IDetailCategoryBuilder& Category = DetailLayout.EditCategory(CategoryName);

				if (IDetailPropertyRow* ExternalRow = Category.AddExternalObjectProperty(ListOfTileMaps, TestProperty->GetFName(), PropertyLocation))
				{
					ExternalRow->Visibility(InternalInstanceVis);
				}
			}
		}
	}

	MyDetailLayout = &DetailLayout;
}

FReply FPaperTileMapDetailsCustomization::EnterTileMapEditingMode()
{
	if (UPaperTileMapComponent* TileMapComponent = TileMapComponentPtr.Get())
	{
		if (TileMapComponent->OwnsTileMap())
		{
			GLevelEditorModeTools().ActivateMode(FEdModeTileMap::EM_TileMap);
		}
		else
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(TileMapComponent->TileMap);
		}
	}
	return FReply::Handled();
}

FReply FPaperTileMapDetailsCustomization::OnNewButtonClicked()
{
	if (UPaperTileMapComponent* TileMapComponent = TileMapComponentPtr.Get())
	{
		UPaperTileSet* OldTileSet = (TileMapComponent->TileMap != nullptr) ? TileMapComponent->TileMap->SelectedTileSet.Get() : nullptr;

		const FScopedTransaction Transaction(LOCTEXT("CreateNewTileMapTransaction", "New Tile Map"));
		TileMapComponent->Modify();
		TileMapComponent->CreateNewOwnedTileMap();

		// Add a layer and set things up
		GetDefault<UPaperImporterSettings>()->ApplySettingsForTileMapInit(TileMapComponent->TileMap, OldTileSet);

		MyDetailLayout->ForceRefreshDetails();
	}

	return FReply::Handled();
}

FReply FPaperTileMapDetailsCustomization::OnPromoteToAssetButtonClicked()
{
	if (UPaperTileMapComponent* TileMapComponent = TileMapComponentPtr.Get())
	{
		if (TileMapComponent->OwnsTileMap())
		{
			if (TileMapComponent->TileMap != nullptr)
			{
				const FScopedTransaction Transaction(LOCTEXT("PromoteToAssetTransactionTransaction", "Convert Tile Map instance to an asset"));

				// Try promoting the tile map to be an asset (prompts for a name&path, creates a package and then calls the factory, which renames the existing asset and sets RF_Public)
				UPaperTileMapPromotionFactory* PromotionFactory = NewObject<UPaperTileMapPromotionFactory>();
				PromotionFactory->AssetToRename = TileMapComponent->TileMap;

				FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
				UObject* NewAsset = AssetToolsModule.Get().CreateAssetWithDialog(PromotionFactory->GetSupportedClass(), PromotionFactory);
			
				// Show it in the content browser
				TArray<UObject*> ObjectsToSync;
				ObjectsToSync.Add(NewAsset);
				GEditor->SyncBrowserToObjects(ObjectsToSync);
			}
		}
	}

	MyDetailLayout->ForceRefreshDetails();

	return FReply::Handled();
}

FReply FPaperTileMapDetailsCustomization::OnMakeInstanceFromAssetButtonClicked()
{
	if (UPaperTileMapComponent* TileMapComponent = TileMapComponentPtr.Get())
	{
		if (!TileMapComponent->OwnsTileMap())
		{
			const FScopedTransaction Transaction(LOCTEXT("ConvertToInstanceTransactionTransaction", "Convert Tile Map asset to unique instance"));

			TileMapComponent->Modify();
			TileMapComponent->MakeTileMapEditable();
		}
	}

	MyDetailLayout->ForceRefreshDetails();

	return FReply::Handled();
}

bool FPaperTileMapDetailsCustomization::GetIsEditModeEnabled() const
{
	if (UPaperTileMapComponent* TileMapComponent = TileMapComponentPtr.Get())
	{
		return (TileMapComponent->TileMap != nullptr);
	}
	
	return false;
}

EVisibility FPaperTileMapDetailsCustomization::GetNonEditModeVisibility() const
{
	return InLevelEditorContext() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FPaperTileMapDetailsCustomization::GetNewButtonVisiblity() const
{
	return (TileMapComponentPtr.Get() != nullptr) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FPaperTileMapDetailsCustomization::GetVisibilityForInstancedOnlyProperties() const
{
	return IsInstanced() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FPaperTileMapDetailsCustomization::GetVisibilityForMakeIntoInstance() const
{
	return (!IsInstanced() && InLevelEditorContext()) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FPaperTileMapDetailsCustomization::InLevelEditorContext() const
{
	if (UPaperTileMapComponent* TileMapComponent = TileMapComponentPtr.Get())
	{
		return TileMapComponent->GetOwner() != nullptr;
	}

	return false;

	//@TODO: This isn't the right question, we should instead look and see if we're a customization for an actor versus a component
	//return GLevelEditorModeTools().IsModeActive(FEdModeTileMap::EM_TileMap);
}

bool FPaperTileMapDetailsCustomization::IsInstanced() const
{
	if (UPaperTileMapComponent* TileMapComponent = TileMapComponentPtr.Get())
	{
		return TileMapComponent->OwnsTileMap();
	}

	return false;
}

void FPaperTileMapDetailsCustomization::OnSelectedLayerChanged()
{
	if (MyDetailLayout != nullptr)
	{
		IDetailLayoutBuilder* OldLayout = MyDetailLayout;
		MyDetailLayout = nullptr;

		OldLayout->ForceRefreshDetails();
	}
}

FText FPaperTileMapDetailsCustomization::GetLayerSettingsHeadingText() const
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		if (TileMap->TileLayers.IsValidIndex(TileMap->SelectedLayerIndex))
		{
			UPaperTileLayer* SelectedLayer = TileMap->TileLayers[TileMap->SelectedLayerIndex];
			return SelectedLayer->LayerName;
		}
	}

	return FText::GetEmpty();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
