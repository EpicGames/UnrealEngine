// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
struct EVisibility;

class IDetailLayoutBuilder;

//////////////////////////////////////////////////////////////////////////
// FPaperTileMapDetailsCustomization

class FPaperTileMapDetailsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	TWeakObjectPtr<class UPaperTileMap> TileMapPtr;
	TWeakObjectPtr<class UPaperTileMapComponent> TileMapComponentPtr;

	IDetailLayoutBuilder* MyDetailLayout;

private:
	FReply EnterTileMapEditingMode();
	FReply OnNewButtonClicked();
	FReply OnPromoteToAssetButtonClicked();
	FReply OnMakeInstanceFromAssetButtonClicked();

	EVisibility GetNonEditModeVisibility() const;

	EVisibility GetVisibilityForInstancedOnlyProperties() const;
	
	EVisibility GetVisibilityForMakeIntoInstance() const;

	EVisibility GetNewButtonVisiblity() const;

	bool GetIsEditModeEnabled() const;

	bool InLevelEditorContext() const;
	bool IsInstanced() const;

	void OnSelectedLayerChanged();

	FText GetLayerSettingsHeadingText() const;
};
