// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/STileLayerItem.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "PaperTileLayer.h"
#include "ScopedTransaction.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "PaperStyle.h"

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// STileLayerItem

void STileLayerItem::Construct(const FArguments& InArgs, int32 Index, class UPaperTileMap* InMap, FIsSelected InIsSelectedDelegate)
{
	MyMap = InMap;
	MyIndex = Index;

	static const FName EyeClosedBrushName("TileMapEditor.LayerEyeClosed");
	static const FName EyeOpenedBrushName("TileMapEditor.LayerEyeOpened");
	EyeClosed = FPaperStyle::Get()->GetBrush(EyeClosedBrushName);
	EyeOpened = FPaperStyle::Get()->GetBrush(EyeOpenedBrushName);

	LayerNameWidget = SNew(SInlineEditableTextBlock)
		.Text(this, &STileLayerItem::GetLayerDisplayName)
		.ToolTipText(this, &STileLayerItem::GetLayerDisplayName)
		.OnTextCommitted(this, &STileLayerItem::OnLayerNameCommitted)
		.IsSelected(InIsSelectedDelegate);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SAssignNew( VisibilityButton, SButton )
			.ContentPadding(FMargin(4.0f, 4.0f, 4.0f, 4.0f))
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.OnClicked( this, &STileLayerItem::OnToggleVisibility )
			.ToolTipText( LOCTEXT("LayerVisibilityButtonToolTip", "Toggle Layer Visibility") )
			.ForegroundColor( FSlateColor::UseForeground() )
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.Content()
			[
				SNew(SImage)
				.Image(this, &STileLayerItem::GetVisibilityBrushForLayer)
				.ColorAndOpacity(this, &STileLayerItem::GetForegroundColorForVisibilityButton)
			]
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(4.0f, 4.0f, 4.0f, 4.0f))
		[
			LayerNameWidget.ToSharedRef()
		]
	];
}

void STileLayerItem::BeginEditingName()
{
	LayerNameWidget->EnterEditingMode();
}

FText STileLayerItem::GetLayerDisplayName() const
{
	const FText UnnamedText = LOCTEXT("NoLayerName", "(unnamed)");

	return GetMyLayer()->LayerName.IsEmpty() ? UnnamedText : GetMyLayer()->LayerName;
}

void STileLayerItem::OnLayerNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	const FScopedTransaction Transaction( LOCTEXT("TileMapRenameLayerTransaction", "Rename Layer") );
	UPaperTileLayer* MyLayer = GetMyLayer();
	MyLayer->SetFlags(RF_Transactional);
	MyLayer->Modify();
	MyLayer->LayerName = NewText;
}

FReply STileLayerItem::OnToggleVisibility()
{
	const FScopedTransaction Transaction( LOCTEXT("ToggleVisibilityTransaction", "Toggle Layer Visibility") );
	UPaperTileLayer* MyLayer = GetMyLayer();
	MyLayer->SetFlags(RF_Transactional);
	MyLayer->Modify();
	MyLayer->SetShouldRenderInEditor(!MyLayer->ShouldRenderInEditor());
	MyLayer->PostEditChange();
	return FReply::Handled();
}

const FSlateBrush* STileLayerItem::GetVisibilityBrushForLayer() const
{
	return GetMyLayer()->ShouldRenderInEditor() ? EyeOpened : EyeClosed;
}

FSlateColor STileLayerItem::GetForegroundColorForVisibilityButton() const
{
	static const FName InvertedForeground("InvertedForeground");
	return FAppStyle::GetSlateColor(InvertedForeground);
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
