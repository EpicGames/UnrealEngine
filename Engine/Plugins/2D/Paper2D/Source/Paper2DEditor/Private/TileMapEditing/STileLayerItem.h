// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "PaperTileMap.h"

class SInlineEditableTextBlock;

class SButton;
class UPaperTileLayer;
struct FSlateBrush;

//////////////////////////////////////////////////////////////////////////
// STileLayerItem

class STileLayerItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STileLayerItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, int32 Index, class UPaperTileMap* InMap, FIsSelected InIsSelectedDelegate);

	void BeginEditingName();

protected:
	int32 MyIndex;
	UPaperTileMap* MyMap;
	UPaperTileLayer* GetMyLayer() const { return MyMap->TileLayers[MyIndex]; }

	TSharedPtr<SButton> VisibilityButton;

	const FSlateBrush* EyeClosed;
	const FSlateBrush* EyeOpened;

	TSharedPtr<SInlineEditableTextBlock> LayerNameWidget;

protected:
	FText GetLayerDisplayName() const;
	void OnLayerNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	const FSlateBrush* GetVisibilityBrushForLayer() const;
	FSlateColor GetForegroundColorForVisibilityButton() const;
	FReply OnToggleVisibility();
};
