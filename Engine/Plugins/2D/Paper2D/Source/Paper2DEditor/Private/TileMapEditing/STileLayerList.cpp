// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/STileLayerList.h"
#include "Framework/Commands/UICommandList.h"
#include "PaperTileLayer.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Misc/NotifyHook.h"
#include "Misc/StringOutputDevice.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Views/SListView.h"
#include "Exporters/Exporter.h"
#include "Editor.h"
#include "PaperTileMapComponent.h"
#include "TileMapEditing/STileLayerItem.h"
#include "HAL/PlatformApplicationMisc.h"

#include "ScopedTransaction.h"

#include "TileMapEditing/TileMapEditorCommands.h"
#include "Framework/Commands/GenericCommands.h"

#include "UnrealExporter.h"
#include "Factories.h"

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// FLayerTextFactory

// Text object factory for pasting layers
struct FLayerTextFactory : public FCustomizableTextObjectFactory
{
public:
	TArray<UPaperTileLayer*> CreatedLayers;
public:
	FLayerTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory interface
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		// Only allow layers to be created
		return ObjectClass->IsChildOf(UPaperTileLayer::StaticClass());
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		CreatedLayers.Add(CastChecked<UPaperTileLayer>(NewObject));
	}
	// End of FCustomizableTextObjectFactory interface
};

//////////////////////////////////////////////////////////////////////////
// STileLayerList

void STileLayerList::Construct(const FArguments& InArgs, UPaperTileMap* InTileMap, FNotifyHook* InNotifyHook, TSharedPtr<class FUICommandList> InParentCommandList)
{
	OnSelectedLayerChanged = InArgs._OnSelectedLayerChanged;
	TileMapPtr = InTileMap;
	NotifyHook = InNotifyHook;

	FTileMapEditorCommands::Register();
	FGenericCommands::Register();
	const FTileMapEditorCommands& TileMapCommands = FTileMapEditorCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	CommandList = MakeShareable(new FUICommandList);
	InParentCommandList->Append(CommandList.ToSharedRef());

	CommandList->MapAction(
		TileMapCommands.AddNewLayerAbove,
		FExecuteAction::CreateSP(this, &STileLayerList::AddNewLayerAbove));

	CommandList->MapAction(
		TileMapCommands.AddNewLayerBelow,
		FExecuteAction::CreateSP(this, &STileLayerList::AddNewLayerBelow));

	CommandList->MapAction(
		GenericCommands.Cut,
		FExecuteAction::CreateSP(this, &STileLayerList::CutLayer),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingSelectedLayer));

	CommandList->MapAction(
		GenericCommands.Copy,
		FExecuteAction::CreateSP(this, &STileLayerList::CopyLayer),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingSelectedLayer));

	CommandList->MapAction(
		GenericCommands.Paste,
		FExecuteAction::CreateSP(this, &STileLayerList::PasteLayerAbove),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanPasteLayer));

	CommandList->MapAction(
		GenericCommands.Duplicate,
		FExecuteAction::CreateSP(this, &STileLayerList::DuplicateLayer),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingSelectedLayer));

	CommandList->MapAction(
		GenericCommands.Delete,
		FExecuteAction::CreateSP(this, &STileLayerList::DeleteLayer),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingSelectedLayer));

	CommandList->MapAction(
		GenericCommands.Rename,
		FExecuteAction::CreateSP(this, &STileLayerList::RenameLayer),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingSelectedLayer));

	CommandList->MapAction(
		TileMapCommands.MergeLayerDown,
		FExecuteAction::CreateSP(this, &STileLayerList::MergeLayerDown),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingLayerBelow));

	CommandList->MapAction(
		TileMapCommands.MoveLayerUp,
		FExecuteAction::CreateSP(this, &STileLayerList::MoveLayerUp, /*bForceToTop=*/ false),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingLayerAbove));

	CommandList->MapAction(
		TileMapCommands.MoveLayerDown,
		FExecuteAction::CreateSP(this, &STileLayerList::MoveLayerDown, /*bForceToBottom=*/ false),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingLayerBelow));

	CommandList->MapAction(
		TileMapCommands.MoveLayerToTop,
		FExecuteAction::CreateSP(this, &STileLayerList::MoveLayerUp, /*bForceToTop=*/ true),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingLayerAbove));

	CommandList->MapAction(
		TileMapCommands.MoveLayerToBottom,
		FExecuteAction::CreateSP(this, &STileLayerList::MoveLayerDown, /*bForceToBottom=*/ true),
		FCanExecuteAction::CreateSP(this, &STileLayerList::CanExecuteActionNeedingLayerBelow));

	CommandList->MapAction(
		TileMapCommands.SelectLayerAbove,
		FExecuteAction::CreateSP(this, &STileLayerList::SelectLayerAbove, /*bTopmost=*/ false));

	CommandList->MapAction(
		TileMapCommands.SelectLayerBelow,
		FExecuteAction::CreateSP(this, &STileLayerList::SelectLayerBelow, /*bBottommost=*/ false));

	//
	FToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization("TileLayerBrowserToolbar"), TSharedPtr<FExtender>(), /*InForceSmallIcons=*/ true);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	ToolbarBuilder.AddToolBarButton(TileMapCommands.AddNewLayerAbove);
	ToolbarBuilder.AddToolBarButton(TileMapCommands.MoveLayerUp);
	ToolbarBuilder.AddToolBarButton(TileMapCommands.MoveLayerDown);

	FSlateIcon DuplicateIcon(FPaperStyle::GetStyleSetName(), "TileMapEditor.DuplicateLayer");
	ToolbarBuilder.AddToolBarButton(GenericCommands.Duplicate, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DuplicateIcon);

	FSlateIcon DeleteIcon(FPaperStyle::GetStyleSetName(), "TileMapEditor.DeleteLayer");
	ToolbarBuilder.AddToolBarButton(GenericCommands.Delete, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DeleteIcon);

	TSharedRef<SWidget> Toolbar = ToolbarBuilder.MakeWidget();

	ListViewWidget = SNew(SPaperLayerListView)
		.SelectionMode(ESelectionMode::Single)
		.ClearSelectionOnClick(false)
		.ListItemsSource(&MirrorList)
		.OnSelectionChanged(this, &STileLayerList::OnSelectionChanged)
		.OnGenerateRow(this, &STileLayerList::OnGenerateLayerListRow)
		.OnContextMenuOpening(this, &STileLayerList::OnConstructContextMenu);

	RefreshMirrorList();

	// Restore the selection
	InTileMap->ValidateSelectedLayerIndex();
	if (InTileMap->TileLayers.IsValidIndex(InTileMap->SelectedLayerIndex))
	{
		UPaperTileLayer* SelectedLayer = InTileMap->TileLayers[InTileMap->SelectedLayerIndex];
		SetSelectedLayer(SelectedLayer);
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SBox)
			.HeightOverride(115.0f)
			[
				ListViewWidget.ToSharedRef()
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			Toolbar
		]
	];

	GEditor->RegisterForUndo(this);
}

TSharedRef<ITableRow> STileLayerList::OnGenerateLayerListRow(FMirrorEntry Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	typedef STableRow<FMirrorEntry> RowType;

	TSharedRef<RowType> NewRow = SNew(RowType, OwnerTable)
		.Style(&FPaperStyle::Get()->GetWidgetStyle<FTableRowStyle>("TileMapEditor.LayerBrowser.TableViewRow"));

	FIsSelected IsSelectedDelegate = FIsSelected::CreateSP(NewRow, &RowType::IsSelectedExclusively);
	NewRow->SetContent(SNew(STileLayerItem, *Item, TileMapPtr.Get(), IsSelectedDelegate));

	return NewRow;
}

UPaperTileLayer* STileLayerList::GetSelectedLayer() const
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		if (ListViewWidget->GetNumItemsSelected() > 0)
		{
			FMirrorEntry SelectedItem = ListViewWidget->GetSelectedItems()[0];
			const int32 SelectedIndex = *SelectedItem;
			if (TileMap->TileLayers.IsValidIndex(SelectedIndex))
			{
				return TileMap->TileLayers[SelectedIndex];
			}
		}
	}

	return nullptr;
}

FText STileLayerList::GenerateDuplicatedLayerName(const FString& InputNameRaw, UPaperTileMap* TileMap)
{
	// Create a set of existing names
	bool bFoundName = false;
	TSet<FString> ExistingNames;
	for (UPaperTileLayer* ExistingLayer : TileMap->TileLayers)
	{
		const FString LayerName = ExistingLayer->LayerName.ToString();
		ExistingNames.Add(LayerName);

		if (LayerName == InputNameRaw)
		{
			bFoundName = true;
		}
	}

	// If the name doesn't already exist, then we're done (can happen when pasting a cut layer, etc...)
	if (!bFoundName)
	{
		return FText::FromString(InputNameRaw);
	}

	FString BaseName = InputNameRaw;
	int32 TestIndex = 0;
	bool bAddNumber = false;

	// See if this is the result of a previous duplication operation, and change the desired name accordingly
	int32 SpaceIndex;
	if (InputNameRaw.FindLastChar(' ', /*out*/ SpaceIndex))
	{
		FString PossibleDuplicationSuffix = InputNameRaw.Mid(SpaceIndex + 1);

		if (PossibleDuplicationSuffix == TEXT("copy"))
		{
			bAddNumber = true;
			BaseName = InputNameRaw.Left(SpaceIndex);
			TestIndex = 2;
		}
		else
		{
			int32 ExistingIndex = FCString::Atoi(*PossibleDuplicationSuffix);

			const FString TestSuffix = FString::Printf(TEXT(" copy %d"), ExistingIndex);

			if (InputNameRaw.EndsWith(TestSuffix))
			{
				bAddNumber = true;
				BaseName = InputNameRaw.Left(InputNameRaw.Len() - TestSuffix.Len());
				TestIndex = ExistingIndex + 1;
			}
		}
	}

	// Find a good name
	FString TestLayerName = BaseName + TEXT(" copy");

	if (bAddNumber || ExistingNames.Contains(TestLayerName))
	{
		do
		{
			TestLayerName = FString::Printf(TEXT("%s copy %d"), *BaseName, TestIndex++);
		} while (ExistingNames.Contains(TestLayerName));
	}

	return FText::FromString(TestLayerName);
}

UPaperTileLayer* STileLayerList::AddLayer(int32 InsertionIndex)
{
	UPaperTileLayer* NewLayer = nullptr;

	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const FScopedTransaction Transaction(LOCTEXT("TileMapAddLayerTransaction", "Add New Layer"));
		TileMap->SetFlags(RF_Transactional);
		TileMap->Modify();

		NewLayer = TileMap->AddNewLayer(InsertionIndex);

		PostEditNotfications();

		// Change the selection set to select it
		SetSelectedLayer(NewLayer);
	}

	return NewLayer;
}

void STileLayerList::ChangeLayerOrdering(int32 OldIndex, int32 NewIndex)
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		if (TileMap->TileLayers.IsValidIndex(OldIndex) && TileMap->TileLayers.IsValidIndex(NewIndex))
		{
			const FScopedTransaction Transaction(LOCTEXT("TileMapReorderLayerTransaction", "Reorder Layer"));
			TileMap->SetFlags(RF_Transactional);
			TileMap->Modify();

			UPaperTileLayer* LayerToMove = TileMap->TileLayers[OldIndex];
			TileMap->TileLayers.RemoveAt(OldIndex);
			TileMap->TileLayers.Insert(LayerToMove, NewIndex);

			if (TileMap->SelectedLayerIndex == OldIndex)
			{
				TileMap->SelectedLayerIndex = NewIndex;
				SetSelectedLayer(LayerToMove);
			}

			PostEditNotfications();
		}
	}
}

void STileLayerList::AddNewLayerAbove()
{
	AddLayer(GetSelectionIndex());
}

void STileLayerList::AddNewLayerBelow()
{
	AddLayer(GetSelectionIndex() + 1);
}

int32 STileLayerList::GetSelectionIndex() const
{
	int32 SelectionIndex = INDEX_NONE;

	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		if (UPaperTileLayer* SelectedLayer = GetSelectedLayer())
		{
			TileMap->TileLayers.Find(SelectedLayer, /*out*/ SelectionIndex);
		}
		else
		{
			SelectionIndex = TileMap->TileLayers.Num() - 1;
		}
	}

	return SelectionIndex;
}

void STileLayerList::DeleteSelectedLayerWithNoTransaction()
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const int32 DeleteIndex = GetSelectionIndex();
		if (DeleteIndex != INDEX_NONE)
		{
			TileMap->TileLayers.RemoveAt(DeleteIndex);

			PostEditNotfications();

			// Select the item below the one that just got deleted
			const int32 NewSelectionIndex = FMath::Min<int32>(DeleteIndex, TileMap->TileLayers.Num() - 1);
			if (TileMap->TileLayers.IsValidIndex(NewSelectionIndex))
			{
				SetSelectedLayer(TileMap->TileLayers[NewSelectionIndex]);
			}
		}
	}
}

void STileLayerList::DeleteLayer()
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const FScopedTransaction Transaction(LOCTEXT("TileMapDeleteLayerTransaction", "Delete Layer"));
		TileMap->SetFlags(RF_Transactional);
		TileMap->Modify();

		DeleteSelectedLayerWithNoTransaction();
	}
}

void STileLayerList::RenameLayer()
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const int32 RenameIndex = GetSelectionIndex();
		if (MirrorList.IsValidIndex(RenameIndex))
		{
			TSharedPtr<ITableRow> LayerRowWidget = ListViewWidget->WidgetFromItem(MirrorList[RenameIndex]);
			if (LayerRowWidget.IsValid())
			{
				TSharedPtr<SWidget> RowContent = LayerRowWidget->GetContent();
				if (RowContent.IsValid())
				{
					TSharedPtr<STileLayerItem> LayerWidget = StaticCastSharedPtr<STileLayerItem>(RowContent);
					LayerWidget->BeginEditingName();
				}
			}
		}
	}
}


void STileLayerList::DuplicateLayer()
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const int32 DuplicateIndex = GetSelectionIndex();
		if (DuplicateIndex != INDEX_NONE)
		{
			const FScopedTransaction Transaction(LOCTEXT("TileMapDuplicateLayerTransaction", "Duplicate Layer"));
			TileMap->SetFlags(RF_Transactional);
			TileMap->Modify();

			UPaperTileLayer* NewLayer = DuplicateObject<UPaperTileLayer>(TileMap->TileLayers[DuplicateIndex], TileMap);
			TileMap->TileLayers.Insert(NewLayer, DuplicateIndex);
			NewLayer->LayerName = GenerateDuplicatedLayerName(NewLayer->LayerName.ToString(), TileMap);

			PostEditNotfications();

			// Select the duplicated layer
			SetSelectedLayer(NewLayer);
		}
	}
}

void STileLayerList::MergeLayerDown()
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const int32 SourceIndex = GetSelectionIndex();
		const int32 TargetIndex = SourceIndex + 1;
		if ((SourceIndex != INDEX_NONE) && (TargetIndex != INDEX_NONE))
		{
			const FScopedTransaction Transaction(LOCTEXT("TileMapMergeLayerDownTransaction", "Merge Layer Down"));
			TileMap->SetFlags(RF_Transactional);
			TileMap->Modify();

			UPaperTileLayer* SourceLayer = TileMap->TileLayers[SourceIndex];
			UPaperTileLayer* TargetLayer = TileMap->TileLayers[TargetIndex];

			TargetLayer->SetFlags(RF_Transactional);
			TargetLayer->Modify();

			// Copy the non-empty tiles from the source to the target layer
			for (int32 Y = 0; Y < SourceLayer->GetLayerHeight(); ++Y)
			{
				for (int32 X = 0; X < SourceLayer->GetLayerWidth(); ++X)
				{
					FPaperTileInfo TileInfo = SourceLayer->GetCell(X, Y);
					if (TileInfo.IsValid())
					{
						TargetLayer->SetCell(X, Y, TileInfo);
					}
				}
			}

			// Remove the source layer
			TileMap->TileLayers.RemoveAt(SourceIndex);

			// Update viewers
			PostEditNotfications();
		}
	}
}

void STileLayerList::MoveLayerUp(bool bForceToTop)
{
	const int32 SelectedIndex = GetSelectionIndex();
	const int32 NewIndex = bForceToTop ? 0 : (SelectedIndex - 1);
	ChangeLayerOrdering(SelectedIndex, NewIndex);
}

void STileLayerList::MoveLayerDown(bool bForceToBottom)
{
	const int32 SelectedIndex = GetSelectionIndex();
	const int32 NewIndex = bForceToBottom ? (GetNumLayers() - 1) : (SelectedIndex + 1);
	ChangeLayerOrdering(SelectedIndex, NewIndex);
}

void STileLayerList::SelectLayerAbove(bool bTopmost)
{
	const int32 SelectedIndex = GetSelectionIndex();
	const int32 NumLayers = GetNumLayers();
	const int32 NewIndex = bTopmost ? 0 : ((NumLayers + SelectedIndex - 1) % NumLayers);
	SetSelectedLayerIndex(NewIndex);
}

void STileLayerList::SelectLayerBelow(bool bBottommost)
{
	const int32 SelectedIndex = GetSelectionIndex();
	const int32 NumLayers = GetNumLayers();
	const int32 NewIndex = bBottommost ? (NumLayers - 1) : ((SelectedIndex + 1) % NumLayers);
	SetSelectedLayerIndex(NewIndex);
}

void STileLayerList::CutLayer()
{
	CopyLayer();

	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const FScopedTransaction Transaction(LOCTEXT("TileMapCutLayerTransaction", "Cut Layer"));
		TileMap->SetFlags(RF_Transactional);
		TileMap->Modify();

		DeleteSelectedLayerWithNoTransaction();
	}
}

void STileLayerList::CopyLayer()
{
	if (UPaperTileLayer* SelectedLayer = GetSelectedLayer())
	{
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
		FStringOutputDevice ExportArchive;
		const FExportObjectInnerContext Context;
		UExporter::ExportToOutputDevice(&Context, SelectedLayer, nullptr, ExportArchive, TEXT("copy"), 0, PPF_Copy, false, nullptr);

		FPlatformApplicationMisc::ClipboardCopy(*ExportArchive);
	}
}

void STileLayerList::PasteLayerAbove()
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		FString ClipboardContent;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

		if (!ClipboardContent.IsEmpty())
		{
			const FScopedTransaction Transaction(LOCTEXT("TileMapPasteLayerTransaction", "Paste Layer"));
			TileMap->SetFlags(RF_Transactional);
			TileMap->Modify();

			// Turn the text buffer into objects
			FLayerTextFactory Factory;
			Factory.ProcessBuffer(TileMap, RF_Transactional, ClipboardContent);

			// Add them to the map and select them (there will currently only ever be 0 or 1)
			for (UPaperTileLayer* NewLayer : Factory.CreatedLayers)
			{
				NewLayer->LayerName = GenerateDuplicatedLayerName(NewLayer->LayerName.ToString(), TileMap);
				TileMap->AddExistingLayer(NewLayer, GetSelectionIndex());
				PostEditNotfications();
				SetSelectedLayer(NewLayer);
			}
		}
	}
}

bool STileLayerList::CanPasteLayer() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return !ClipboardContent.IsEmpty();
}

void STileLayerList::SetSelectedLayerIndex(int32 NewIndex)
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		if (TileMap->TileLayers.IsValidIndex(NewIndex))
		{
			SetSelectedLayer(TileMap->TileLayers[NewIndex]);
			PostEditNotfications();
		}
	}
}

int32 STileLayerList::GetNumLayers() const
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		return TileMap->TileLayers.Num();
	}
	return 0;
}

bool STileLayerList::CanExecuteActionNeedingLayerAbove() const
{
	return GetSelectionIndex() > 0;
}

bool STileLayerList::CanExecuteActionNeedingLayerBelow() const
{
	const int32 SelectedLayer = GetSelectionIndex();
	return (SelectedLayer != INDEX_NONE) && (SelectedLayer + 1 < GetNumLayers());
}

bool STileLayerList::CanExecuteActionNeedingSelectedLayer() const
{
	return GetSelectionIndex() != INDEX_NONE;
}

void STileLayerList::SetSelectedLayer(UPaperTileLayer* SelectedLayer)
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		int32 NewIndex;
		if (TileMap->TileLayers.Find(SelectedLayer, /*out*/ NewIndex))
		{
			if (MirrorList.IsValidIndex(NewIndex))
			{
				ListViewWidget->SetSelection(MirrorList[NewIndex]);
			}
		}
	}
}

void STileLayerList::OnSelectionChanged(FMirrorEntry ItemChangingState, ESelectInfo::Type SelectInfo)
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		TileMap->SelectedLayerIndex = GetSelectionIndex();

		PostEditNotfications();
	}
}

TSharedPtr<SWidget> STileLayerList::OnConstructContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	const FTileMapEditorCommands& TileMapCommands = FTileMapEditorCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	FSlateIcon DummyIcon(NAME_None, NAME_None);

	MenuBuilder.BeginSection("BasicOperations", LOCTEXT("BasicOperationsHeader", "Layer actions"));
 	{
		MenuBuilder.AddMenuEntry(GenericCommands.Cut, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(GenericCommands.Copy, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(GenericCommands.Paste, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(GenericCommands.Duplicate, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(GenericCommands.Delete, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(GenericCommands.Rename, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(TileMapCommands.MergeLayerDown, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuSeparator();
		MenuBuilder.AddMenuEntry(TileMapCommands.SelectLayerAbove, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(TileMapCommands.SelectLayerBelow, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("OrderingOperations", LOCTEXT("OrderingOperationsHeader", "Order actions"));
	{
		MenuBuilder.AddMenuEntry(TileMapCommands.MoveLayerToTop, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(TileMapCommands.MoveLayerUp, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(TileMapCommands.MoveLayerDown, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
		MenuBuilder.AddMenuEntry(TileMapCommands.MoveLayerToBottom, NAME_None, TAttribute<FText>(), TAttribute<FText>(), DummyIcon);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void STileLayerList::PostEditNotfications()
{
	RefreshMirrorList();

	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		TileMap->PostEditChange();
	}

	if (NotifyHook != nullptr)
	{
		FProperty* TileMapProperty = FindFieldChecked<FProperty>(UPaperTileMapComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UPaperTileMapComponent, TileMap));
		NotifyHook->NotifyPreChange(TileMapProperty);
		NotifyHook->NotifyPostChange(FPropertyChangedEvent(TileMapProperty), TileMapProperty);
	}

	OnSelectedLayerChanged.Execute();
}

void STileLayerList::RefreshMirrorList()
{
	if (UPaperTileMap* TileMap = TileMapPtr.Get())
	{
		const int32 NumEntriesToAdd = TileMap->TileLayers.Num() - MirrorList.Num();
		if (NumEntriesToAdd < 0)
		{
			MirrorList.RemoveAt(TileMap->TileLayers.Num(), -NumEntriesToAdd);
		}
		else if (NumEntriesToAdd > 0)
		{
			for (int32 Count = 0; Count < NumEntriesToAdd; ++Count)
			{
				TSharedPtr<int32> NewEntry = MakeShareable(new int32);
				*NewEntry = MirrorList.Num();
				MirrorList.Add(NewEntry);
			}
		}
	}
	else
	{
		MirrorList.Empty();
	}

	ListViewWidget->RequestListRefresh();
}

void STileLayerList::PostUndo(bool bSuccess)
{
	RefreshMirrorList();
}

void STileLayerList::PostRedo(bool bSuccess)
{
	RefreshMirrorList();
}

STileLayerList::~STileLayerList()
{
	GEditor->UnregisterForUndo(this);
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
