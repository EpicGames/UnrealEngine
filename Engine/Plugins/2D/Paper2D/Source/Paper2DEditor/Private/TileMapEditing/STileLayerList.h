// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "EditorUndoClient.h"

class ITableRow;
class STableViewBase;
class UPaperTileMap;
namespace ESelectInfo { enum Type : int; }

class FNotifyHook;
class UPaperTileLayer;
template <typename ItemType> class SListView;

//////////////////////////////////////////////////////////////////////////
// STileLayerList

class STileLayerList : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(STileLayerList) {}
		SLATE_EVENT(FSimpleDelegate, OnSelectedLayerChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UPaperTileMap* TileMap, FNotifyHook* InNotifyHook, TSharedPtr<class FUICommandList> InCommandList);

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	~STileLayerList();
protected:
	typedef TSharedPtr<int32> FMirrorEntry;

	TArray<FMirrorEntry> MirrorList;
	typedef SListView<FMirrorEntry> SPaperLayerListView;

protected:
	TSharedPtr<SPaperLayerListView> ListViewWidget;
	TSharedPtr<class FUICommandList> CommandList;
	TWeakObjectPtr<class UPaperTileMap> TileMapPtr;
	FNotifyHook* NotifyHook;
	FSimpleDelegate OnSelectedLayerChanged;
protected:
	TSharedRef<ITableRow> OnGenerateLayerListRow(FMirrorEntry Item, const TSharedRef<STableViewBase>& OwnerTable);

	class UPaperTileLayer* GetSelectedLayer() const;

	// Returns the selected index if anything is selected, or the top item otherwise (only returns INDEX_NONE if there are no layers)
	int32 GetSelectionIndex() const;

	static FText GenerateDuplicatedLayerName(const FString& InputNameRaw, UPaperTileMap* TileMap);

	class UPaperTileLayer* AddLayer(int32 InsertionIndex = INDEX_NONE);

	// Moves a layer from OldIndex to NewIndex if both are valid, otherwise it does nothing silently
	void ChangeLayerOrdering(int32 OldIndex, int32 NewIndex);

	void AddNewLayerAbove();
	void AddNewLayerBelow();

	void CutLayer();
	void CopyLayer();
	void PasteLayerAbove();
	bool CanPasteLayer() const;

	void DuplicateLayer();
	void DeleteLayer();
	void RenameLayer();

	void DeleteSelectedLayerWithNoTransaction();

	void MergeLayerDown();
	void MoveLayerUp(bool bForceToTop);
	void MoveLayerDown(bool bForceToBottom);
	void SelectLayerAbove(bool bTopmost);
	void SelectLayerBelow(bool bBottommost);


	void SetSelectedLayerIndex(int32 NewIndex);

	int32 GetNumLayers() const;
	bool CanExecuteActionNeedingLayerAbove() const;
	bool CanExecuteActionNeedingLayerBelow() const;
	bool CanExecuteActionNeedingSelectedLayer() const;

	void SetSelectedLayer(UPaperTileLayer* SelectedLayer);

	void OnSelectionChanged(FMirrorEntry ItemChangingState, ESelectInfo::Type SelectInfo);
	TSharedPtr<SWidget> OnConstructContextMenu();

	void RefreshMirrorList();

	// Called after edits are finished
	void PostEditNotfications();
};
