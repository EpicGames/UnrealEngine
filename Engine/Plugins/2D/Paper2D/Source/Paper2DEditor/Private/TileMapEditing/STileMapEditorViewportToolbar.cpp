// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/STileMapEditorViewportToolbar.h"
#include "Framework/Commands/UICommandList.h"
#include "TileMapEditing/TileMapEditorCommands.h"
#include "SEditorViewport.h"

#define LOCTEXT_NAMESPACE "STileMapEditorViewportToolbar"

///////////////////////////////////////////////////////////
// STileMapEditorViewportToolbar

void STileMapEditorViewportToolbar::Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider)
{
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InInfoProvider);
}

TSharedRef<SWidget> STileMapEditorViewportToolbar::GenerateShowMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();

	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const FTileMapEditorCommands& Commands = FTileMapEditorCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		ShowMenuBuilder.AddMenuEntry(Commands.SetShowPivot);

		ShowMenuBuilder.AddMenuSeparator();

		ShowMenuBuilder.AddMenuEntry(Commands.SetShowTileGrid);
		ShowMenuBuilder.AddMenuEntry(Commands.SetShowLayerGrid);
		ShowMenuBuilder.AddMenuEntry(Commands.SetShowTileMapStats);

		ShowMenuBuilder.AddMenuSeparator();

		ShowMenuBuilder.AddMenuEntry(Commands.SetShowCollision);
	}

	return ShowMenuBuilder.MakeWidget();
}


#undef LOCTEXT_NAMESPACE
