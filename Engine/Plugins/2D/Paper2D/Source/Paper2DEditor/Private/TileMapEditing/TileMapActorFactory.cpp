// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapActorFactory.h"
#include "AssetRegistry/AssetData.h"
#include "PaperTileMapActor.h"
#include "PaperTileMap.h"
#include "PaperImporterSettings.h"
#include "PaperTileMapComponent.h"
#include "PaperTileSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TileMapActorFactory)

//////////////////////////////////////////////////////////////////////////
// UTileMapActorFactory

UTileMapActorFactory::UTileMapActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = NSLOCTEXT("Paper2D", "TileMapFactoryDisplayName", "Paper2D Tile Map");
	NewActorClass = APaperTileMapActor::StaticClass();
}

void UTileMapActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	APaperTileMapActor* TypedActor = CastChecked<APaperTileMapActor>(NewActor);
	UPaperTileMapComponent* RenderComponent = TypedActor->GetRenderComponent();
	check(RenderComponent);

	if (UPaperTileMap* TileMapAsset = Cast<UPaperTileMap>(Asset))
	{
		RenderComponent->UnregisterComponent();
		RenderComponent->TileMap = TileMapAsset;
		RenderComponent->RegisterComponent();
	}
	else if (RenderComponent->OwnsTileMap())
	{
		RenderComponent->UnregisterComponent();

		UPaperTileMap* OwnedTileMap = RenderComponent->TileMap;
		check(OwnedTileMap);

		GetDefault<UPaperImporterSettings>()->ApplySettingsForTileMapInit(OwnedTileMap, Cast<UPaperTileSet>(Asset));

		RenderComponent->RegisterComponent();
	}
}

bool UTileMapActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (AssetData.IsValid())
	{
		UClass* AssetClass = AssetData.GetClass();
		if ((AssetClass != nullptr) && (AssetClass->IsChildOf(UPaperTileMap::StaticClass()) || AssetClass->IsChildOf(UPaperTileSet::StaticClass())))
		{
			return true;
		}
		else
		{
			OutErrorMsg = NSLOCTEXT("Paper2D", "CanCreateActorFrom_NoTileMap", "No tile map was specified.");
			return false;
		}
	}
	else
	{
		return true;
	}
}

