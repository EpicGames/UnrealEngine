// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapAssetImportData.h"
#include "PaperTileMap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TileMapAssetImportData)

//////////////////////////////////////////////////////////////////////////
// UTileMapAssetImportData

UTileMapAssetImportData::UTileMapAssetImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UTileMapAssetImportData* UTileMapAssetImportData::GetImportDataForTileMap(UPaperTileMap* TileMap/*, UFbxSkeletalMeshImportData* TemplateForCreation*/)
{
	check(TileMap);

	UTileMapAssetImportData* ImportData = Cast<UTileMapAssetImportData>(TileMap->AssetImportData);
	if (ImportData == nullptr)
	{
		ImportData = NewObject<UTileMapAssetImportData>(TileMap, NAME_None, RF_NoFlags/*, TemplateForCreation*/);

		// Try to preserve the source file path if possible
		if (TileMap->AssetImportData != nullptr)
		{
			ImportData->SourceData = TileMap->AssetImportData->SourceData;
		}

		TileMap->AssetImportData = ImportData;
	}

	return ImportData;
}

