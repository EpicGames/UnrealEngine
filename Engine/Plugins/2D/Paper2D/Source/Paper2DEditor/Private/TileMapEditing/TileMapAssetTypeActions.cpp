// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/TileMapAssetTypeActions.h"

#include "Math/ColorList.h"
#include "TileMapEditing/TileMapEditor.h"
#include "PaperTileMap.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

//////////////////////////////////////////////////////////////////////////
// FTileMapAssetTypeActions

FTileMapAssetTypeActions::FTileMapAssetTypeActions(EAssetTypeCategories::Type InAssetCategory)
	: MyAssetCategory(InAssetCategory)
{
}

FText FTileMapAssetTypeActions::GetName() const
{
	return LOCTEXT("FTileMapAssetTypeActionsName", "Tile Map");
}

FColor FTileMapAssetTypeActions::GetTypeColor() const
{
	return FColorList::BrightGold;
}

UClass* FTileMapAssetTypeActions::GetSupportedClass() const
{
	return UPaperTileMap::StaticClass();
}

void FTileMapAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UPaperTileMap* TileMap = Cast<UPaperTileMap>(*ObjIt))
		{
			TSharedRef<FTileMapEditor> NewTileMapEditor(new FTileMapEditor());
			NewTileMapEditor->InitTileMapEditor(Mode, EditWithinLevelEditor, TileMap);
		}
	}
}

uint32 FTileMapAssetTypeActions::GetCategories()
{
	return MyAssetCategory;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
