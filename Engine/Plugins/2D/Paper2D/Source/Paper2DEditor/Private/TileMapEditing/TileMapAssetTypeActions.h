// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class FTileMapAssetTypeActions : public FAssetTypeActions_Base
{
public:
	FTileMapAssetTypeActions(EAssetTypeCategories::Type InAssetCategory);

	// IAssetTypeActions interface
	virtual FText GetName() const override;
	virtual FColor GetTypeColor() const override;
	virtual UClass* GetSupportedClass() const override;
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
	virtual uint32 GetCategories() override;
	// End of IAssetTypeActions interface

private:
	EAssetTypeCategories::Type MyAssetCategory;
};
