// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/TileMapEdModeToolkit.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "PaperTileSet.h"
#include "Widgets/Input/SButton.h"
#include "PaperTileMapComponent.h"
#include "PaperTileMap.h"

#include "Toolkits/AssetEditorToolkit.h"
#include "TileSetEditor/TileSetSelectorViewport.h"
#include "WorkflowOrientedApp/SContentReference.h"
#include "TileMapEditing/TileMapEditorCommands.h"
#include "SAssetDropTarget.h"

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// FTileMapEdModeToolkit

FTileMapEdModeToolkit::FTileMapEdModeToolkit(class FEdModeTileMap* InOwningMode)
{
	TileMapEditor = InOwningMode;
}


FName FTileMapEdModeToolkit::GetToolkitFName() const
{
	return FName("TileMapToolkit");
}

FText FTileMapEdModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT( "TileMapAppLabel", "Tile Map Editor" );
}

FText FTileMapEdModeToolkit::GetToolkitName() const
{
	if (CurrentTileSetPtr.IsValid())
	{
		return FText::FromString(CurrentTileSetPtr->GetName());
	}
	return GetBaseToolkitName();
}

FText FTileMapEdModeToolkit::GetToolkitToolTipText() const
{
	if (CurrentTileSetPtr.IsValid())
	{
		return FAssetEditorToolkit::GetToolTipTextForObject(CurrentTileSetPtr.Get());
	}
	return GetBaseToolkitName();
}

FEdMode* FTileMapEdModeToolkit::GetEditorMode() const
{
	return TileMapEditor;
}

TSharedPtr<SWidget> FTileMapEdModeToolkit::GetInlineContent() const
{
	return MyWidget;
}

void FTileMapEdModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost)
{
	const float ContentRefWidth = 140.0f;

	BindCommands();

	// Try to determine a good default tile set based on the current selection set
	if (UPaperTileMapComponent* TargetComponent = TileMapEditor->FindSelectedComponent())
	{
		if (TargetComponent->TileMap != nullptr)
		{
			CurrentTileSetPtr = TargetComponent->TileMap->SelectedTileSet.Get();
		}
	}

	TileSetPalette = SNew(STileSetSelectorViewport, CurrentTileSetPtr.Get(), TileMapEditor);

	TSharedRef<SWidget> TileSetPaletteWidget = SNew(SOverlay)
		// The palette widget
		+SOverlay::Slot()
		[
			SNew(SAssetDropTarget)
			.OnAreAssetsAcceptableForDrop(this, &FTileMapEdModeToolkit::OnAssetDraggedOver)
			.OnAssetsDropped(this, &FTileMapEdModeToolkit::OnAssetsDropped)
			[
				TileSetPalette.ToSharedRef()
			]
		]
		// The no tile set selected warning text/button
		+SOverlay::Slot()
		.Padding(8.0f)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.Visibility(this, &FTileMapEdModeToolkit::GetTileSetPaletteCornerTextVisibility)
			.OnClicked(this, &FTileMapEdModeToolkit::ClickedOnTileSetPaletteCornerText)
			.Content()
			[
				SNew(STextBlock)
				.TextStyle(FPaperStyle::Get(), "TileMapEditor.TileSetPalette.NothingSelectedText")
				.Text(LOCTEXT("NoTileSetSelected", "Pick a tile set"))
				.ToolTipText(LOCTEXT("NoTileSetSelectedTooltip", "A tile set must be selected before painting the tile map.\nClick here to select one."))
			]
		];

	TileSetAssetReferenceWidget = SNew(SContentReference)
		.WidthOverride(ContentRefWidth)
		.AssetReference(this, &FTileMapEdModeToolkit::GetCurrentTileSet)
		.OnSetReference(this, &FTileMapEdModeToolkit::OnChangeTileSet)
		.AllowedClass(UPaperTileSet::StaticClass())
		.AllowSelectingNewAsset(true)
		.AllowClearingReference(false);

	// Create the contents of the editor mode toolkit
	MyWidget = 
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush( "ToolPanel.GroupBorder" ))
		.Content()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(4.0f)
			[
				BuildToolBar()
			]

			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.VAlign(VAlign_Fill)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CurrentTileSetAssetToPaintWith", "Active Tile Set"))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						TileSetAssetReferenceWidget.ToSharedRef()
					]
				]

				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				.VAlign(VAlign_Fill)
				.Padding(4.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					[
						TileSetPaletteWidget
					]
				]
			]
		];

	FModeToolkit::Init(InitToolkitHost);
}

void FTileMapEdModeToolkit::OnAssetsDropped(const FDragDropEvent&, TArrayView<FAssetData> InAssets)
{
	OnChangeTileSet(InAssets[0].GetAsset());
}

void FTileMapEdModeToolkit::OnChangeTileSet(UObject* NewAsset)
{
	if (UPaperTileSet* NewTileSet = Cast<UPaperTileSet>(NewAsset))
	{
		if (CurrentTileSetPtr.Get() != NewTileSet)
		{
			CurrentTileSetPtr = NewTileSet;
			if (TileSetPalette.IsValid())
			{
				TileSetPalette->ChangeTileSet(NewTileSet);
			}

			// Save the newly selected tile set in the asset so it can be restored next time we edit it
			if (UPaperTileMapComponent* TargetComponent = TileMapEditor->FindSelectedComponent())
			{
				if (TargetComponent->TileMap != nullptr)
				{
					TargetComponent->TileMap->SelectedTileSet = NewTileSet;
				}
			}
		}
	}
}

UObject* FTileMapEdModeToolkit::GetCurrentTileSet() const
{
	return CurrentTileSetPtr.Get();
}

void FTileMapEdModeToolkit::BindCommands()
{
	FTileMapEditorCommands::Register();
	const FTileMapEditorCommands& Commands = FTileMapEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.SelectPaintTool,
		FExecuteAction::CreateSP(this, &FTileMapEdModeToolkit::OnSelectTool, ETileMapEditorTool::Paintbrush),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTileMapEdModeToolkit::IsToolSelected, ETileMapEditorTool::Paintbrush) );
	ToolkitCommands->MapAction(
		Commands.SelectEraserTool,
		FExecuteAction::CreateSP(this, &FTileMapEdModeToolkit::OnSelectTool, ETileMapEditorTool::Eraser),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTileMapEdModeToolkit::IsToolSelected, ETileMapEditorTool::Eraser) );
	ToolkitCommands->MapAction(
		Commands.SelectFillTool,
		FExecuteAction::CreateSP(this, &FTileMapEdModeToolkit::OnSelectTool, ETileMapEditorTool::PaintBucket),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTileMapEdModeToolkit::IsToolSelected, ETileMapEditorTool::PaintBucket) );
	ToolkitCommands->MapAction(
		Commands.SelectEyeDropperTool,
		FExecuteAction::CreateSP(this, &FTileMapEdModeToolkit::OnSelectTool, ETileMapEditorTool::EyeDropper),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTileMapEdModeToolkit::IsToolSelected, ETileMapEditorTool::EyeDropper),
		FIsActionButtonVisible::CreateSP(this, &FTileMapEdModeToolkit::IsToolSelected, ETileMapEditorTool::EyeDropper));
	ToolkitCommands->MapAction(
		Commands.SelectTerrainTool,
		FExecuteAction::CreateSP(this, &FTileMapEdModeToolkit::OnSelectTool, ETileMapEditorTool::TerrainBrush),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTileMapEdModeToolkit::IsToolSelected, ETileMapEditorTool::TerrainBrush),
		FIsActionButtonVisible::CreateSP(this, &FTileMapEdModeToolkit::DoesSelectedTileSetHaveTerrains));

	// Selection actions
	ToolkitCommands->MapAction(
		Commands.FlipSelectionHorizontally,
		FExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::FlipSelectionHorizontally),
		FCanExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::HasValidSelection));
	ToolkitCommands->MapAction(
		Commands.FlipSelectionVertically,
		FExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::FlipSelectionVertically),
		FCanExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::HasValidSelection));
	ToolkitCommands->MapAction(
		Commands.RotateSelectionCW,
		FExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::RotateSelectionCW),
		FCanExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::HasValidSelection));
	ToolkitCommands->MapAction(
		Commands.RotateSelectionCCW,
		FExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::RotateSelectionCCW),
		FCanExecuteAction::CreateSP(TileMapEditor, &FEdModeTileMap::HasValidSelection));
}

void FTileMapEdModeToolkit::OnSelectTool(ETileMapEditorTool::Type NewTool)
{
	TileMapEditor->SetActiveTool(NewTool);
}

bool FTileMapEdModeToolkit::IsToolSelected(ETileMapEditorTool::Type QueryTool) const
{
	return (TileMapEditor->GetActiveTool() == QueryTool);
}

bool FTileMapEdModeToolkit::DoesSelectedTileSetHaveTerrains() const
{
	if (UPaperTileSet* CurrentTileSet = CurrentTileSetPtr.Get())
	{
		return CurrentTileSet->GetNumTerrains() > 0;
	}
	else
	{
		return false;
	}
}

TSharedRef<SWidget> FTileMapEdModeToolkit::BuildToolBar() const
{
	const FTileMapEditorCommands& Commands = FTileMapEditorCommands::Get();

	FToolBarBuilder SelectionFlipToolsToolbar(ToolkitCommands, FMultiBoxCustomization::None, TSharedPtr<FExtender>(), /*bForceSmallIcons=*/ true);
	{
		SelectionFlipToolsToolbar.AddToolBarButton(Commands.FlipSelectionHorizontally, NAME_None, LOCTEXT("FlipHorizontalShortLabel", "|X"));
		SelectionFlipToolsToolbar.AddToolBarButton(Commands.FlipSelectionVertically, NAME_None, LOCTEXT("FlipVerticalShortLabel", "|Y"));
		SelectionFlipToolsToolbar.AddToolBarButton(Commands.RotateSelectionCW, NAME_None, LOCTEXT("RotateClockwiseShortLabel", "CW"));
		SelectionFlipToolsToolbar.AddToolBarButton(Commands.RotateSelectionCCW, NAME_None, LOCTEXT("RotateCounterclockwiseShortLabel", "CCW"));
	}

	FToolBarBuilder ToolsToolbar(ToolkitCommands, FMultiBoxCustomization::None);
	{
		ToolsToolbar.AddToolBarButton(Commands.SelectEyeDropperTool);
		ToolsToolbar.AddToolBarButton(Commands.SelectPaintTool);
		ToolsToolbar.AddToolBarButton(Commands.SelectEraserTool);
		ToolsToolbar.AddToolBarButton(Commands.SelectFillTool);
		ToolsToolbar.AddToolBarButton(Commands.SelectTerrainTool);

		//@TODO: TileMapTerrain: Ugly styling
		FUIAction TerrainTypeDropdownAction;
		TerrainTypeDropdownAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateSP(this, &FTileMapEdModeToolkit::DoesSelectedTileSetHaveTerrains);
		ToolsToolbar.AddComboButton(TerrainTypeDropdownAction, FOnGetContent::CreateSP(const_cast<FTileMapEdModeToolkit*>(this), &FTileMapEdModeToolkit::GenerateTerrainMenu));
	}

	return
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
			[
				SelectionFlipToolsToolbar.MakeWidget()
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(4.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
			[
				ToolsToolbar.MakeWidget()
			]
		];
}

TSharedRef<SWidget> FTileMapEdModeToolkit::GenerateTerrainMenu()
{
	FMenuBuilder TerrainMenu(/*bInShouldCloseWindowAfterMenuSelection=*/ true, ToolkitCommands);

	if (UPaperTileSet* TileSet = CurrentTileSetPtr.Get())
	{
		const FText MenuHeading = FText::Format(LOCTEXT("TerrainMenu", "Terrain types for {0}"), FText::AsCultureInvariant(TileSet->GetName()));
		TerrainMenu.BeginSection(NAME_None, MenuHeading);

		for (int32 TerrainIndex = 0; TerrainIndex < TileSet->GetNumTerrains(); ++TerrainIndex)
		{
			FPaperTileSetTerrain TerrainInfo = TileSet->GetTerrain(TerrainIndex);

			const FText TerrainName = FText::AsCultureInvariant(TerrainInfo.TerrainName);
			const FText TerrainLabel = FText::Format(LOCTEXT("TerrainLabel", "Terrain '{0}'"), TerrainName);
			const FText TerrainTooltip = FText::Format(LOCTEXT("TerrainTooltip", "Change the active terrain brush type to '{0}'"), TerrainName);
			FUIAction TerrainSwitchAction(FExecuteAction::CreateSP(this, &FTileMapEdModeToolkit::SetTerrainBrush, TerrainIndex));

			TerrainMenu.AddMenuEntry(TerrainLabel, TerrainTooltip, FSlateIcon(), TerrainSwitchAction);
		}

		TerrainMenu.EndSection();
	}

	return TerrainMenu.MakeWidget();
}

void FTileMapEdModeToolkit::SetTerrainBrush(int32 NewTerrainTypeIndex)
{
	//@TODO: TileMapTerrain: Do something here...
	UE_LOGF(LogInit, Warning, "Set terrain brush to %d", NewTerrainTypeIndex);
}

EVisibility FTileMapEdModeToolkit::GetTileSetPaletteCornerTextVisibility() const
{
	return (GetCurrentTileSet() != nullptr) ? EVisibility::Collapsed : EVisibility::Visible;
}

FReply FTileMapEdModeToolkit::ClickedOnTileSetPaletteCornerText()
{
	TileSetAssetReferenceWidget->OpenAssetPickerMenu();

	return FReply::Handled();
}

bool FTileMapEdModeToolkit::OnAssetDraggedOver(TArrayView<FAssetData> InAssets) const
{
	return Cast<UPaperTileSet>(InAssets[0].GetAsset()) != nullptr;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
