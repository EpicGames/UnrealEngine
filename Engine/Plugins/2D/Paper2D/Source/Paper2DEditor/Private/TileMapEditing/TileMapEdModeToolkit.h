// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TileMapEditing/EdModeTileMap.h"
#include "Toolkits/BaseToolkit.h"

class SContentReference;
class STileSetSelectorViewport;

//////////////////////////////////////////////////////////////////////////
// FTileMapEdModeToolkit

class FTileMapEdModeToolkit : public FModeToolkit
{
public:
	FTileMapEdModeToolkit(class FEdModeTileMap* InOwningMode);

	// IToolkit interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual class FEdMode* GetEditorMode() const override;
	virtual TSharedPtr<class SWidget> GetInlineContent() const override;
	// End of IToolkit interface

	// FModeToolkit interface
	virtual void Init(const TSharedPtr<class IToolkitHost>& InitToolkitHost) override;
	// End of FModeToolkit interface

protected:
	void OnAssetsDropped(const FDragDropEvent&, TArrayView<FAssetData> InAssets);
	void OnChangeTileSet(UObject* NewAsset);
	UObject* GetCurrentTileSet() const;

	void BindCommands();

	void OnSelectTool(ETileMapEditorTool::Type NewTool);
	bool IsToolSelected(ETileMapEditorTool::Type QueryTool) const;

	bool DoesSelectedTileSetHaveTerrains() const;

	TSharedRef<SWidget> BuildToolBar() const;
	TSharedRef<SWidget> GenerateTerrainMenu();
	void SetTerrainBrush(int32 NewTerrainTypeIndex);

	EVisibility GetTileSetPaletteCornerTextVisibility() const;
	FReply ClickedOnTileSetPaletteCornerText();

	bool OnAssetDraggedOver(TArrayView<FAssetData> InAssets) const;

private:
	class FEdModeTileMap* TileMapEditor;

	TWeakObjectPtr<class UPaperTileSet> CurrentTileSetPtr;

	// All of the inline content for this toolkit
	TSharedPtr<class SWidget> MyWidget;

	// The tile set selector palette
	TSharedPtr<class STileSetSelectorViewport> TileSetPalette;

	// The tile set asset reference widget
	TSharedPtr<class SContentReference> TileSetAssetReferenceWidget;
};
