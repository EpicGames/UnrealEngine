// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/TileMapEditor.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Text/STextBlock.h"
#include "SEditorViewport.h"
#include "SSingleObjectDetailsPanel.h"
#include "PaperTileMap.h"


#include "TileMapEditing/TileMapEditorViewportClient.h"
#include "TileMapEditing/TileMapEditorCommands.h"
#include "TileMapEditing/STileMapEditorViewportToolbar.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "TileMapEditor"

//////////////////////////////////////////////////////////////////////////

const FName TileMapEditorAppName = FName(TEXT("TileMapEditorApp"));

//////////////////////////////////////////////////////////////////////////

struct FTileMapEditorTabs
{
	// Tab identifiers
	static const FName DetailsID;
	static const FName ViewportID;
	static const FName ToolboxHostID;
};

//////////////////////////////////////////////////////////////////////////

const FName FTileMapEditorTabs::DetailsID(TEXT("Details"));
const FName FTileMapEditorTabs::ViewportID(TEXT("Viewport"));
const FName FTileMapEditorTabs::ToolboxHostID(TEXT("Toolbox"));

//////////////////////////////////////////////////////////////////////////
// STileMapEditorViewport

class STileMapEditorViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(STileMapEditorViewport) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FTileMapEditor> InTileMapEditor);

	// SEditorViewport interface
	virtual void BindCommands() override;
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual EVisibility GetTransformToolbarVisibility() const override;
	virtual void OnFocusViewportToSelection() override;
	// End of SEditorViewport interface

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	// Invalidate any references to the tile map being edited; it has changed
	void NotifyTileMapBeingEditedHasChanged()
	{
		EditorViewportClient->NotifyTileMapBeingEditedHasChanged();
	}

	void ActivateEditMode()
	{
		EditorViewportClient->ActivateEditMode();
	}
private:
	// Pointer back to owning tile map editor instance (the keeper of state)
	TWeakPtr<class FTileMapEditor> TileMapEditorPtr;

	// Viewport client
	TSharedPtr<FTileMapEditorViewportClient> EditorViewportClient;
};

void STileMapEditorViewport::Construct(const FArguments& InArgs, TSharedPtr<FTileMapEditor> InTileMapEditor)
{
	TileMapEditorPtr = InTileMapEditor;

	SEditorViewport::Construct(SEditorViewport::FArguments());
}

void STileMapEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FTileMapEditorCommands& Commands = FTileMapEditorCommands::Get();

	TSharedRef<FTileMapEditorViewportClient> EditorViewportClientRef = EditorViewportClient.ToSharedRef();

	// Show toggles
	CommandList->MapAction(
		Commands.SetShowCollision,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FEditorViewportClient::SetShowCollision),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FEditorViewportClient::IsSetShowCollisionChecked));

	CommandList->MapAction(
		Commands.SetShowPivot,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::ToggleShowPivot),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::IsShowPivotChecked));

	CommandList->MapAction(
		Commands.SetShowTileGrid,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::ToggleShowTileGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::IsShowTileGridChecked));

	CommandList->MapAction(
		Commands.SetShowLayerGrid,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::ToggleShowLayerGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::IsShowLayerGridChecked));

	CommandList->MapAction(
		Commands.SetShowTileMapStats,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::ToggleShowTileMapStats),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FTileMapEditorViewportClient::IsShowTileMapStatsChecked));
}

TSharedRef<FEditorViewportClient> STileMapEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable(new FTileMapEditorViewportClient(TileMapEditorPtr, SharedThis(this)));

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> STileMapEditorViewport::BuildViewportToolbar()
{
	return SNew(STileMapEditorViewportToolbar, SharedThis(this));
}

EVisibility STileMapEditorViewport::GetTransformToolbarVisibility() const
{
	return EVisibility::Visible;
}

void STileMapEditorViewport::OnFocusViewportToSelection()
{
	EditorViewportClient->RequestFocusOnSelection(/*bInstant=*/ false);
}

TSharedRef<class SEditorViewport> STileMapEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> STileMapEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void STileMapEditorViewport::OnFloatingButtonClicked()
{
}

/////////////////////////////////////////////////////
// STileMapPropertiesTabBody

class STileMapPropertiesTabBody : public SSingleObjectDetailsPanel
{
public:
	SLATE_BEGIN_ARGS(STileMapPropertiesTabBody) {}
	SLATE_END_ARGS()

private:
	// Pointer back to owning TileMap editor instance (the keeper of state)
	TWeakPtr<class FTileMapEditor> TileMapEditorPtr;
public:
	void Construct(const FArguments& InArgs, TSharedPtr<FTileMapEditor> InTileMapEditor)
	{
		TileMapEditorPtr = InTileMapEditor;

		SSingleObjectDetailsPanel::Construct(SSingleObjectDetailsPanel::FArguments().HostCommandList(InTileMapEditor->GetToolkitCommands()).HostTabManager(InTileMapEditor->GetTabManager()), /*bAutoObserve=*/ true, /*bAllowSearch=*/ true);
	}

	// SSingleObjectDetailsPanel interface
	virtual UObject* GetObjectToObserve() const override
	{
		return TileMapEditorPtr.Pin()->GetTileMapBeingEdited();
	}

	virtual TSharedRef<SWidget> PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget) override
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1)
			[
				PropertyEditorWidget
			];
	}
	// End of SSingleObjectDetailsPanel interface
};

//////////////////////////////////////////////////////////////////////////
// FTileMapEditor

FTileMapEditor::FTileMapEditor()
	: TileMapBeingEdited(nullptr)
{
}

TSharedRef<SDockTab> FTileMapEditor::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("ViewportTab_Title", "Viewport"))
		[
			SNew(SOverlay)

			// The TileMap editor viewport
			+ SOverlay::Slot()
			[
				ViewportPtr.ToSharedRef()
			]

			// Bottom-right corner text indicating the preview nature of the tile map editor
			+ SOverlay::Slot()
				.Padding(10)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Visibility(EVisibility::HitTestInvisible)
					.TextStyle(FAppStyle::Get(), "Graph.CornerText")
					.Text(LOCTEXT("TileMapEditorViewportEarlyAccessPreviewWarning", "Beta preview"))
				]
		];
}

TSharedRef<SDockTab> FTileMapEditor::SpawnTab_ToolboxHost(const FSpawnTabArgs& Args)
{
	TSharedPtr<FTileMapEditor> TileMapEditorPtr = SharedThis(this);

	// Spawn the tab
	return SNew(SDockTab)
		.Label(LOCTEXT("ToolboxHost_Title", "Toolbox"))
		[
			ToolboxPtr.ToSharedRef()
		];
}

TSharedRef<SDockTab> FTileMapEditor::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<FTileMapEditor> TileMapEditorPtr = SharedThis(this);

	// Spawn the tab
	return SNew(SDockTab)
		.Label(LOCTEXT("DetailsTab_Title", "Details"))
		[
			SNew(STileMapPropertiesTabBody, TileMapEditorPtr)
		];
}

void FTileMapEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TileMapEditor", "Tile Map Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(FTileMapEditorTabs::ViewportID, FOnSpawnTab::CreateSP(this, &FTileMapEditor::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(FTileMapEditorTabs::ToolboxHostID, FOnSpawnTab::CreateSP(this, &FTileMapEditor::SpawnTab_ToolboxHost))
		.SetDisplayName(LOCTEXT("ToolboxHostLabel", "Toolbox"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Modes"));

	InTabManager->RegisterTabSpawner(FTileMapEditorTabs::DetailsID, FOnSpawnTab::CreateSP(this, &FTileMapEditor::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("DetailsTabLabel", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FTileMapEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FTileMapEditorTabs::ViewportID);
	InTabManager->UnregisterTabSpawner(FTileMapEditorTabs::ToolboxHostID);
	InTabManager->UnregisterTabSpawner(FTileMapEditorTabs::DetailsID);
}

void FTileMapEditor::InitTileMapEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UPaperTileMap* InitTileMap)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseOtherEditors(InitTileMap, this);
	TileMapBeingEdited = InitTileMap;

	FTileMapEditorCommands::Register();

	BindCommands();

	ViewportPtr = SNew(STileMapEditorViewport, SharedThis(this));
	ToolboxPtr = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0.f);

	// Default layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_TileMapEditor_Layout_v3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->SetHideTabWell(true)
					->AddTab(FTileMapEditorTabs::ToolboxHostID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell(true)
					->AddTab(FTileMapEditorTabs::ViewportID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.75f)
						->AddTab(FTileMapEditorTabs::DetailsID, ETabState::OpenedTab)
					)
				)
			)
		);

	// Initialize the asset editor and spawn the layout above
	InitAssetEditor(Mode, InitToolkitHost, TileMapEditorAppName, StandaloneDefaultLayout, /*bCreateDefaultStandaloneMenu=*/ true, /*bCreateDefaultToolbar=*/ true, InitTileMap);

	// Activate the edit mode
	ViewportPtr->ActivateEditMode();

	// Extend things
	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

void FTileMapEditor::BindCommands()
{
	// Commands would go here
}

FName FTileMapEditor::GetToolkitFName() const
{
	return FName("TileMapEditor");
}

FText FTileMapEditor::GetBaseToolkitName() const
{
	return LOCTEXT("TileMapEditorAppLabelBase", "Tile Map Editor");
}

FText FTileMapEditor::GetToolkitName() const
{
	const bool bDirtyState = TileMapBeingEdited->GetOutermost()->IsDirty();

	FFormatNamedArguments Args;
	Args.Add(TEXT("TileMapName"), FText::FromString(TileMapBeingEdited->GetName()));
	Args.Add(TEXT("DirtyState"), bDirtyState ? FText::FromString(TEXT("*")) : FText::GetEmpty());
	return FText::Format(LOCTEXT("TileMapEditorAppLabel", "{TileMapName}{DirtyState}"), Args);
}

FText FTileMapEditor::GetToolkitToolTipText() const
{
	return GetToolTipTextForObject(TileMapBeingEdited);
}

FString FTileMapEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("TileMapEditor");
}

FString FTileMapEditor::GetDocumentationLink() const
{
	return TEXT("AnimatingObjects/Paper2D/TileMaps");
}

void FTileMapEditor::OnToolkitHostingStarted(const TSharedRef< class IToolkit >& Toolkit)
{
	TSharedPtr<SWidget> InlineContent = Toolkit->GetInlineContent();
	if (InlineContent.IsValid())
	{
		ToolboxPtr->SetContent(InlineContent.ToSharedRef());
	}
}

void FTileMapEditor::OnToolkitHostingFinished(const TSharedRef< class IToolkit >& Toolkit)
{
	ToolboxPtr->SetContent(SNullWidget::NullWidget);

	//@TODO: MODETOOLS: How to handle multiple ed modes at once in a standalone asset editor?
#if 0
	bool FoundAnotherToolkit = false;
	const TArray< TSharedPtr< IToolkit > >& HostedToolkits = LevelEditor.Pin()->GetHostedToolkits();
	for (auto HostedToolkitIt = HostedToolkits.CreateConstIterator(); HostedToolkitIt; ++HostedToolkitIt)
	{
		if ((*HostedToolkitIt) != Toolkit)
		{
			UpdateInlineContent((*HostedToolkitIt)->GetInlineContent());
			FoundAnotherToolkit = true;
			break;
		}
	}

	if (!FoundAnotherToolkit)
	{
		UpdateInlineContent(SNullWidget::NullWidget);
	}

#endif
}

FLinearColor FTileMapEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

void FTileMapEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TileMapBeingEdited);
}

void FTileMapEditor::ExtendMenu()
{
}

void FTileMapEditor::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			const FTileMapEditorCommands& Commands = FTileMapEditorCommands::Get();
			ToolbarBuilder.AddToolBarButton(Commands.SetShowTileGrid);
			ToolbarBuilder.AddToolBarButton(Commands.SetShowLayerGrid);
			ToolbarBuilder.AddToolBarButton(Commands.SetShowTileMapStats);
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		ViewportPtr->GetCommandList(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar)
		);

	AddToolbarExtender(ToolbarExtender);
}

void FTileMapEditor::CreateEditorModeManager()
{
	check(ViewportPtr.IsValid());
	TSharedPtr<FEditorViewportClient> ViewportClient = ViewportPtr->GetViewportClient();
	check(ViewportClient.IsValid());
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewportClient->TakeOwnershipOfModeManager(EditorModeManager);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FTileMapEditor::SetTileMapBeingEdited(UPaperTileMap* NewTileMap)
{
	if ((NewTileMap != TileMapBeingEdited) && (NewTileMap != nullptr))
	{
		UPaperTileMap* OldTileMap = TileMapBeingEdited;
		TileMapBeingEdited = NewTileMap;

		// Let the viewport know that we are editing something different
		ViewportPtr->NotifyTileMapBeingEditedHasChanged();

		// Let the editor know that are editing something different
		RemoveEditingObject(OldTileMap);
		AddEditingObject(NewTileMap);
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
