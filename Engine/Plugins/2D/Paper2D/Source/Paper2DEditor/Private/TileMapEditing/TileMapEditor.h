// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class FSpawnTabArgs;

class STileMapEditorViewport;
class UPaperTileMap;

//////////////////////////////////////////////////////////////////////////
// FTileMapEditor

class FTileMapEditor : public FAssetEditorToolkit, public FGCObject
{
public:
	FTileMapEditor();

	// IToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	// End of IToolkit interface

	// FAssetEditorToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FString GetDocumentationLink() const override;
	virtual void OnToolkitHostingStarted(const TSharedRef< class IToolkit >& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef< class IToolkit >& Toolkit) override;
	// End of FAssetEditorToolkit

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTileMapEditor");
	}
	// End of FSerializableObject interface
public:
	void InitTileMapEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UPaperTileMap* InitTileMap);

	UPaperTileMap* GetTileMapBeingEdited() const { return TileMapBeingEdited; }
	void SetTileMapBeingEdited(UPaperTileMap* NewTileMap);
protected:
	TObjectPtr<UPaperTileMap> TileMapBeingEdited;
	TSharedPtr<class STileMapEditorViewport> ViewportPtr;
	TSharedPtr<class SBorder> ToolboxPtr;

protected:
	void BindCommands();
	void ExtendMenu();
	void ExtendToolbar();
	virtual void CreateEditorModeManager() override;

	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_ToolboxHost(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
};
