// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/TileMapEditorCommands.h"

//////////////////////////////////////////////////////////////////////////
// FTileMapEditorCommands

#define LOCTEXT_NAMESPACE "TileMapEditor"

void FTileMapEditorCommands::RegisterCommands()
{
	UI_COMMAND(EnterTileMapEditMode, "Enable Tile Map Mode", "Enables Tile Map editing mode", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SelectPaintTool, "Paint", "Paint", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::B));
	UI_COMMAND(SelectEraserTool, "Eraser", "Eraser", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::E));
	UI_COMMAND(SelectFillTool, "Fill", "Paint Bucket", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::G));
	UI_COMMAND(SelectEyeDropperTool, "Select", "Select already-painted tiles", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SelectTerrainTool, "Terrain", "Terrain", EUserInterfaceActionType::RadioButton, FInputChord());

	// Show toggles
	UI_COMMAND(SetShowCollision, "Collision", "Toggles display of the simplified collision mesh of the static mesh, if one has been assigned.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::C, EModifierKey::Alt));

	UI_COMMAND(SetShowPivot, "Pivot", "Display the pivot location of the sprite.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(SetShowTileGrid, "Tile Grid", "Display the tile grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowLayerGrid, "Layer Grid", "Display the layer grid.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(SetShowTileMapStats, "Stats", "Display render and collision stats about the tile map.", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	// Selection actions
	UI_COMMAND(FlipSelectionHorizontally, "Flip brush horizontally", "Flips the brush horizontally", EUserInterfaceActionType::Button, FInputChord(EKeys::X));
	UI_COMMAND(FlipSelectionVertically, "Flip brush vertically", "Flips the brush vertically", EUserInterfaceActionType::Button, FInputChord(EKeys::Y));
	UI_COMMAND(RotateSelectionCW, "Rotate brush clockwise", "Rotates the brush clockwise", EUserInterfaceActionType::Button, FInputChord(EKeys::Z));
	UI_COMMAND(RotateSelectionCCW, "Rotate brush counterclockwise", "Rotates the brush counterclockwise", EUserInterfaceActionType::Button, FInputChord(EKeys::Z, EModifierKey::Shift));

	// Layer actions
	UI_COMMAND(AddNewLayerAbove, "Add new layer", "Add a new layer above the current layer", EUserInterfaceActionType::Button, FInputChord(EKeys::N, EModifierKey::Control | EModifierKey::Shift));
	UI_COMMAND(AddNewLayerBelow, "Add new layer below", "Add a new layer below the current layer", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MergeLayerDown, "Merge layer down", "Merge the current layer down to the layer below it", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MoveLayerUp, "Bring forward", "Bring forward", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket, EModifierKey::Control));
	UI_COMMAND(MoveLayerDown, "Send backward", "Send backward", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket, EModifierKey::Control));
	UI_COMMAND(MoveLayerToTop, "Bring to front", "Bring to front", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket, EModifierKey::Shift | EModifierKey::Control));
	UI_COMMAND(MoveLayerToBottom, "Send to back", "Send to back", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket, EModifierKey::Shift | EModifierKey::Control));
	UI_COMMAND(SelectLayerAbove, "Select next layer", "Changes the active layer to the next layer", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket, EModifierKey::Alt));
	UI_COMMAND(SelectLayerBelow, "Select previous layer", "Changes the active layer to the previous layer", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket, EModifierKey::Alt));
}

#undef LOCTEXT_NAMESPACE
