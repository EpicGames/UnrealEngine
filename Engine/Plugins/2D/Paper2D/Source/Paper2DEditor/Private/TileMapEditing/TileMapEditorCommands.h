// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "PaperStyle.h"
#include "Styling/ISlateStyle.h"

class FTileMapEditorCommands : public TCommands<FTileMapEditorCommands>
{
public:
	FTileMapEditorCommands()
		: TCommands<FTileMapEditorCommands>(
			TEXT("TileMapEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "TileMapEditor", "Tile Map Editor"), // Localized context name for displaying
			NAME_None, // Parent
			FPaperStyle::Get()->GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:
	// Edit mode trigger commands
	TSharedPtr<FUICommandInfo> EnterTileMapEditMode;

	// Commands inside tile map editing mode
	TSharedPtr<FUICommandInfo> SelectPaintTool;
	TSharedPtr<FUICommandInfo> SelectEraserTool;
	TSharedPtr<FUICommandInfo> SelectFillTool;
	TSharedPtr<FUICommandInfo> SelectEyeDropperTool;
	TSharedPtr<FUICommandInfo> SelectTerrainTool;

	// Show toggles
	TSharedPtr<FUICommandInfo> SetShowCollision;
	
	TSharedPtr<FUICommandInfo> SetShowPivot;
	TSharedPtr<FUICommandInfo> SetShowTileGrid;
	TSharedPtr<FUICommandInfo> SetShowLayerGrid;

	TSharedPtr<FUICommandInfo> SetShowTileMapStats;

	// Selection actions
	TSharedPtr<FUICommandInfo> FlipSelectionHorizontally;
	TSharedPtr<FUICommandInfo> FlipSelectionVertically;
	TSharedPtr<FUICommandInfo> RotateSelectionCW;
	TSharedPtr<FUICommandInfo> RotateSelectionCCW;

	// Layer actions
	TSharedPtr<FUICommandInfo> AddNewLayerAbove;
	TSharedPtr<FUICommandInfo> AddNewLayerBelow;
	TSharedPtr<FUICommandInfo> DeleteLayer;
	TSharedPtr<FUICommandInfo> MergeLayerDown;
	TSharedPtr<FUICommandInfo> MoveLayerUp;
	TSharedPtr<FUICommandInfo> MoveLayerDown;
	TSharedPtr<FUICommandInfo> MoveLayerToTop;
	TSharedPtr<FUICommandInfo> MoveLayerToBottom;
	TSharedPtr<FUICommandInfo> SelectLayerAbove;
	TSharedPtr<FUICommandInfo> SelectLayerBelow;
};
