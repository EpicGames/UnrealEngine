// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/TileMapEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TileMapEditorSettings)

//////////////////////////////////////////////////////////////////////////
// UTileMapEditorSettings

UTileMapEditorSettings::UTileMapEditorSettings()
	: DefaultBackgroundColor(55, 55, 55)
	, bShowGridByDefault(false)
	, DefaultTileGridColor(150, 150, 150)
	, DefaultMultiTileGridColor(255, 0, 0)
	, DefaultMultiTileGridWidth(0)
	, DefaultMultiTileGridHeight(0)
	, DefaultMultiTileGridOffsetX(0)
	, DefaultMultiTileGridOffsetY(0)
	, DefaultLayerGridColor(255, 255, 0)
{
}

