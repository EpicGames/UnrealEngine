// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TileMapEditorSettings.generated.h"

// Settings for the Paper2D tile map editor
UCLASS(config=EditorPerProjectUserSettings)
class UTileMapEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UTileMapEditorSettings();

	/** Default background color for new tile map assets */
	UPROPERTY(config, EditAnywhere, Category=Background, meta=(HideAlphaChannel))
	FColor DefaultBackgroundColor;

	/** Should the grid be shown by default when the editor is opened? */
	UPROPERTY(config, EditAnywhere, Category=Grid)
	bool bShowGridByDefault;

	/** Default per tile grid color for new tile map assets */
	UPROPERTY(config, EditAnywhere, Category=Grid, meta=(HideAlphaChannel))
	FColor DefaultTileGridColor;

	/** Default multi tile grid color for new tile map assets */
	UPROPERTY(config, EditAnywhere, Category=Grid, meta=(HideAlphaChannel))
	FColor DefaultMultiTileGridColor;

	/** Default number of tiles the multi tile grid spans horizontally for new tile map assets. 0 removes vertical lines */
	UPROPERTY(config, EditAnywhere, Category=Grid)
	int32 DefaultMultiTileGridWidth;

	/** Default number of tiles the multi tile grid spans vertically for new tile map assets. 0 removes horizontal lines */
	UPROPERTY(config, EditAnywhere, Category=Grid)
	int32 DefaultMultiTileGridHeight;

	/** Default number of tiles the multi tile grid is shifted to the right for new tile map assets */
	UPROPERTY(config, EditAnywhere, Category=Grid)
	int32 DefaultMultiTileGridOffsetX;

	/** Default number of tiles the multi tile grid is shifted downwards for new tile map assets */
	UPROPERTY(config, EditAnywhere, Category=Grid)
	int32 DefaultMultiTileGridOffsetY;

	/** Default layer grid color for new tile map assets */
	UPROPERTY(config, EditAnywhere, Category=Grid, meta=(HideAlphaChannel))
	FColor DefaultLayerGridColor;
};
