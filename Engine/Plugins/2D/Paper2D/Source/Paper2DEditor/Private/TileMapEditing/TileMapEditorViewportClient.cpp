// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileMapEditing/TileMapEditorViewportClient.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "PaperTileMapComponent.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "CanvasItem.h"
#include "AssetEditorModeManager.h"
#include "Selection.h"
#include "TileMapEditing/EdModeTileMap.h"
#include "PaperEditorShared/SpriteGeometryEditMode.h"
#include "PaperTileMap.h"

#include "ComponentReregisterContext.h"
#include "CanvasTypes.h"
#include "TileMapEditing/TileMapEditorSettings.h"
#include "UnrealWidget.h"

#define LOCTEXT_NAMESPACE "TileMapEditor"

//////////////////////////////////////////////////////////////////////////
// FTileMapEditorViewportClient

FTileMapEditorViewportClient::FTileMapEditorViewportClient(TWeakPtr<FTileMapEditor> InTileMapEditor, TWeakPtr<class STileMapEditorViewport> InTileMapEditorViewportPtr)
	: TileMapEditorPtr(InTileMapEditor)
	, TileMapEditorViewportPtr(InTileMapEditorViewportPtr)
{
	// The tile map editor fully supports mode tools and isn't doing any incompatible stuff with the Widget
	Widget->SetUsesEditorModeTools(ModeTools.Get());

	check(TileMapEditorPtr.IsValid() && TileMapEditorViewportPtr.IsValid());

	PreviewScene = &OwnedPreviewScene;
	((FAssetEditorModeManager*)ModeTools.Get())->SetPreviewScene(PreviewScene);

	SetRealtime(true);

	WidgetMode = UE::Widget::WM_Translate;
	bManipulating = false;
	bManipulationDirtiedSomething = false;
	bShowTileMapStats = true;
	ScopedTransaction = nullptr;

	DrawHelper.bDrawGrid = GetDefault<UTileMapEditorSettings>()->bShowGridByDefault;
	DrawHelper.bDrawPivot = false;
	bShowPivot = false;

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetCompositeEditorPrimitives(true);

	// Create a render component for the tile map being edited
	{
		RenderTileMapComponent = NewObject<UPaperTileMapComponent>();
		UPaperTileMap* TileMap = GetTileMapBeingEdited();
		RenderTileMapComponent->TileMap = TileMap;

		PreviewScene->AddComponent(RenderTileMapComponent, FTransform::Identity);
	}

	// Select the render component
	ModeTools->GetSelectedObjects()->Select(RenderTileMapComponent);
}

void FTileMapEditorViewportClient::ActivateEditMode()
{
	// Activate the tile map edit mode
	ModeTools->SetDefaultMode(FEdModeTileMap::EM_TileMap);
	ModeTools->ActivateDefaultMode();
	
	//@TODO: Need to be able to register the widget in the toolbox panel with ToolkitHost, so it can instance the ed mode widgets into it
}

void FTileMapEditorViewportClient::DrawBoundsAsText(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, int32& YPos)
{
	FNumberFormattingOptions NoDigitGroupingFormat;
	NoDigitGroupingFormat.UseGrouping = false;

	UPaperTileMap* TileMap = GetTileMapBeingEdited();
	FBoxSphereBounds Bounds = TileMap->GetRenderBounds();

	const FText DisplaySizeText = FText::Format(LOCTEXT("BoundsSize", "Approx. Size: {0}x{1}x{2}"),
		FText::AsNumber((int32)(Bounds.BoxExtent.X * 2.0f), &NoDigitGroupingFormat),
		FText::AsNumber((int32)(Bounds.BoxExtent.Y * 2.0f), &NoDigitGroupingFormat),
		FText::AsNumber((int32)(Bounds.BoxExtent.Z * 2.0f), &NoDigitGroupingFormat));

	Canvas.DrawShadowedString(
		6,
		YPos,
		*DisplaySizeText.ToString(),
		GEngine->GetSmallFont(),
		FLinearColor::White);
	YPos += 18;
}

void FTileMapEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FPaperEditorViewportClient::Draw(View, PDI);

	if (bShowPivot)
	{
		FColor PivotColor = FColor::Red;
		float PivotSize = 0.02f;


		//@TODO: Merge this with FEditorCommonDrawHelper::DrawPivot, which needs to take the pivot position as an argument
		const FMatrix CameraToWorld = View->ViewMatrices.GetViewToWorld();

		const FVector PivLoc = FVector::ZeroVector;

		const float ZoomFactor = FMath::Min<float>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
		const float WidgetRadius = View->ViewMatrices.GetWorldToClip().TransformPosition(PivLoc).W * (PivotSize / ZoomFactor);

		const FVector CamX = CameraToWorld.TransformVector(FVector(1, 0, 0));
		const FVector CamY = CameraToWorld.TransformVector(FVector(0, 1, 0));

		PDI->DrawLine(PivLoc - (WidgetRadius*CamX), PivLoc + (WidgetRadius*CamX), PivotColor, SDPG_Foreground);
		PDI->DrawLine(PivLoc - (WidgetRadius*CamY), PivLoc + (WidgetRadius*CamY), PivotColor, SDPG_Foreground);
	}
}

void FTileMapEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);

	const bool bIsHitTesting = Canvas.IsHitTesting();
	if (!bIsHitTesting)
	{
		Canvas.SetHitProxy(nullptr);
	}

	if (!TileMapEditorPtr.IsValid())
	{
		return;
	}

	int32 YPos = 42;

	// Draw the stats (if enabled)
	if (bShowTileMapStats)
	{
		if (UPaperTileMap* TileMap = GetTileMapBeingEdited())
		{
			// Show baked collision stats
			if (TileMap->BodySetup != nullptr)
			{
				FSpriteGeometryEditMode::DrawCollisionStats(InViewport, View, Canvas, TileMap->BodySetup, /*inout*/ YPos);
			}
			else
			{
				FCanvasTextItem TextItem(FVector2D(6, YPos), LOCTEXT("NoCollisionDataMainScreen", "No collision data"), GEngine->GetSmallFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Draw(&Canvas);
				YPos += 18.0f;
			}


			// Show baked rendering stats
			int32 NumTriangles = 0;
			int32 NumBatches = 0;
			RenderTileMapComponent->GetRenderingStats(/*out*/ NumTriangles, /*out*/ NumBatches);

			{

				FCanvasTextItem TextItem(FVector2D(6, YPos), LOCTEXT("RenderGeomBaked", "Render Geometry (baked)"), GEngine->GetSmallFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);

				TextItem.Draw(&Canvas);
				TextItem.Position += FVector2D(6.0f, 18.0f);

				// Draw the number of batches
				TextItem.Text = FText::Format(LOCTEXT("SectionCount", "Sections: {0}"), FText::AsNumber(NumBatches));
				TextItem.Draw(&Canvas);
				TextItem.Position.Y += 18.0f;

				// Determine the material type
				//@TODO: Similar code happens in the sprite editor and sprite details panel, and should be consolidated if possible
				static const FText Opaque = LOCTEXT("OpaqueMaterial", "Opaque");
				static const FText Translucent = LOCTEXT("TranslucentMaterial", "Translucent");
				static const FText Masked = LOCTEXT("MaskedMaterial", "Masked");

				FText MaterialType = LOCTEXT("NoMaterial", "No material set!");
				if (TileMap->Material != nullptr)
				{
					if (IsOpaqueBlendMode(*TileMap->Material))
					{
						MaterialType = Opaque;
					}
					else if (IsMaskedBlendMode(*TileMap->Material))
					{
						MaterialType = Masked;
					}
					else
					{
						MaterialType = Translucent;
					}
				}

				// Draw the number of triangles
				TextItem.Text = FText::Format(LOCTEXT("TriangleCountAndMaterialBlendMode", "Triangles: {0} ({1})"), FText::AsNumber(NumTriangles), MaterialType);
				TextItem.Draw(&Canvas);
				TextItem.Position.Y += 18.0f;

				YPos = (int32)TextItem.Position.Y;
			}
		}

		// Draw the render bounds
		DrawBoundsAsText(InViewport, View, Canvas, /*inout*/ YPos);
	}
}

void FTileMapEditorViewportClient::Tick(float DeltaSeconds)
{
	FPaperEditorViewportClient::Tick(DeltaSeconds);

	OwnedPreviewScene.GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

FLinearColor FTileMapEditorViewportClient::GetBackgroundColor() const
{
	if (UPaperTileMap* TileMap = RenderTileMapComponent->TileMap)
	{
		return TileMap->BackgroundColor;
	}
	else
	{
		return GetDefault<UTileMapEditorSettings>()->DefaultBackgroundColor;
	}
}

void FTileMapEditorViewportClient::ToggleShowTileGrid()
{
	FComponentReregisterContext RefreshComponentHelper(RenderTileMapComponent);
	RenderTileMapComponent->bShowPerTileGridWhenUnselected = !RenderTileMapComponent->bShowPerTileGridWhenUnselected;
	Invalidate();
}

bool FTileMapEditorViewportClient::IsShowTileGridChecked() const
{
	return RenderTileMapComponent->bShowPerTileGridWhenUnselected;
}

void FTileMapEditorViewportClient::ToggleShowLayerGrid()
{
	FComponentReregisterContext RefreshComponentHelper(RenderTileMapComponent);
	RenderTileMapComponent->bShowPerLayerGridWhenUnselected = !RenderTileMapComponent->bShowPerLayerGridWhenUnselected;
	Invalidate();
}

bool FTileMapEditorViewportClient::IsShowLayerGridChecked() const
{
	return RenderTileMapComponent->bShowPerLayerGridWhenUnselected;
}

void FTileMapEditorViewportClient::ToggleShowMeshEdges()
{
	EngineShowFlags.SetMeshEdges(!EngineShowFlags.MeshEdges);
	Invalidate();
}

bool FTileMapEditorViewportClient::IsShowMeshEdgesChecked() const
{
	return EngineShowFlags.MeshEdges;
}

void FTileMapEditorViewportClient::ToggleShowTileMapStats()
{
	bShowTileMapStats = !bShowTileMapStats;
	Invalidate();
}

bool FTileMapEditorViewportClient::IsShowTileMapStatsChecked() const
{
	return bShowTileMapStats;
}

void FTileMapEditorViewportClient::EndTransaction()
{
	if (bManipulationDirtiedSomething)
	{
		RenderTileMapComponent->TileMap->PostEditChange();
	}
	
	bManipulationDirtiedSomething = false;

	if (ScopedTransaction != nullptr)
	{
		delete ScopedTransaction;
		ScopedTransaction = nullptr;
	}
}

void FTileMapEditorViewportClient::NotifyTileMapBeingEditedHasChanged()
{
	//@TODO: Ideally we do this before switching
	EndTransaction();

	// Update components to know about the new tile map being edited
	UPaperTileMap* TileMap = GetTileMapBeingEdited();
	RenderTileMapComponent->TileMap = TileMap;

	RequestFocusOnSelection(/*bInstant=*/ true);
}

FBox FTileMapEditorViewportClient::GetDesiredFocusBounds() const
{
	return RenderTileMapComponent->Bounds.GetBox();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
