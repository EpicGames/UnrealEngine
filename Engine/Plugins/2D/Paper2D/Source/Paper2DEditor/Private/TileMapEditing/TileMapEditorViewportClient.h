// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreviewScene.h"
#include "TileMapEditing/TileMapEditor.h"
#include "PaperEditorViewportClient.h"

class FCanvas;
class FScopedTransaction;
class STileMapEditorViewport;
class UPaperTileMap;
class UPaperTileMapComponent;

//////////////////////////////////////////////////////////////////////////
// FTileMapEditorViewportClient

class FTileMapEditorViewportClient : public FPaperEditorViewportClient
{
public:
	/** Constructor */
	FTileMapEditorViewportClient(TWeakPtr<FTileMapEditor> InTileMapEditor, TWeakPtr<STileMapEditorViewport> InTileMapEditorViewportPtr);

	// FViewportClient interface
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual void Tick(float DeltaSeconds) override;
	// End of FViewportClient interface

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	// End of FEditorViewportClient interface

	void ToggleShowPivot() { bShowPivot = !bShowPivot; Invalidate(); }
	bool IsShowPivotChecked() const { return bShowPivot; }

	void ToggleShowTileGrid();
	bool IsShowTileGridChecked() const;

	void ToggleShowLayerGrid();
	bool IsShowLayerGridChecked() const;

	void ToggleShowMeshEdges();
	bool IsShowMeshEdgesChecked() const;

	void ToggleShowTileMapStats();
	bool IsShowTileMapStatsChecked() const;

	//
	void FocusOnTileMap();

	// Invalidate any references to the tile map being edited; it has changed
	void NotifyTileMapBeingEditedHasChanged();

	// Note: Has to be delayed due to an unfortunate init ordering
	void ActivateEditMode();
private:
	// The preview scene
	FPreviewScene OwnedPreviewScene;

	// Tile map editor that owns this viewport
	TWeakPtr<FTileMapEditor> TileMapEditorPtr;

	// Render component for the tile map being edited
	UPaperTileMapComponent* RenderTileMapComponent;

	// Widget mode
	UE::Widget::EWidgetMode WidgetMode;

	// Are we currently manipulating something?
	bool bManipulating;

	// Did we dirty something during manipulation?
	bool bManipulationDirtiedSomething;

	// Are we showing tile map stats?
	bool bShowTileMapStats;

	// Pointer back to the tile map editor viewport control that owns us
	TWeakPtr<STileMapEditorViewport> TileMapEditorViewportPtr;

	// The current transaction for undo/redo
	FScopedTransaction* ScopedTransaction;

	// Should we show the sprite pivot?
	bool bShowPivot;

protected:
	// FPaperEditorViewportClient interface
	virtual FBox GetDesiredFocusBounds() const override;
	// End of FPaperEditorViewportClient interface

private:
	UPaperTileMap* GetTileMapBeingEdited() const
	{
		return TileMapEditorPtr.Pin()->GetTileMapBeingEdited();
	}

	void DrawBoundsAsText(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, int32& YPos);
	
	void BeginTransaction(const FText& SessionName);
	void EndTransaction();
};
