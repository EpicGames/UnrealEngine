// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetAssetTypeActions.h"
#include "Math/ColorList.h"
#include "Misc/PackageName.h"
#include "TileSetEditor.h"
#include "PaperTileSet.h"
#include "PaperTileMap.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "PaperTileMapFactory.h"
#include "TileSetEditor/TileSetEditorSettings.h"
#include "TileSetEditor/TileSheetPaddingFactory.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

//////////////////////////////////////////////////////////////////////////
// FTileSetAssetTypeActions

FTileSetAssetTypeActions::FTileSetAssetTypeActions(EAssetTypeCategories::Type InAssetCategory)
	: MyAssetCategory(InAssetCategory)
{
}

FText FTileSetAssetTypeActions::GetName() const
{
	return LOCTEXT("FTileSetAssetTypeActionsName", "Tile Set");
}

FColor FTileSetAssetTypeActions::GetTypeColor() const
{
	return FColorList::Orange;
}

UClass* FTileSetAssetTypeActions::GetSupportedClass() const
{
	return UPaperTileSet::StaticClass();
}

void FTileSetAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UPaperTileSet* TileSet = Cast<UPaperTileSet>(*ObjIt))
		{
			TSharedRef<FTileSetEditor> NewTileSetEditor(new FTileSetEditor());
			NewTileSetEditor->InitTileSetEditor(Mode, EditWithinLevelEditor, TileSet);
		}
	}
}

uint32 FTileSetAssetTypeActions::GetCategories()
{
	return MyAssetCategory;
}

void FTileSetAssetTypeActions::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	TArray<TWeakObjectPtr<UPaperTileSet>> TileSets = GetTypedWeakObjectPtrs<UPaperTileSet>(InObjects);

	if (TileSets.Num() == 1)
	{
		Section.AddMenuEntry(
			"TileSet_CreateTileMap",
			LOCTEXT("TileSet_CreateTileMap", "Create Tile Map"),
			LOCTEXT("TileSet_CreateTileMapTooltip", "Creates a tile map using the selected tile set as a guide for tile size, etc..."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PaperTileSet"),
			FUIAction(FExecuteAction::CreateSP(this, &FTileSetAssetTypeActions::ExecuteCreateTileMap, TileSets[0]))
			);

		Section.AddMenuEntry(
			"TileSet_ConditionTileSet",
			LOCTEXT("TileSet_ConditionTileSet", "Condition Tile Sheet Texture"),
			LOCTEXT("TileSet_ConditionTileSetTooltip", "Conditions the tile sheet texture for the selected tile set by duplicating tile edges to create a buffer zone around each tile"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Texture2D"),
			FUIAction(FExecuteAction::CreateSP(this, &FTileSetAssetTypeActions::ExecutePadTileSetTexture, TileSets[0]))
			);
	}
}

void FTileSetAssetTypeActions::ExecuteCreateTileMap(TWeakObjectPtr<UPaperTileSet> TileSetPtr)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SuffixToIgnore(TEXT("Set"));
	const FString TileMapSuffix(TEXT("Map"));

	if (UPaperTileSet* TileSet = TileSetPtr.Get())
	{
		// Figure out what to call the new tile map
		FString EffectiveTileSetName = TileSet->GetName();
		EffectiveTileSetName.RemoveFromEnd(SuffixToIgnore);

		const FString TileSetPathName = TileSet->GetOutermost()->GetPathName();
		const FString LongPackagePath = FPackageName::GetLongPackagePath(TileSetPathName);

		const FString NewTileMapDefaultPath = LongPackagePath / EffectiveTileSetName;

		// Make sure the name is unique
		FString AssetName;
		FString PackageName;
		AssetToolsModule.Get().CreateUniqueAssetName(NewTileMapDefaultPath, TileMapSuffix, /*out*/ PackageName, /*out*/ AssetName);
		const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

		// Create the new tile map
		UPaperTileMapFactory* TileMapFactory = NewObject<UPaperTileMapFactory>();
		TileMapFactory->InitialTileSet = TileSet;
		ContentBrowserModule.Get().CreateNewAsset(AssetName, PackagePath, UPaperTileMap::StaticClass(), TileMapFactory);
	}
}

void FTileSetAssetTypeActions::ExecutePadTileSetTexture(TWeakObjectPtr<UPaperTileSet> TileSetPtr)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const UTileSetEditorSettings* TileSetEditorSettings = GetDefault<UTileSetEditorSettings>();

	if (UPaperTileSet* TileSet = TileSetPtr.Get())
	{
		if (UTexture2D* TileSheetTexture = TileSet->GetTileSheetTexture())
		{
			const FString TileSheetSuffix(TEXT("Padded"));
			const FString TileSheetPathName = TileSheetTexture->GetOutermost()->GetPathName();
			const FString LongPackagePath = FPackageName::GetLongPackagePath(TileSheetPathName);

			const FString EffectiveTileSheetName = TileSheetTexture->GetName();
			const FString NewTileSheetDefaultPath = LongPackagePath / EffectiveTileSheetName;

			// Make sure the name is unique
			FString AssetName;
			FString PackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(NewTileSheetDefaultPath, TileSheetSuffix, /*out*/ PackageName, /*out*/ AssetName);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

			// Create the new tile sheet
			UTileSheetPaddingFactory* TileSheetPaddingFactory = NewObject<UTileSheetPaddingFactory>();
			TileSheetPaddingFactory->SourceTileSet = TileSet;
			TileSheetPaddingFactory->ExtrusionAmount = TileSetEditorSettings->ExtrusionAmount;
			TileSheetPaddingFactory->bPadToPowerOf2 = TileSetEditorSettings->bPadToPowerOf2;
			TileSheetPaddingFactory->bFillWithTransparentBlack = TileSetEditorSettings->bFillWithTransparentBlack;
			ContentBrowserModule.Get().CreateNewAsset(AssetName, PackagePath, UTexture::StaticClass(), TileSheetPaddingFactory);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
