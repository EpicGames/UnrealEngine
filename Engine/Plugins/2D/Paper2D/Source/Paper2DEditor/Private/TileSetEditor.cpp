// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor.h"

#include "EditorModeManager.h"
#include "Framework/Commands/UICommandList.h"
#include "PaperTileSet.h"
#include "SSingleObjectDetailsPanel.h"
#include "Widgets/Docking/SDockTab.h"
#include "TileSetEditor/TileSetSelectorViewport.h"
#include "TileSetEditor/SingleTileEditorViewport.h"
#include "TileSetEditor/SingleTileEditorViewportClient.h"
#include "TileSetEditor/TileSetEditorCommands.h"
#include "TileSetEditor/TileSetDetailsCustomization.h"
#include "PaperEditorShared/SpriteGeometryEditCommands.h"
#include "IDetailsView.h"

#define LOCTEXT_NAMESPACE "TileSetEditor"

//////////////////////////////////////////////////////////////////////////

const FName TileSetEditorAppName = FName(TEXT("TileSetEditorApp"));

//////////////////////////////////////////////////////////////////////////

struct FTileSetEditorTabs
{
	// Tab identifiers
	static const FName DetailsID;
	static const FName TextureViewID;
	static const FName SingleTileEditorID;
};

//////////////////////////////////////////////////////////////////////////

const FName FTileSetEditorTabs::DetailsID(TEXT("Details"));
const FName FTileSetEditorTabs::TextureViewID(TEXT("TextureCanvas"));
const FName FTileSetEditorTabs::SingleTileEditorID(TEXT("SingleTileEditor"));

/////////////////////////////////////////////////////
// STileSetPropertiesTabBody

class STileSetPropertiesTabBody : public SSingleObjectDetailsPanel
{
public:
	SLATE_BEGIN_ARGS(STileSetPropertiesTabBody) {}
	SLATE_END_ARGS()

private:
	TSharedRef<IDetailCustomization> MakeEmbeddedInstance()
	{
		TSharedRef<FTileSetDetailsCustomization> Customization = FTileSetDetailsCustomization::MakeEmbeddedInstance();
		CurrentCustomizationPtr = Customization;

		// Make sure the customization starts off looking at the right tile index
		TSharedPtr<FSingleTileEditorViewportClient> SingleTileEditor = TileSetEditorPtr.Pin()->GetSingleTileEditor();

		const int32 TileIndex = SingleTileEditor->GetTileIndex();
		if (TileIndex != INDEX_NONE)
		{
			Customization->OnTileIndexChanged(TileIndex, INDEX_NONE);
		}

		return Customization;
	}

	void OnTileIndexChanged(int32 NewIndex, int32 OldIndex)
	{
		if (FTileSetDetailsCustomization* CurrentCustomization = CurrentCustomizationPtr.Pin().Get())
		{
			CurrentCustomization->OnTileIndexChanged(NewIndex, OldIndex);
		}
	}

private:
	// Pointer back to owning TileSet editor instance (the keeper of state)
	TWeakPtr<class FTileSetEditor> TileSetEditorPtr;

	// Pointer to the allocated customization
	TWeakPtr<FTileSetDetailsCustomization> CurrentCustomizationPtr;

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FTileSetEditor> InTileSetEditor)
	{
		TileSetEditorPtr = InTileSetEditor;

		SSingleObjectDetailsPanel::Construct(SSingleObjectDetailsPanel::FArguments().HostCommandList(InTileSetEditor->GetToolkitCommands()).HostTabManager(InTileSetEditor->GetTabManager()));

		// Register for index change notifications
		TSharedPtr<FSingleTileEditorViewportClient> SingleTileEditor = TileSetEditorPtr.Pin()->GetSingleTileEditor();
		SingleTileEditor->GetOnSingleTileIndexChanged().AddSP(this, &STileSetPropertiesTabBody::OnTileIndexChanged);

		// Register our customization that will be notified of tile index changes
		FOnGetDetailCustomizationInstance LayoutTileSetDetails = FOnGetDetailCustomizationInstance::CreateSP(this, &STileSetPropertiesTabBody::MakeEmbeddedInstance);
		PropertyView->RegisterInstancedCustomPropertyLayout(UPaperTileSet::StaticClass(), LayoutTileSetDetails);
	}

	// SSingleObjectDetailsPanel interface
	virtual UObject* GetObjectToObserve() const override
	{
		return TileSetEditorPtr.Pin()->GetTileSetBeingEdited();
	}

	virtual TSharedRef<SWidget> PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget) override
	{
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				PropertyEditorWidget
			];
	}
	// End of SSingleObjectDetailsPanel interface
};

//////////////////////////////////////////////////////////////////////////
// FTileSetEditor

FTileSetEditor::FTileSetEditor()
	: TileSetBeingEdited(nullptr)
	, bUseAlternateLayout(false)
{
	// Register to be notified when properties are edited
	FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate OnPropertyChangedDelegate = FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate::CreateRaw(this, &FTileSetEditor::OnPropertyChanged);
	OnPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.Add(OnPropertyChangedDelegate);
}

FTileSetEditor::~FTileSetEditor()
{
	// Unregister the property modification handler
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnPropertyChangedHandle);
}

void FTileSetEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TileSetEditor", "Tile Set Editor"));
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(FTileSetEditorTabs::TextureViewID, FOnSpawnTab::CreateSP(this, &FTileSetEditor::SpawnTab_TextureView))
		.SetDisplayName(LOCTEXT("TextureViewTabMenu_Description", "Tile Set View"))
		.SetTooltipText(LOCTEXT("TextureViewTabMenu_ToolTip", "Shows the tile set viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(FTileSetEditorTabs::DetailsID, FOnSpawnTab::CreateSP(this, &FTileSetEditor::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("DetailsTabLabel", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FTileSetEditorTabs::SingleTileEditorID, FOnSpawnTab::CreateSP(this, &FTileSetEditor::SpawnTab_SingleTileEditor))
		.SetDisplayName(LOCTEXT("SingleTileEditTabMenu_Description", "Single Tile Editor"))
		.SetTooltipText(LOCTEXT("SingleTileEditTabMenu_ToolTip", "Shows the single tile editor viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
}

void FTileSetEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FTileSetEditorTabs::TextureViewID);
	InTabManager->UnregisterTabSpawner(FTileSetEditorTabs::DetailsID);
	InTabManager->UnregisterTabSpawner(FTileSetEditorTabs::SingleTileEditorID);
}

void FTileSetEditor::InitTileSetEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UPaperTileSet* InitTileSet)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseOtherEditors(InitTileSet, this);
	TileSetBeingEdited = InitTileSet;

	TileSetViewport = SNew(STileSetSelectorViewport, InitTileSet, /*EdMode=*/ nullptr);
	TileEditorViewportClient = MakeShareable(new FSingleTileEditorViewportClient(InitTileSet));
	TileSetViewport->GetTileSelectionChanged().AddRaw(TileEditorViewportClient.Get(), &FSingleTileEditorViewportClient::OnTileSelectionRegionChanged);

	TileEditorViewport = SNew(SSingleTileEditorViewport, TileEditorViewportClient);

	FTileSetEditorCommands::Register();
	FSpriteGeometryEditCommands::Register();

	BindCommands();
	CreateLayouts();

	// Initialize the asset editor
	TSharedRef<FTabManager::FLayout> StartupLayout = GetDesiredLayout();
	InitAssetEditor(Mode, InitToolkitHost, TileSetEditorAppName, StartupLayout, /*bCreateDefaultStandaloneMenu=*/ true, /*bCreateDefaultToolbar=*/ true, InitTileSet);
	
	TileEditorViewportClient->ActivateEditMode(TileEditorViewport->GetCommandList());
	if (FEditorModeTools* const ModeTools = TileEditorViewportClient->GetModeTools())
	{
		// Opt out of new ITF gizmos and viewport inputs for now
		ModeTools->SetSupportsViewportITF(false);
	}

	// Extend things
	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

TSharedRef<SDockTab> FTileSetEditor::SpawnTab_TextureView(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("TextureViewTabLabel", "Tile Set View"))
		[
			TileSetViewport.ToSharedRef()
		];
}

TSharedRef<SDockTab> FTileSetEditor::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<FTileSetEditor> TileSetEditorPtr = SharedThis(this);

	return SNew(SDockTab)
		.Label(LOCTEXT("DetailsTabLabel", "Details"))
		[
			SNew(STileSetPropertiesTabBody, TileSetEditorPtr)
		];
}

TSharedRef<SDockTab> FTileSetEditor::SpawnTab_SingleTileEditor(const FSpawnTabArgs& Args)
{
	TSharedPtr<FTileSetEditor> TileSetEditorPtr = SharedThis(this);

	return SNew(SDockTab)
		.Label(LOCTEXT("SingleTileEditTabLabel", "Single Tile Editor"))
		[
			TileEditorViewport.ToSharedRef()
		];
}

void FTileSetEditor::BindCommands()
{
	const TSharedRef<FUICommandList>& CommandList = GetToolkitCommands();
	
	FTileSetEditorCommands::Register();
	const FTileSetEditorCommands& Commands = FTileSetEditorCommands::Get();
	
	CommandList->MapAction(
		Commands.SwapTileSetEditorViewports,
		FExecuteAction::CreateSP(this, &FTileSetEditor::ToggleActiveLayout));
}

void FTileSetEditor::ExtendMenu()
{
}

void FTileSetEditor::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("TileHighlights");
			{
				ToolbarBuilder.AddToolBarButton(FTileSetEditorCommands::Get().SetShowTilesWithCollision);
				ToolbarBuilder.AddToolBarButton(FTileSetEditorCommands::Get().SetShowTilesWithMetaData);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("Actions");
			{
				ToolbarBuilder.AddToolBarButton(FTileSetEditorCommands::Get().SetShowTileStats);
				ToolbarBuilder.AddToolBarButton(FTileSetEditorCommands::Get().ApplyCollisionEdits);
				ToolbarBuilder.AddToolBarButton(FTileSetEditorCommands::Get().SwapTileSetEditorViewports);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("Tools");
			{
				ToolbarBuilder.AddToolBarButton(FSpriteGeometryEditCommands::Get().AddBoxShape);
				ToolbarBuilder.AddToolBarButton(FSpriteGeometryEditCommands::Get().ToggleAddPolygonMode);
				ToolbarBuilder.AddToolBarButton(FSpriteGeometryEditCommands::Get().AddCircleShape);
				ToolbarBuilder.AddToolBarButton(FSpriteGeometryEditCommands::Get().SnapAllVertices);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolkitCommands->Append(TileEditorViewport->GetCommandList().ToSharedRef());
	ToolkitCommands->Append(TileSetViewport->GetCommandList().ToSharedRef());

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		ToolkitCommands,
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar)
		);

	AddToolbarExtender(ToolbarExtender);
}

void FTileSetEditor::CreateEditorModeManager()
{
	check(TileEditorViewportClient.IsValid());
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TileEditorViewportClient->TakeOwnershipOfModeManager(EditorModeManager);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FTileSetEditor::OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ObjectBeingModified == TileSetBeingEdited)
	{
		TileEditorViewportClient->SetTileIndex(TileEditorViewportClient->GetTileIndex());
		TileSetViewport->RefreshSelectionRectangle();
	}
}

void FTileSetEditor::CreateLayouts()
{
	// Default layout
	TileSelectorPreferredLayout = FTabManager::NewLayout("Standalone_TileSetEditor_Layout_v5")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->AddTab(FTileSetEditorTabs::TextureViewID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.4f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(FTileSetEditorTabs::SingleTileEditorID, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(true)
						->AddTab(FTileSetEditorTabs::DetailsID, ETabState::OpenedTab)
					)
				)
			)
		);

	// Alternate layout
	SingleTileEditorPreferredLayout = FTabManager::NewLayout("Standalone_TileSetEditor_AlternateLayout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->AddTab(FTileSetEditorTabs::SingleTileEditorID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.4f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(FTileSetEditorTabs::TextureViewID, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(true)
						->AddTab(FTileSetEditorTabs::DetailsID, ETabState::OpenedTab)
					)
				)
			)
		);
}

void FTileSetEditor::ToggleActiveLayout()
{
	// Save the existing layout
	FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, GetTabManager()->PersistLayout());

	// Switch and load the new layout
	bUseAlternateLayout = !bUseAlternateLayout;

	TSharedRef<FTabManager::FLayout> NewLayout = GetDesiredLayout();
	FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, NewLayout);

	// Activate the new layout
	RestoreFromLayout(NewLayout);
}

TSharedRef<FTabManager::FLayout> FTileSetEditor::GetDesiredLayout() const
{
	return bUseAlternateLayout ? SingleTileEditorPreferredLayout.ToSharedRef() : TileSelectorPreferredLayout.ToSharedRef();
}

FName FTileSetEditor::GetToolkitFName() const
{
	return FName("TileSetEditor");
}

FText FTileSetEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "TileSet Editor" );
}

FText FTileSetEditor::GetToolkitName() const
{
	return FText::FromString(TileSetBeingEdited->GetName());
}

FText FTileSetEditor::GetToolkitToolTipText() const
{
	return FAssetEditorToolkit::GetToolTipTextForObject(TileSetBeingEdited);
}

FString FTileSetEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("TileSetEditor");
}

FLinearColor FTileSetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

FString FTileSetEditor::GetDocumentationLink() const
{
	return TEXT("AnimatingObjects/Paper2D/TileMaps");
}

void FTileSetEditor::OnToolkitHostingStarted(const TSharedRef<class IToolkit>& Toolkit)
{
	//@TODO: MODETOOLS: Need to be able to register the widget in the toolbox panel with ToolkitHost, so it can instance the ed mode widgets into it
	// 	TSharedPtr<SWidget> InlineContent = Toolkit->GetInlineContent();
	// 	if (InlineContent.IsValid())
	// 	{
	// 		ToolboxPtr->SetContent(InlineContent.ToSharedRef());
	// 	}
}

void FTileSetEditor::OnToolkitHostingFinished(const TSharedRef<class IToolkit>& Toolkit)
{
	//ToolboxPtr->SetContent(SNullWidget::NullWidget);
	//@TODO: MODETOOLS: How to handle multiple ed modes at once in a standalone asset editor?
}

void FTileSetEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TileSetBeingEdited);
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
