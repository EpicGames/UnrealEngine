// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class FSingleTileEditorViewportClient;
class SSingleTileEditorViewport;
class STileSetSelectorViewport;
class UPaperTileSet;
struct FPropertyChangedEvent;

//////////////////////////////////////////////////////////////////////////
// FTileSetEditor

class FTileSetEditor : public FAssetEditorToolkit, public FGCObject
{
public:
	FTileSetEditor();
	~FTileSetEditor();

	// IToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	// End of IToolkit interface

	// FAssetEditorToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FString GetDocumentationLink() const override;
	virtual void OnToolkitHostingStarted(const TSharedRef<class IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<class IToolkit>& Toolkit) override;
	// End of FAssetEditorToolkit

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTileSetEditor");
	}
	// End of FSerializableObject interface

public:
	void InitTileSetEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UPaperTileSet* InitTileSet);

	UPaperTileSet* GetTileSetBeingEdited() const { return TileSetBeingEdited; }

	TSharedPtr<FSingleTileEditorViewportClient> GetSingleTileEditor() const { return TileEditorViewportClient; }

protected:
	TSharedRef<class SDockTab> SpawnTab_TextureView(const FSpawnTabArgs& Args);
	TSharedRef<class SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
	TSharedRef<class SDockTab> SpawnTab_SingleTileEditor(const FSpawnTabArgs& Args);

	void BindCommands();
	void ExtendMenu();
	void ExtendToolbar();
	virtual void CreateEditorModeManager() override;

	void OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	void CreateLayouts();
	void ToggleActiveLayout();

	TSharedRef<FTabManager::FLayout> GetDesiredLayout() const;
protected:
	TObjectPtr<UPaperTileSet> TileSetBeingEdited;

	TSharedPtr<STileSetSelectorViewport> TileSetViewport;
	TSharedPtr<SSingleTileEditorViewport> TileEditorViewport;
	TSharedPtr<FSingleTileEditorViewportClient> TileEditorViewportClient;

	FDelegateHandle OnPropertyChangedHandle;

	// Should we use the default layout or the alternate (single tile editor) layout?
	bool bUseAlternateLayout;

	// Layout with the tile selector large and on the left
	TSharedPtr<FTabManager::FLayout> TileSelectorPreferredLayout;

	// Layout with the single tile editor large and on the left
	TSharedPtr<FTabManager::FLayout> SingleTileEditorPreferredLayout;
};
