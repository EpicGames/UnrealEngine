// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/SingleTileEditorViewport.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "PaperEditorShared/SpriteGeometryEditCommands.h"
#include "TileSetEditor/SingleTileEditorViewportClient.h"
#include "TileSetEditor/TileSetEditorCommands.h"

#define LOCTEXT_NAMESPACE "TileSetEditor"

//////////////////////////////////////////////////////////////////////////
// STileSetEditorViewportToolbar

// In-viewport toolbar widget used in the tile set editor
class STileSetEditorViewportToolbar : public SCommonEditorViewportToolbarBase
{
public:
	SLATE_BEGIN_ARGS(STileSetEditorViewportToolbar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider);

	// SCommonEditorViewportToolbarBase interface
	virtual TSharedRef<SWidget> GenerateShowMenu() const override;
	// End of SCommonEditorViewportToolbarBase
};

void STileSetEditorViewportToolbar::Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider)
{
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InInfoProvider);
}

TSharedRef<SWidget> STileSetEditorViewportToolbar::GenerateShowMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();

	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		ShowMenuBuilder.AddMenuEntry(FSpriteGeometryEditCommands::Get().SetShowNormals);
		ShowMenuBuilder.AddMenuEntry(FTileSetEditorCommands::Get().SetShowGrid);
	}

	return ShowMenuBuilder.MakeWidget();
}

//////////////////////////////////////////////////////////////////////////
// SSingleTileEditorViewport

SSingleTileEditorViewport::~SSingleTileEditorViewport()
{
	TypedViewportClient.Reset();
}

void SSingleTileEditorViewport::Construct(const FArguments& InArgs, TSharedPtr<class FSingleTileEditorViewportClient> InViewportClient)
{
	TypedViewportClient = InViewportClient;

	SEditorViewport::Construct(SEditorViewport::FArguments());

	TSharedRef<SWidget> ParentContents = ChildSlot.GetWidget();

	this->ChildSlot
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			ParentContents
		]

		+SOverlay::Slot()
		.VAlign(VAlign_Bottom)
		[
			SNew(SBorder)
			.BorderImage(FPaperStyle::Get()->GetBrush("Paper2D.Common.ViewportTitleBackground"))
			.HAlign(HAlign_Fill)
			.Visibility(EVisibility::HitTestInvisible)
			[
				SNew(SVerticalBox)
				// Title text/icon
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.TextStyle(FPaperStyle::Get(), "Paper2D.Common.ViewportTitleTextStyle")
						.Text(this, &SSingleTileEditorViewport::GetTitleText)
					]
				]
			]
		]
	];
}

TSharedPtr<SWidget> SSingleTileEditorViewport::BuildViewportToolbar()
{
	return SNew(STileSetEditorViewportToolbar, SharedThis(this));
}

TSharedRef<FEditorViewportClient> SSingleTileEditorViewport::MakeEditorViewportClient()
{
	return TypedViewportClient.ToSharedRef();
}

void SSingleTileEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	FTileSetEditorCommands::Register();
	const FTileSetEditorCommands& Commands = FTileSetEditorCommands::Get();

	TSharedRef<FSingleTileEditorViewportClient> EditorViewportClientRef = TypedViewportClient.ToSharedRef();

	// Show toggles
	CommandList->MapAction(
		Commands.SetShowGrid,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FEditorViewportClient::SetShowGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FEditorViewportClient::IsSetShowGridChecked));

	CommandList->MapAction(
		Commands.SetShowTileStats,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSingleTileEditorViewportClient::ToggleShowStats),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FSingleTileEditorViewportClient::IsShowStatsChecked));

	// Collision commands
	CommandList->MapAction(
		Commands.ApplyCollisionEdits,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FSingleTileEditorViewportClient::ApplyCollisionGeometryEdits));
}

void SSingleTileEditorViewport::OnFocusViewportToSelection()
{
	TypedViewportClient->RequestFocusOnSelection(/*bInstant=*/ false);
}

TSharedRef<class SEditorViewport> SSingleTileEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SSingleTileEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SSingleTileEditorViewport::OnFloatingButtonClicked()
{
}

FText SSingleTileEditorViewport::GetTitleText() const
{
	const int32 CurrentTileIndex = TypedViewportClient->GetTileIndex();
	if (CurrentTileIndex != INDEX_NONE)
	{
		FNumberFormattingOptions NoGroupingFormat;
		NoGroupingFormat.SetUseGrouping(false);

		return FText::Format(LOCTEXT("SingleTileEditorViewportTitle", "Editing tile #{0}"), FText::AsNumber(CurrentTileIndex, &NoGroupingFormat));
	}
	else
	{
		return LOCTEXT("SingleTileEditorViewportTitle_NoTile", "Tile Editor - Select a tile");
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
