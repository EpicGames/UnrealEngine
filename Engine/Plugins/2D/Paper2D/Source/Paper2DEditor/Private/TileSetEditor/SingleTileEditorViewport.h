// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

class FEdModeTileMap;
class FSingleTileEditorViewportClient;

//////////////////////////////////////////////////////////////////////////
// SSingleTileEditorViewport

class SSingleTileEditorViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SSingleTileEditorViewport) {}
	SLATE_END_ARGS()

	~SSingleTileEditorViewport();

	void Construct(const FArguments& InArgs, TSharedPtr<FSingleTileEditorViewportClient> InViewportClient);

	// SEditorViewport interface
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	// End of SEditorViewport interface

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

protected:
	FText GetTitleText() const;

private:
	TSharedPtr<FSingleTileEditorViewportClient> TypedViewportClient;
	FEdModeTileMap* TileMapEditor;
};
