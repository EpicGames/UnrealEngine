// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/SingleTileEditorViewportClient.h"
#include "Materials/MaterialInterface.h"
#include "PaperTileSet.h"
#include "UObject/UObjectIterator.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "AssetEditorModeManager.h"
#include "PaperSpriteComponent.h"
#include "PaperTileMap.h"
#include "PaperEditorShared/SpriteGeometryEditMode.h"
#include "ScopedTransaction.h"
#include "TileSetEditor/TileSetEditorSettings.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "UnrealWidget.h"

#define LOCTEXT_NAMESPACE "TileSetEditor"

//////////////////////////////////////////////////////////////////////////
// FSingleTileEditorViewportClient

FSingleTileEditorViewportClient::FSingleTileEditorViewportClient(UPaperTileSet* InTileSet)
	: TileSet(InTileSet)
	, TileBeingEditedIndex(INDEX_NONE)
	, bManipulating(false)
	, bManipulationDirtiedSomething(false)
	, bShowStats(false)
	, ScopedTransaction(nullptr)
{
	//@TODO: Should be able to set this to false eventually
	SetRealtime(true);

	// The tile map editor fully supports mode tools and isn't doing any incompatible stuff with the Widget
	Widget->SetUsesEditorModeTools(ModeTools.Get());

	DrawHelper.bDrawGrid = GetDefault<UTileSetEditorSettings>()->bShowGridByDefault;
	DrawHelper.bDrawPivot = false;

	PreviewScene = &OwnedPreviewScene;
	((FAssetEditorModeManager*)ModeTools.Get())->SetPreviewScene(PreviewScene);

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetCompositeEditorPrimitives(true);

	// Create a render component for the tile preview
	PreviewTileSpriteComponent = NewObject<UPaperSpriteComponent>();

	FSoftObjectPath DefaultTranslucentMaterialName("/Paper2D/TranslucentUnlitSpriteMaterial.TranslucentUnlitSpriteMaterial");
	UMaterialInterface* TranslucentMaterial = Cast<UMaterialInterface>(DefaultTranslucentMaterialName.TryLoad());
	PreviewTileSpriteComponent->SetMaterial(0, TranslucentMaterial);

	PreviewScene->AddComponent(PreviewTileSpriteComponent, FTransform::Identity);
}

FBox FSingleTileEditorViewportClient::GetDesiredFocusBounds() const
{
	UPaperSpriteComponent* ComponentToFocusOn = PreviewTileSpriteComponent;
	return ComponentToFocusOn->Bounds.GetBox();
}

void FSingleTileEditorViewportClient::Tick(float DeltaSeconds)
{
	FPaperEditorViewportClient::Tick(DeltaSeconds);
	OwnedPreviewScene.GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

FLinearColor FSingleTileEditorViewportClient::GetBackgroundColor() const
{
	return TileSet->GetBackgroundColor();
}

void FSingleTileEditorViewportClient::TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge)
{
	//@TODO: Should push this into FEditorViewportClient
	// Begin transacting.  Give the current editor mode an opportunity to do the transacting.
	const bool bTrackingHandledExternally = ModeTools->StartTracking(this, Viewport);

	if (!bManipulating && bIsDragging && !bTrackingHandledExternally)
	{
		HandleBeginTransform();
	}
}

void FSingleTileEditorViewportClient::TrackingStopped()
{
	// Stop transacting.  Give the current editor mode an opportunity to do the transacting.
	const bool bTransactingHandledByEditorMode = ModeTools->EndTracking(this, Viewport);

	if (bManipulating && !bTransactingHandledByEditorMode)
	{
		HandleEndTransform();
	}
}

bool FSingleTileEditorViewportClient::BeginTransform(const FGizmoState& InState)
{
	return HandleBeginTransform();
}

bool FSingleTileEditorViewportClient::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform();
}

bool FSingleTileEditorViewportClient::OnTransformCanceled(const FGizmoState& InState)
{
	return HandleCancelTransform();
}

bool FSingleTileEditorViewportClient::HandleBeginTransform()
{
	if (!bManipulating)
	{
		BeginTransaction(LOCTEXT("ModificationInViewportTransaction", "Modification in Viewport"));
		bManipulating = true;
		bManipulationDirtiedSomething = false;
		return true;
	}
	return false;
}

bool FSingleTileEditorViewportClient::HandleEndTransform()
{
	if (bManipulating)
	{
		EndTransaction();
		bManipulating = false;
		return true;
	}
	return false;
}

bool FSingleTileEditorViewportClient::HandleCancelTransform()
{
	if (TileSet)
	{
		TileSet->PostEditChange();
	}
	return true;
}

void FSingleTileEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	const bool bIsHitTesting = Canvas.IsHitTesting();
	if (!bIsHitTesting)
	{
		Canvas.SetHitProxy(nullptr);
	}

	int32 YPos = 42;

	if (TileBeingEditedIndex != INDEX_NONE)
	{
		// Draw the collision geometry stats
		YPos += 60; //@TODO: Need a better way to determine this from the editor mode

		if (bShowStats)
		{
			bool bHasCollision = false;
			if (const FPaperTileMetadata* TileData = TileSet->GetTileMetadata(TileBeingEditedIndex))
			{
				if (TileData->HasCollision())
				{
					bHasCollision = true;
					FSpriteGeometryEditMode::DrawGeometryStats(InViewport, View, Canvas, TileData->CollisionData, false, /*inout*/ YPos);
				}
			}

			if (!bHasCollision)
			{
				FCanvasTextItem TextItem(FVector2D(6, YPos), LOCTEXT("NoCollisionDataMainScreen", "No collision data"), GEngine->GetSmallFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Draw(&Canvas);
			}
		}
	}

	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);
}

FVector2D FSingleTileEditorViewportClient::SelectedItemConvertWorldSpaceDeltaToLocalSpace(const FVector& WorldSpaceDelta) const
{
	const FVector ProjectionX = WorldSpaceDelta.ProjectOnTo(PaperAxisX);
	const FVector ProjectionY = WorldSpaceDelta.ProjectOnTo(PaperAxisY);

	const float XValue = FMath::Sign(ProjectionX | PaperAxisX) * ProjectionX.Size();
	const float YValue = FMath::Sign(ProjectionY | PaperAxisY) * ProjectionY.Size();

	return FVector2D(XValue, YValue);
}

FVector2D FSingleTileEditorViewportClient::WorldSpaceToTextureSpace(const FVector& SourcePoint) const
{
	const FVector ProjectionX = SourcePoint.ProjectOnTo(PaperAxisX);
	const FVector ProjectionY = -SourcePoint.ProjectOnTo(PaperAxisY);

	const float XValue = FMath::Sign(ProjectionX | PaperAxisX) * ProjectionX.Size();
	const float YValue = FMath::Sign(ProjectionY | PaperAxisY) * ProjectionY.Size();

	return FVector2D(XValue, YValue);
}

FVector FSingleTileEditorViewportClient::TextureSpaceToWorldSpace(const FVector2D& SourcePoint) const
{
	return (SourcePoint.X * PaperAxisX) - (SourcePoint.Y * PaperAxisY);
}

float FSingleTileEditorViewportClient::SelectedItemGetUnitsPerPixel() const
{
	return 1.0f;
}

void FSingleTileEditorViewportClient::BeginTransaction(const FText& SessionName)
{
	if (ScopedTransaction == nullptr)
	{
		ScopedTransaction = new FScopedTransaction(SessionName);

		TileSet->Modify();
	}
}

void FSingleTileEditorViewportClient::MarkTransactionAsDirty()
{
	bManipulationDirtiedSomething = true;
	Invalidate();
}

void FSingleTileEditorViewportClient::EndTransaction()
{
	if (bManipulationDirtiedSomething)
	{
		TileSet->PostEditChange();
	}

	bManipulationDirtiedSomething = false;

	if (ScopedTransaction != nullptr)
	{
		delete ScopedTransaction;
		ScopedTransaction = nullptr;
	}
}

void FSingleTileEditorViewportClient::InvalidateViewportAndHitProxies()
{
	Invalidate();
}

void FSingleTileEditorViewportClient::SetTileIndex(int32 InTileIndex)
{
	const int32 OldTileIndex = TileBeingEditedIndex;
	const bool bNewIndexValid = (InTileIndex >= 0) && (InTileIndex < TileSet->GetTileCount());
	TileBeingEditedIndex = bNewIndexValid ? InTileIndex : INDEX_NONE;

	FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
	check(GeometryEditMode);
	FSpriteGeometryCollection* GeomToEdit = nullptr;
	
	if (TileBeingEditedIndex != INDEX_NONE)
	{
		if (FPaperTileMetadata* Metadata = TileSet->GetMutableTileMetadata(InTileIndex))
		{
			GeomToEdit = &(Metadata->CollisionData);
		}
	}

	// Tell the geometry editor about the new tile (if it exists)
	GeometryEditMode->SetGeometryBeingEdited(GeomToEdit, /*bAllowCircles=*/ true, /*bAllowSubtractivePolygons=*/ false);

	// Update the visual representation
	UPaperSprite* DummySprite = nullptr;
	if (TileBeingEditedIndex != INDEX_NONE)
	{
		DummySprite = NewObject<UPaperSprite>();
 		DummySprite->SpriteCollisionDomain = ESpriteCollisionMode::None;
 		DummySprite->PivotMode = ESpritePivotMode::Center_Center;
 		DummySprite->CollisionGeometry.GeometryType = ESpritePolygonMode::SourceBoundingBox;
 		DummySprite->RenderGeometry.GeometryType = ESpritePolygonMode::SourceBoundingBox;

		FSpriteAssetInitParameters SpriteReinitParams;

		SpriteReinitParams.Texture = TileSet->GetTileSheetTexture();

		//@TODO: Should analyze the texture (*at a higher level, not per tile click!*) to pick the correct material
		FVector2D UV;
		TileSet->GetTileUV(TileBeingEditedIndex, /*out*/ UV);
		SpriteReinitParams.Offset = FIntPoint((int32)UV.X, (int32)(UV.Y));
		SpriteReinitParams.Dimension = TileSet->GetTileSize();
		SpriteReinitParams.SetPixelsPerUnrealUnit(1.0f);
		DummySprite->InitializeSprite(SpriteReinitParams);
	}
	PreviewTileSpriteComponent->SetSprite(DummySprite);

	// Update the default geometry bounds
	const FVector2D HalfTileSize(TileSet->GetTileSize().X * 0.5f, TileSet->GetTileSize().Y * 0.5f);
	FBox2D DesiredBounds(ForceInitToZero);
	DesiredBounds.Min = -HalfTileSize;
	DesiredBounds.Max = HalfTileSize;
	GeometryEditMode->SetNewGeometryPreferredBounds(DesiredBounds);

	// Zoom to fit when we start editing a tile and weren't before, but leave it alone if we just changed tiles, in case they zoomed in or out further
	if ((TileBeingEditedIndex != INDEX_NONE) && (OldTileIndex == INDEX_NONE))
	{
		RequestFocusOnSelection(/*bInstant=*/ true);
	}

	// Trigger a details panel customization rebuild
	OnSingleTileIndexChanged.Broadcast(TileBeingEditedIndex, OldTileIndex);

	// Redraw the viewport
	Invalidate();
}

void FSingleTileEditorViewportClient::OnTileSelectionRegionChanged(const FIntPoint& TopLeft, const FIntPoint& Dimensions)
{
	const int32 TopLeftIndex = (TileSet->GetTileCountX() * TopLeft.Y) + TopLeft.X;
	const int32 SelectionArea = Dimensions.X * Dimensions.Y;

	if (SelectionArea > 0)
	{
		const int32 NewIndex = (SelectionArea == 1) ? TopLeftIndex : INDEX_NONE;
		SetTileIndex(NewIndex);
	}
}

int32 FSingleTileEditorViewportClient::GetTileIndex() const
{
	return TileBeingEditedIndex;
}

void FSingleTileEditorViewportClient::ActivateEditMode(TSharedPtr<FUICommandList> InCommandList)
{
	// Activate the sprite geometry edit mode
	ModeTools->SetDefaultMode(FSpriteGeometryEditMode::EM_SpriteGeometry);
	ModeTools->ActivateDefaultMode();
	ModeTools->SetWidgetMode(UE::Widget::WM_Translate);

	FSpriteGeometryEditMode* GeometryEditMode = ModeTools->GetActiveModeTyped<FSpriteGeometryEditMode>(FSpriteGeometryEditMode::EM_SpriteGeometry);
	check(GeometryEditMode);
	GeometryEditMode->SetEditorContext(this);
	GeometryEditMode->BindCommands(InCommandList /*SpriteEditorViewportPtr.Pin()->GetCommandList()*/);

	const FLinearColor CollisionShapeColor(0.0f, 0.7f, 1.0f, 1.0f); //@TODO: Duplicated constant from SpriteEditingConstants
	GeometryEditMode->SetGeometryColors(CollisionShapeColor, FLinearColor::White);
}

void FSingleTileEditorViewportClient::ApplyCollisionGeometryEdits()
{
	bool bConditionedSomething = false;

	// See if anything needs to be conditioned
	const int32 NumTiles = TileSet->GetTileCount();
	for (int32 TileIndex = 0; TileIndex < NumTiles; ++TileIndex)
	{
		if (TileSet->GetTileMetadata(TileIndex) != nullptr)
		{
			FPaperTileMetadata* TileData = TileSet->GetMutableTileMetadata(TileIndex);
			check(TileData);

			if (TileData->HasCollision())
			{
				if (TileData->CollisionData.ConditionGeometry())
				{
					bConditionedSomething = true;
				}
			}
		}
	}

	if (bConditionedSomething)
	{
		TileSet->Modify();

		// Create and display a notification about the tile set being modified
		const FText NotificationText = FText::Format(LOCTEXT("NeedToSaveTileSet", "Optimized collision on one or more tiles.\n'{0}' needs to be saved."), FText::AsCultureInvariant(TileSet->GetName()));
		FNotificationInfo Info(NotificationText);
		Info.ExpireDuration = 2.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	// Apply changes to all tile maps that use this tile set
	for (TObjectIterator<UPaperTileMap> TileMapIt; TileMapIt; ++TileMapIt)
	{
		UPaperTileMap* TileMap = *TileMapIt;

		if (TileMap->UsesTileSet(TileSet))
		{
			TileMap->Modify();
			TileMap->PostEditChange();
		}
	}
}

void FSingleTileEditorViewportClient::ToggleShowStats()
{
	bShowStats = !bShowStats;
	Invalidate();
}

bool FSingleTileEditorViewportClient::IsShowStatsChecked() const
{
	return bShowStats;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
