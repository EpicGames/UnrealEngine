// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreviewScene.h"
#include "PaperEditorViewportClient.h"
#include "SpriteEditor/SpriteEditorSelections.h"

struct FInputEventState;

class FCanvas;
class FScopedTransaction;
class FUICommandList;
class UPaperSpriteComponent;
class UPaperTileSet;
struct FGizmoState;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSingleTileIndexChanged, int32 /*NewIndex*/, int32 /*OldIndex*/);

//////////////////////////////////////////////////////////////////////////
// FSingleTileEditorViewportClient

class FSingleTileEditorViewportClient : public FPaperEditorViewportClient, public ISpriteSelectionContext
{
public:
	FSingleTileEditorViewportClient(UPaperTileSet* InTileSet);

	// FViewportClient interface
	virtual void Tick(float DeltaSeconds) override;
	// End of FViewportClient interface

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void TrackingStarted(const FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
	virtual void TrackingStopped() override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	virtual bool OnTransformCanceled(const FGizmoState& InState) override;
	virtual void DrawCanvas(FViewport& Viewport, FSceneView& View, FCanvas& Canvas) override;
	// End of FEditorViewportClient interface

	// ISpriteSelectionContext interface
	virtual FVector2D SelectedItemConvertWorldSpaceDeltaToLocalSpace(const FVector& WorldSpaceDelta) const override;
	virtual FVector2D WorldSpaceToTextureSpace(const FVector& SourcePoint) const override;
	virtual FVector TextureSpaceToWorldSpace(const FVector2D& SourcePoint) const override;
	virtual float SelectedItemGetUnitsPerPixel() const override;
	virtual void BeginTransaction(const FText& SessionName) override;
	virtual void MarkTransactionAsDirty() override;
	virtual void EndTransaction() override;
	virtual void InvalidateViewportAndHitProxies() override;
	// End of ISpriteSelectionContext interface

	void SetTileIndex(int32 InTileIndex);
	int32 GetTileIndex() const;
	void OnTileSelectionRegionChanged(const FIntPoint& TopLeft, const FIntPoint& Dimensions);

	void ActivateEditMode(TSharedPtr<FUICommandList> InCommandList);

	void ApplyCollisionGeometryEdits();

	void ToggleShowStats();
	bool IsShowStatsChecked() const;

	// Delegate for when the index of the single tile being edited changes
	FOnSingleTileIndexChanged& GetOnSingleTileIndexChanged()
	{
		return OnSingleTileIndexChanged;
	}

protected:
	// FPaperEditorViewportClient interface
	virtual FBox GetDesiredFocusBounds() const override;
	// End of FPaperEditorViewportClient

private:
	// Tile set
	UPaperTileSet* TileSet;

	// The current tile being edited or INDEX_NONE
	int32 TileBeingEditedIndex;

	// Are we currently manipulating something?
	bool bManipulating;

	// Did we dirty something during manipulation?
	bool bManipulationDirtiedSomething;

	// Should we show stats for the tile?
	bool bShowStats;

	// Pointer back to the sprite editor viewport control that owns us
	TWeakPtr<SEditorViewport> SpriteEditorViewportPtr;

	// The current transaction for undo/redo
	FScopedTransaction* ScopedTransaction;

	// The preview scene
	FPreviewScene OwnedPreviewScene;

	// The preview sprite in the scene
	UPaperSpriteComponent* PreviewTileSpriteComponent;

	// Called when TileBeingEditedIndex changes
	FOnSingleTileIndexChanged OnSingleTileIndexChanged;
	
	bool HandleBeginTransform();
	bool HandleEndTransform();
	bool HandleCancelTransform();
};
