// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/TileSetDetailsCustomization.h"
#include "PaperTileSet.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"

#define LOCTEXT_NAMESPACE "Paper2D"

//////////////////////////////////////////////////////////////////////////
// FTileSetDetailsCustomization

FTileSetDetailsCustomization::FTileSetDetailsCustomization(bool bInIsEmbedded)
	: bIsEmbeddedInTileSetEditor(bInIsEmbedded)
	, SelectedSingleTileIndex(INDEX_NONE)
	, MyDetailLayout(nullptr)
{
}

TSharedRef<IDetailCustomization> FTileSetDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FTileSetDetailsCustomization(/*bIsEmbedded=*/ false));
}

TSharedRef<FTileSetDetailsCustomization> FTileSetDetailsCustomization::MakeEmbeddedInstance()
{
	return MakeShareable(new FTileSetDetailsCustomization(/*bIsEmbedded=*/ true));
}

void FTileSetDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	MyDetailLayout = &DetailLayout;
	
	for (const TWeakObjectPtr<UObject>& SelectedObject : DetailLayout.GetSelectedObjects())
	{
		if (UPaperTileSet* TileSet = Cast<UPaperTileSet>(SelectedObject.Get()))
		{
			TileSetPtr = TileSet;
			break;
		}
	}

 	IDetailCategoryBuilder& TileSetCategory = DetailLayout.EditCategory("TileSet", FText::GetEmpty());

	// Add the width and height in cells of this tile set to the header
	TileSetCategory.HeaderContent
	(
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(5.0f, 0.0f))
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("TinyText"))
				.Text(this, &FTileSetDetailsCustomization::GetCellDimensionHeaderText)
				.ColorAndOpacity(this, &FTileSetDetailsCustomization::GetCellDimensionHeaderColor)
				.ToolTipText(LOCTEXT("NumCellsTooltip", "Number of tile cells in this tile set"))
			]
		]
	);


	if (bIsEmbeddedInTileSetEditor)
	{
		// Hide the array to start with
		const FName MetadataArrayName = UPaperTileSet::GetPerTilePropertyName();
		TSharedPtr<IPropertyHandle> PerTileArrayProperty = DetailLayout.GetProperty(MetadataArrayName);
		DetailLayout.HideProperty(PerTileArrayProperty);
		// this array is potentially huge and has a costly validation overhead.  We only ever show one element in the array so there is no need to validate every element.
		PerTileArrayProperty->SetIgnoreValidation(true);

		if (SelectedSingleTileIndex != INDEX_NONE)
		{
			// Customize for the single tile being edited
			IDetailCategoryBuilder& SingleTileCategory = DetailLayout.EditCategory("SingleTileEditor", FText::GetEmpty());
			
			uint32 NumChildren;
			if ((PerTileArrayProperty->GetNumChildren(/*out*/ NumChildren) == FPropertyAccess::Success) && ((uint32)SelectedSingleTileIndex < NumChildren))
			{
				TSharedPtr<IPropertyHandle> OneTileEntry = PerTileArrayProperty->GetChildHandle(SelectedSingleTileIndex);
				SingleTileCategory.AddProperty(OneTileEntry)
					.ShouldAutoExpand(true);
			}

			// Add a display of the tile index being edited to the header
			const FText TileIndexHeaderText = FText::Format(LOCTEXT("SingleTileSectionHeader", "Editing Tile #{0}"), FText::AsNumber(SelectedSingleTileIndex));
			SingleTileCategory.HeaderContent
			(
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(FMargin(5.0f, 0.0f))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("TinyText"))
						.Text(TileIndexHeaderText)
					]
				]
			);
		}
	}
}

FText FTileSetDetailsCustomization::GetCellDimensionHeaderText() const
{
	FText Result;

	if (UPaperTileSet* TileSet = TileSetPtr.Get())
	{
		const int32 NumTilesX = TileSet->GetTileCountX();
		const int32 NumTilesY = TileSet->GetTileCountY();

		if (TileSet->GetTileSheetTexture() == nullptr)
		{
			Result = LOCTEXT("NoTexture", "No Tile Sheet");
		}
		else if (NumTilesX == 0)
		{
			Result = LOCTEXT("TextureTooNarrow", "Tile Sheet too narrow");
		}
		else if (NumTilesY == 0)
		{
			Result = LOCTEXT("TextureTooShort", "Tile Sheet too short");
		}
		else
		{
			const FText TextNumTilesX = FText::AsNumber(NumTilesX, &FNumberFormattingOptions::DefaultNoGrouping());
			const FText TextNumTilesY = FText::AsNumber(NumTilesY, &FNumberFormattingOptions::DefaultNoGrouping());
			Result = FText::Format(LOCTEXT("CellDimensions", "{0} x {1} tiles"), TextNumTilesX, TextNumTilesY);
		}
	}

	return Result;
}

FSlateColor FTileSetDetailsCustomization::GetCellDimensionHeaderColor() const
{
	if (UPaperTileSet* TileSet = TileSetPtr.Get())
	{
		if (TileSet->GetTileCount() == 0)
		{
			return FSlateColor(FLinearColor::Red);
		}
	}

	return FSlateColor::UseForeground();
}

void FTileSetDetailsCustomization::OnTileIndexChanged(int32 NewIndex, int32 OldIndex)
{
	SelectedSingleTileIndex = NewIndex;
	if (NewIndex != OldIndex)
	{
		if (MyDetailLayout != nullptr)
		{
			MyDetailLayout->ForceRefreshDetails();
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
