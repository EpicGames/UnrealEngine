// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

struct FSlateColor;

class IDetailLayoutBuilder;

//////////////////////////////////////////////////////////////////////////
// FTileSetDetailsCustomization

class FTileSetDetailsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	// Makes a new instance of this detail layout class for use inside of the tile set editor (with a route to the tile being edited)
	static TSharedRef<FTileSetDetailsCustomization> MakeEmbeddedInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

	// Called when the tile being edited index changes
	void OnTileIndexChanged(int32 NewIndex, int32 OldIndex);

private:
	// Are we embedded in the tile set editor?
	bool bIsEmbeddedInTileSetEditor;

	// Current index being viewed
	int32 SelectedSingleTileIndex;

	// The tile set being edited
	TWeakObjectPtr<class UPaperTileSet> TileSetPtr;

	// The detail layout builder that is using us
	IDetailLayoutBuilder* MyDetailLayout;

private:
	FTileSetDetailsCustomization(bool bInIsEmbedded);

	FText GetCellDimensionHeaderText() const;
	FSlateColor GetCellDimensionHeaderColor() const;
};
