// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/TileSetEditorCommands.h"

//////////////////////////////////////////////////////////////////////////
// FTileSetEditorCommands

#define LOCTEXT_NAMESPACE "TileSetEditor"

void FTileSetEditorCommands::RegisterCommands()
{
	// Show toggles
	UI_COMMAND(SetShowGrid, "Grid", "Display the grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowTileStats, "Stats", "Display statistics about the tile being edited.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(SetShowTilesWithCollision, "Colliding Tiles", "Toggles highlight of tiles that have custom collision geometry.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetShowTilesWithMetaData, "Metadata Tiles", "Toggles highlight of tiles that have custom metadata.", EUserInterfaceActionType::RadioButton, FInputChord());

	// Collision commands
	UI_COMMAND(ApplyCollisionEdits, "Refresh Maps", "Refreshes tile maps that use this tile set to ensure they have up-to-date collision geometry.", EUserInterfaceActionType::Button, FInputChord());

	// Editor mode switches
	UI_COMMAND(SwapTileSetEditorViewports, "Swap Views", "Switches the position of the 'single tile editor' and the 'tile selector' viewports.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
