// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "PaperStyle.h"
#include "Styling/ISlateStyle.h"

class FTileSetEditorCommands : public TCommands<FTileSetEditorCommands>
{
public:
	FTileSetEditorCommands()
		: TCommands<FTileSetEditorCommands>(
			TEXT("TileSetEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "TileSetEditor", "Tile Set Editor"), // Localized context name for displaying
			NAME_None, // Parent
			FPaperStyle::Get()->GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:
	// Show toggles
	TSharedPtr<FUICommandInfo> SetShowGrid;
	TSharedPtr<FUICommandInfo> SetShowTileStats;
	TSharedPtr<FUICommandInfo> SetShowTilesWithCollision;
	TSharedPtr<FUICommandInfo> SetShowTilesWithMetaData;

	// Collision commands
	TSharedPtr<FUICommandInfo> ApplyCollisionEdits;

	// Editor mode switches
	TSharedPtr<FUICommandInfo> SwapTileSetEditorViewports;
};
