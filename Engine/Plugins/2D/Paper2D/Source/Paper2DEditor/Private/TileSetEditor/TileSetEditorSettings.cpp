// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/TileSetEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TileSetEditorSettings)

//////////////////////////////////////////////////////////////////////////
// UTileSetEditorSettings

UTileSetEditorSettings::UTileSetEditorSettings()
	: DefaultBackgroundColor(0, 0, 127)
	, bShowGridByDefault(true)
	, ExtrusionAmount(2)
	, bPadToPowerOf2(true)
	, bFillWithTransparentBlack(true)
{
}

