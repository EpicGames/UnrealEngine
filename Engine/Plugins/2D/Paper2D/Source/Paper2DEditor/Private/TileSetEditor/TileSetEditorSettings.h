// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TileSetEditorSettings.generated.h"

// Settings for the Paper2D tile set editor
UCLASS(config=EditorPerProjectUserSettings)
class UTileSetEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UTileSetEditorSettings();

	// Default background color for new tile set assets
	UPROPERTY(config, EditAnywhere, Category=Background, meta=(HideAlphaChannel))
	FColor DefaultBackgroundColor;

	// Should the grid be shown by default when the editor is opened?
	UPROPERTY(config, EditAnywhere, Category="Tile Editor")
	bool bShowGridByDefault;

	// The amount to extrude out from the edge of each tile (in pixels)
	UPROPERTY(config, EditAnywhere, Category="Tile Sheet Conditioning", meta=(UIMin=1, ClampMin=1))
	int32 ExtrusionAmount;

	// Should we pad the conditioned texture to the next power of 2?
	UPROPERTY(config, EditAnywhere, Category="Tile Sheet Conditioning")
	bool bPadToPowerOf2;

	// Should we use transparent black or white when filling the texture areas that aren't covered by tiles?
	UPROPERTY(config, EditAnywhere, Category="Tile Sheet Conditioning")
	bool bFillWithTransparentBlack;
};
