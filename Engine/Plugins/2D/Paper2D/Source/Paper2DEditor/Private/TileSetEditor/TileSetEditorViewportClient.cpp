// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/TileSetEditorViewportClient.h"
#include "PaperTileSet.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "TextureCompiler.h"
#include "TextureResource.h"
#include "GlobalRenderResources.h"

#define LOCTEXT_NAMESPACE "TileSetEditor"

//////////////////////////////////////////////////////////////////////////
// FTileSetEditorViewportClient

FTileSetEditorViewportClient::FTileSetEditorViewportClient(UPaperTileSet* InTileSet)
	: TileSetBeingEdited(InTileSet)
	, bShowTilesWithCollision(false)
	, bShowTilesWithMetaData(false)
	, bHasValidPaintRectangle(false)
	, CurrentSelectedTileIndex(INDEX_NONE)
{
}

void FTileSetEditorViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	// Clear the viewport
	Canvas->Clear(GetBackgroundColor());

	// Can only proceed if we have a valid tile set
	UPaperTileSet* TileSet = TileSetBeingEdited.Get();
	if (TileSet == nullptr)
	{
		return;
	}

	if (UTexture2D* Texture = TileSet->GetTileSheetTexture())
	{
		FTextureCompilingManager::Get().FinishCompilation({ Texture });

		const bool bUseTranslucentBlend = Texture->HasAlphaChannel();

		// Fully stream in the texture before drawing it.
		Texture->SetForceMipLevelsToBeResident(30.0f);
		Texture->WaitForStreaming();

		FLinearColor TextureDrawColor = FLinearColor::White;

		{
			// Draw the tile sheet texture 
			const float XPos = -ZoomPos.X * ZoomAmount;
			const float YPos = -ZoomPos.Y * ZoomAmount;
			const float Width = Texture->GetSurfaceWidth() * ZoomAmount;
			const float Height = Texture->GetSurfaceHeight() * ZoomAmount;

			Canvas->DrawTile(XPos, YPos, Width, Height, 0.0f, 0.0f, 1.0f, 1.0f, TextureDrawColor, Texture->GetResource(), bUseTranslucentBlend);
		}


		const FLinearColor BorderRectangleColor(0.3f, 0.3f, 0.3f, 1.0f);
		{
			const FIntPoint TextureSize = Texture->GetImportedSize();
			const FIntMargin BorderSize = TileSet->GetMargin();
			const FIntRect TileSetRegion(BorderSize.Left, BorderSize.Top, TextureSize.X - BorderSize.Right, TextureSize.Y - BorderSize.Bottom);

			const float X = (TileSetRegion.Min.X - ZoomPos.X) * ZoomAmount;
			const float Y = (TileSetRegion.Min.Y - ZoomPos.Y) * ZoomAmount;
			const float W = TileSetRegion.Width() * ZoomAmount;
			const float H = TileSetRegion.Height() * ZoomAmount;

			FCanvasBoxItem BoxItem(FVector2D(X, Y), FVector2D(W, H));
			BoxItem.SetColor(BorderRectangleColor);
			Canvas->DrawItem(BoxItem);
		}

		if (bShowTilesWithCollision || bShowTilesWithMetaData)
		{
			// Draw an overlay rectangle on top of any tiles that have collision or metadata geometry
			const int32 NumTiles = TileSet->GetTileCount();

			const FLinearColor CollisionOverlayColor(0.0f, 0.7f, 1.0f, 0.5f);
			const FLinearColor MetaDataOverlayColor(1.0f, 0.2f, 0.0f, 0.5f);
			const FLinearColor InfoOverlayColor = bShowTilesWithCollision ? CollisionOverlayColor : MetaDataOverlayColor;

			const FIntPoint TileSetTileSize(TileSet->GetTileSize());
			const float Width = (TileSetTileSize.X - 2) * ZoomAmount;
			const float Height = (TileSetTileSize.Y - 2) * ZoomAmount;

			for (int32 TileIndex = 0; TileIndex < NumTiles; ++TileIndex)
			{
				if (const FPaperTileMetadata* TileMetadata = TileSet->GetTileMetadata(TileIndex))
				{
					const bool bShowDueToCollision = TileMetadata->HasCollision() && bShowTilesWithCollision;
					const bool bShowDueToMetaData = TileMetadata->HasMetaData() && bShowTilesWithMetaData;

					if (bShowDueToCollision || bShowDueToMetaData)
					{
						FVector2D TileUV;
						TileSet->GetTileUV(TileIndex, /*out*/ TileUV);

						const float XPos = (TileUV.X + 1 - ZoomPos.X) * ZoomAmount;
						const float YPos = (TileUV.Y + 1 - ZoomPos.Y) * ZoomAmount;

						Canvas->DrawTile(XPos, YPos, Width, Height, 0.0f, 0.0f, 1.0f, 1.0f, InfoOverlayColor, GWhiteTexture, /*bUseTranslucentBlend=*/ true);
					}
				}
			}
		}
	}

	// Overlay the selection rectangles
	DrawSelectionRectangles(InViewport, Canvas);

	if (bHasValidPaintRectangle)
	{
		const FViewportSelectionRectangle& Rect = ValidPaintRectangle;

		const float X = (Rect.TopLeft.X - ZoomPos.X) * ZoomAmount;
		const float Y = (Rect.TopLeft.Y - ZoomPos.Y) * ZoomAmount;
		const float W = Rect.Dimensions.X * ZoomAmount;
		const float H = Rect.Dimensions.Y * ZoomAmount;

		FCanvasBoxItem BoxItem(FVector2D(X, Y), FVector2D(W, H));
		BoxItem.SetColor(Rect.Color);
		Canvas->DrawItem(BoxItem);
	}

	if (CurrentSelectedTileIndex != INDEX_NONE)
	{
		const FString TileIndexString = FString::Printf(TEXT("Tile# %d"), CurrentSelectedTileIndex);

		int32 XL;
		int32 YL;
		StringSize(GEngine->GetLargeFont(), XL, YL, *TileIndexString);
		const float DrawX = 4.0f;
		const float DrawY = FMath::FloorToFloat(InViewport->GetSizeXY().Y - YL - 4.0f);
		Canvas->DrawShadowedString(DrawX, DrawY, *TileIndexString, GEngine->GetLargeFont(), FLinearColor::White);
	}
}

FLinearColor FTileSetEditorViewportClient::GetBackgroundColor() const
{
	if (UPaperTileSet* TileSet = TileSetBeingEdited.Get())
	{
		return TileSet->GetBackgroundColor();
	}
	else
	{
		return FEditorViewportClient::GetBackgroundColor();
	}
}

void FTileSetEditorViewportClient::ToggleShowTilesWithCollision()
{
	bShowTilesWithCollision = !bShowTilesWithCollision;
	bShowTilesWithMetaData = false;
}

bool FTileSetEditorViewportClient::IsShowTilesWithCollisionChecked() const
{
	return bShowTilesWithCollision;
}

void FTileSetEditorViewportClient::ToggleShowTilesWithMetaData()
{
	bShowTilesWithMetaData = !bShowTilesWithMetaData;
	bShowTilesWithCollision = false;
}

bool FTileSetEditorViewportClient::IsShowTilesWithMetaDataChecked() const
{
	return bShowTilesWithMetaData;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
