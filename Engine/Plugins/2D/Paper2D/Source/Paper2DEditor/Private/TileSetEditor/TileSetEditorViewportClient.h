// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperEditorViewportClient.h"

class FCanvas;
class UPaperTileSet;

//////////////////////////////////////////////////////////////////////////
// FTileSetEditorViewportClient

class FTileSetEditorViewportClient : public FPaperEditorViewportClient
{
public:
	FTileSetEditorViewportClient(UPaperTileSet* InTileSet);

	// FViewportClient interface
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	// End of FViewportClient interface

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	// End of FEditorViewportClient interface

	void ToggleShowTilesWithCollision();
	bool IsShowTilesWithCollisionChecked() const;

	void ToggleShowTilesWithMetaData();
	bool IsShowTilesWithMetaDataChecked() const;
public:
	// Tile set
	TWeakObjectPtr<UPaperTileSet> TileSetBeingEdited;

	bool bShowTilesWithCollision;
	bool bShowTilesWithMetaData;
	bool bHasValidPaintRectangle;
	FViewportSelectionRectangle ValidPaintRectangle;
	int32 CurrentSelectedTileIndex;
};
