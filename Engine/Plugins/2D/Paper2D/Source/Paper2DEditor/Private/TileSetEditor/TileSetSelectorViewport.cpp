// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/TileSetSelectorViewport.h"
#include "Framework/Commands/UICommandList.h"
#include "TileMapEditing/EdModeTileMap.h"
#include "PaperTileSet.h"
#include "Slate/SceneViewport.h"
#include "TileSetEditor/TileSetEditorCommands.h"
#include "TileSetEditor/TileSetEditorViewportClient.h"

#define LOCTEXT_NAMESPACE "TileSetEditor"

//////////////////////////////////////////////////////////////////////////
// STileSetSelectorViewport

STileSetSelectorViewport::~STileSetSelectorViewport()
{
	TypedViewportClient = nullptr;
}

void STileSetSelectorViewport::Construct(const FArguments& InArgs, UPaperTileSet* InTileSet, FEdModeTileMap* InTileMapEditor)
{
	bPendingZoom = true;

	SelectionTopLeft = FIntPoint::ZeroValue;
	SelectionDimensions = FIntPoint::ZeroValue;

	TileSetPtr = InTileSet;
	TileMapEditor = InTileMapEditor;

	TypedViewportClient = MakeShareable(new FTileSetEditorViewportClient(InTileSet));

	SPaperEditorViewport::Construct(
		SPaperEditorViewport::FArguments().OnSelectionChanged(this, &STileSetSelectorViewport::OnSelectionChanged),
		TypedViewportClient.ToSharedRef());

	// Make sure we get input instead of the viewport stealing it
	ViewportWidget->SetVisibility(EVisibility::HitTestInvisible);
}

void STileSetSelectorViewport::ChangeTileSet(UPaperTileSet* InTileSet)
{
	if (InTileSet != TileSetPtr.Get())
	{
		TileSetPtr = InTileSet;
		TypedViewportClient->TileSetBeingEdited = InTileSet;

		// Update the selection rectangle
		RefreshSelectionRectangle();
		TypedViewportClient->Invalidate();
	}
}

FText STileSetSelectorViewport::GetTitleText() const
{
	if (UPaperTileSet* TileSet = TileSetPtr.Get())
	{
		return FText::FromString(TileSet->GetName());
	}

	return LOCTEXT("TileSetSelectorTitle", "Tile Set Selector");
}

void STileSetSelectorViewport::BindCommands()
{
	SPaperEditorViewport::BindCommands();

	FTileSetEditorCommands::Register();
	const FTileSetEditorCommands& Commands = FTileSetEditorCommands::Get();

	TSharedRef<FTileSetEditorViewportClient> EditorViewportClientRef = TypedViewportClient.ToSharedRef();

	CommandList->MapAction(
		Commands.SetShowTilesWithCollision,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FTileSetEditorViewportClient::ToggleShowTilesWithCollision),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FTileSetEditorViewportClient::IsShowTilesWithCollisionChecked));

	CommandList->MapAction(
		Commands.SetShowTilesWithMetaData,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FTileSetEditorViewportClient::ToggleShowTilesWithMetaData),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FTileSetEditorViewportClient::IsShowTilesWithMetaDataChecked));
}

void STileSetSelectorViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bPendingZoom && (SceneViewport->GetSize().GetMin() > 0))
	{
		OnFocusViewportToSelection();
		bPendingZoom = false;
	}

	SPaperEditorViewport::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void STileSetSelectorViewport::OnFocusViewportToSelection()
{
	//@TODO: The rest of this method would get replaced by the following if we use a component to render the tile set
	//TypedViewportClient->RequestFocusOnSelection(/*bInstant=*/ false);

	if (UPaperTileSet* TileSetBeingEdited = TileSetPtr.Get())
	{
		const FIntPoint ViewportSize = SceneViewport->GetSize();
		if (UTexture* Texture = TileSetBeingEdited->GetTileSheetTexture())
		{
			const FIntPoint TextureSize(Texture->Source.GetSizeX(), Texture->Source.GetSizeY());

			const float CopiesInX = ViewportSize.X / (float)TextureSize.X;
			const float CopiesInY = ViewportSize.Y / (float)TextureSize.Y;

			const float NumCopies = FMath::Min<float>(CopiesInX, CopiesInY);

			// Find the zoom level that would match
			ZoomLevel = FindNearestZoomLevel(NumCopies, /*bRoundDown=*/ true);

			// Figure out the desired zoom position
			const float ZoomAmount = GetZoomAmount();
			ViewOffset.X = -(ViewportSize.X / ZoomAmount - TextureSize.X) * 0.5f;
			ViewOffset.Y = -(ViewportSize.Y / ZoomAmount - TextureSize.Y) * 0.5f;
		}
	}
}

void STileSetSelectorViewport::OnSelectionChanged(FMarqueeOperation InMarquee, bool bIsPreview)
{
	UPaperTileSet* TileSetBeingEdited = TileSetPtr.Get();
	if (TileSetBeingEdited == nullptr)
	{
		return;
	}

	const FVector2D TopLeftUnrounded = InMarquee.Rect.GetUpperLeft();
	const FVector2D BottomRightUnrounded = InMarquee.Rect.GetLowerRight();
	if ((TopLeftUnrounded != FVector2D::ZeroVector) || InMarquee.IsValid())
	{
		const int32 TileCountX = TileSetBeingEdited->GetTileCountX();
		const int32 TileCountY = TileSetBeingEdited->GetTileCountY();

		// Round down the top left corner
		const FIntPoint TileTopLeft = TileSetBeingEdited->GetTileXYFromTextureUV(TopLeftUnrounded, false);
		SelectionTopLeft = FIntPoint(FMath::Clamp<int32>(TileTopLeft.X, 0, TileCountX), FMath::Clamp<int32>(TileTopLeft.Y, 0, TileCountY));

		// Round up the bottom right corner
		const FIntPoint TileBottomRight = TileSetBeingEdited->GetTileXYFromTextureUV(BottomRightUnrounded, true);
		const FIntPoint SelectionBottomRight(FMath::Clamp<int32>(TileBottomRight.X, 0, TileCountX), FMath::Clamp<int32>(TileBottomRight.Y, 0, TileCountY));

		// Compute the new selection dimensions
		SelectionDimensions = SelectionBottomRight - SelectionTopLeft;
	}
	else
	{
		SelectionTopLeft.X = 0;
		SelectionTopLeft.Y = 0;
		SelectionDimensions.X = 0;
		SelectionDimensions.Y = 0;
	}

	OnTileSelectionChanged.Broadcast(SelectionTopLeft, SelectionDimensions);

	const bool bHasSelection = (SelectionDimensions.X * SelectionDimensions.Y) > 0;
	if (bIsPreview && bHasSelection)
	{
		if (TileMapEditor != nullptr)
		{
			TileMapEditor->SetActivePaint(TileSetBeingEdited, SelectionTopLeft, SelectionDimensions);

			// Switch to paint brush mode if we were in the eraser mode since the user is trying to select some ink to paint with
			if (TileMapEditor->GetActiveTool() == ETileMapEditorTool::Eraser)
			{
				TileMapEditor->SetActiveTool(ETileMapEditorTool::Paintbrush);
			}
		}

		RefreshSelectionRectangle();
	}
}

void STileSetSelectorViewport::RefreshSelectionRectangle()
{
	if (FTileSetEditorViewportClient* ViewportClient = TypedViewportClient.Get())
	{
		UPaperTileSet* TileSetBeingEdited = TileSetPtr.Get();

		const bool bHasSelection = (SelectionDimensions.X * SelectionDimensions.Y) > 0;
		ViewportClient->bHasValidPaintRectangle = bHasSelection && (TileSetBeingEdited != nullptr);

		const int32 TileIndex = (bHasSelection && (TileSetBeingEdited != nullptr)) ? (SelectionTopLeft.X + SelectionTopLeft.Y * TileSetBeingEdited->GetTileCountX()) : INDEX_NONE;
		ViewportClient->CurrentSelectedTileIndex = TileIndex;

		if (bHasSelection && (TileSetBeingEdited != nullptr))
		{
			ViewportClient->ValidPaintRectangle.Color = FLinearColor::White;

			const FIntPoint TopLeft = TileSetBeingEdited->GetTileUVFromTileXY(SelectionTopLeft);
			const FIntPoint BottomRight = TileSetBeingEdited->GetTileUVFromTileXY(SelectionTopLeft + SelectionDimensions - FIntPoint(1,1)) + TileSetBeingEdited->GetTileSize();
			ViewportClient->ValidPaintRectangle.Dimensions = BottomRight - TopLeft;
			ViewportClient->ValidPaintRectangle.TopLeft = TopLeft;
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
