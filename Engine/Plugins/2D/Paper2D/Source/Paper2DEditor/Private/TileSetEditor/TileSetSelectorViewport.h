// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPaperEditorViewport.h"

class UPaperTileSet;

struct FMarqueeOperation;

//////////////////////////////////////////////////////////////////////////
// STileSetSelectorViewport

class STileSetSelectorViewport : public SPaperEditorViewport
{
public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTileViewportSelectionChanged, const FIntPoint& /*TopLeft*/, const FIntPoint& /*Dimensions*/);

	SLATE_BEGIN_ARGS(STileSetSelectorViewport) {}
	SLATE_END_ARGS()

	~STileSetSelectorViewport();

	void Construct(const FArguments& InArgs, UPaperTileSet* InTileSet, class FEdModeTileMap* InTileMapEditor);

	void ChangeTileSet(UPaperTileSet* InTileSet);

	FOnTileViewportSelectionChanged& GetTileSelectionChanged()
	{
		return OnTileSelectionChanged;
	}

	void RefreshSelectionRectangle();

protected:
	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

	// SPaperEditorViewport interface
	virtual FText GetTitleText() const override;
	// End of SPaperEditorViewport interface

	// SEditorViewport interface
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	// End of SEditorViewport interface

private:
	void OnSelectionChanged(FMarqueeOperation Marquee, bool bIsPreview);

private:
	TWeakObjectPtr<class UPaperTileSet> TileSetPtr;
	TSharedPtr<class FTileSetEditorViewportClient> TypedViewportClient;
	class FEdModeTileMap* TileMapEditor;
	FIntPoint SelectionTopLeft;
	FIntPoint SelectionDimensions;
	bool bPendingZoom;

	FOnTileViewportSelectionChanged OnTileSelectionChanged;
};
