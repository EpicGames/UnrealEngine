// Copyright Epic Games, Inc. All Rights Reserved.

#include "TileSetEditor/TileSheetPaddingFactory.h"
#include "Paper2DEditorLog.h"
#include "PaperTileSet.h"
#include "PaperSpriteAtlas.h"
#include "Atlasing/PaperAtlasTextureHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TileSheetPaddingFactory)

#define LOCTEXT_NAMESPACE "Paper2D"

/////////////////////////////////////////////////////
// UTileSheetPaddingFactory

UTileSheetPaddingFactory::UTileSheetPaddingFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = UTexture::StaticClass();

	ExtrusionAmount = 2;
	bPadToPowerOf2 = true;
	bFillWithTransparentBlack = true;
}

UObject* UTileSheetPaddingFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(SourceTileSet);

	UTexture* SourceTexture = SourceTileSet->GetTileSheetTexture();
	check(SourceTexture);

	if (SourceTexture->Source.GetFormat() != TSF_BGRA8)
	{
		UE_LOGF(LogPaper2DEditor, Error, "Tile sheet texture '%ls' is not BGRA8, cannot create a padded texture from it", *SourceTexture->GetName());
		return nullptr;
	}

	const int32 NumTilesX = SourceTileSet->GetTileCountX();
	const int32 NumTilesY = SourceTileSet->GetTileCountY();
	const FIntPoint TileSize = SourceTileSet->GetTileSize();

	if ((NumTilesX <= 0) || (NumTilesY <= 0))
	{
		UE_LOGF(LogPaper2DEditor, Error, "Tile sheet texture '%ls' is too small to contain any tiles, cannot create a padded texture from it", *SourceTexture->GetName());
		return nullptr;
	}

	// Determine how big the new texture needs to be
	const uint32 NewMinTextureWidth = (uint32)(NumTilesX * (TileSize.X + 2 * ExtrusionAmount));
	const uint32 NewMinTextureHeight = (uint32)(NumTilesY * (TileSize.Y + 2 * ExtrusionAmount));

	const uint32 NewTextureWidth = bPadToPowerOf2 ? FMath::RoundUpToPowerOfTwo(NewMinTextureWidth) : NewMinTextureWidth;
	const uint32 NewTextureHeight = bPadToPowerOf2 ? FMath::RoundUpToPowerOfTwo(NewMinTextureHeight) : NewMinTextureHeight;

	UTexture2D* Result = NewObject<UTexture2D>(InParent, Name, Flags | RF_Transactional);

	//@TODO: Copy more state across (or start by duplicating the texture maybe?)
	Result->LODGroup = SourceTexture->LODGroup;
	Result->CompressionSettings = SourceTexture->CompressionSettings;
	Result->MipGenSettings = bPadToPowerOf2 ? TMGS_FromTextureGroup : TMGS_NoMipmaps;
	Result->DeferCompression = true;

	const uint32 TextureDataSize = NewTextureWidth * NewTextureHeight * sizeof(FColor);

	TArray<uint8> NewTextureData;
	NewTextureData.AddUninitialized(TextureDataSize);
	FMemory::Memset(NewTextureData.GetData(), bFillWithTransparentBlack ? 0x00 : 0xFF, TextureDataSize);

	for (int32 TileY = 0; TileY < NumTilesY; ++TileY)
	{
		for (int32 TileX = 0; TileX < NumTilesX; ++TileX)
		{
			const FIntPoint TileUV = SourceTileSet->GetTileUVFromTileXY(FIntPoint(TileX, TileY));

			TArray<uint8> DummyBuffer;
			FPaperAtlasTextureHelpers::ReadSpriteTexture(SourceTexture, TileUV, TileSize, DummyBuffer);

			FPaperSpriteAtlasSlot Slot;
			Slot.X = TileX * (TileSize.X + (2 * ExtrusionAmount));
			Slot.Y = TileY * (TileSize.Y + (2 * ExtrusionAmount));
			Slot.Width = TileSize.X;
			Slot.Height = TileSize.Y;
			Slot.AtlasIndex = 0;

			FPaperAtlasTextureHelpers::CopyTextureRegionToAtlasTextureData(NewTextureData, NewTextureWidth, NewTextureHeight, sizeof(FColor), EPaperSpriteAtlasPadding::DilateBorder, ExtrusionAmount, DummyBuffer, TileSize, Slot);
		}
	}

	Result->Source.Init(NewTextureWidth, NewTextureHeight, 1, 1, TSF_BGRA8, NewTextureData.GetData());

	Result->UpdateResource();
	Result->PostEditChange();

	// Figure out the margin (the right/bottom might be quite large due the the power-of-2 padding)
	const int32 ExcessWidth = (int32)(NewTextureWidth - NewMinTextureWidth);
	const int32 ExcessHeight = (int32)(NewTextureHeight - NewMinTextureHeight);
	FIntMargin BorderMargin(ExtrusionAmount, ExtrusionAmount, ExtrusionAmount + ExcessWidth, ExtrusionAmount + ExcessHeight);

	// Apply the new tile sheet to the specified tile set
	SourceTileSet->Modify();
	SourceTileSet->SetTileSheetTexture(Result);
	SourceTileSet->SetMargin(BorderMargin);
	SourceTileSet->SetPerTileSpacing(FIntPoint(2 * ExtrusionAmount, 2 * ExtrusionAmount));
	SourceTileSet->PostEditChange();

	return Result;
}

#undef LOCTEXT_NAMESPACE

