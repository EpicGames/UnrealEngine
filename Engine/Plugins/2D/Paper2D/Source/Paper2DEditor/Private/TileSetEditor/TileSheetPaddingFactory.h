// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

class UPaperTileSet;

#include "TileSheetPaddingFactory.generated.h"

/**
 * Factory used to pad out each individual tile in a tile sheet texture
 */

UCLASS()
class UTileSheetPaddingFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Source tile sheet texture
	UPROPERTY()
	TObjectPtr<UPaperTileSet> SourceTileSet;

	// The amount to extrude out from each tile (in pixels)
	UPROPERTY(Category = Settings, EditAnywhere, meta=(UIMin=1, ClampMin=1))
	int32 ExtrusionAmount;

	// Should we pad the texture to the next power of 2?
	UPROPERTY(Category=Settings, EditAnywhere)
	bool bPadToPowerOf2;

	// Should we use transparent black or white when filling the texture areas that aren't covered by tiles?
	UPROPERTY(Category=Settings, EditAnywhere)
	bool bFillWithTransparentBlack;

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};
