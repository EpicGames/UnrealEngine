// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

#define PAPER2D_EDITOR_MODULE_NAME "Paper2DEditor"

//////////////////////////////////////////////////////////////////////////
// IPaper2DEditorModule

class IPaper2DEditorModule : public IModuleInterface
{
public:
	virtual TSharedPtr<class FExtensibilityManager> GetSpriteEditorMenuExtensibilityManager() { return nullptr; }
	virtual TSharedPtr<class FExtensibilityManager> GetSpriteEditorToolBarExtensibilityManager() { return nullptr; }

	virtual TSharedPtr<class FExtensibilityManager> GetFlipbookEditorMenuExtensibilityManager() { return nullptr; }
	virtual TSharedPtr<class FExtensibilityManager> GetFlipbookEditorToolBarExtensibilityManager() { return nullptr; }

	virtual uint32 GetPaper2DAssetCategory() const = 0;
};
