// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

#define UE_API PAPER2DEDITOR_API

class UPaperSprite;

//////////////////////////////////////////////////////////////////////////
// FPaperFlipbookHelpers

class FPaperFlipbookHelpers
{
public:
	// Try and best guess split up the sprites into multiple flipbooks based on names
	// Numbers are naturally sorted
	// Eg. Walk01, Walk02, Run01, Run02 -> Walk (Walk01, Walk02), Run (Run01, Run02) if autoGroup is true
	// All ungrouped sprites are stored in one flipbook
	// Expected input: 
	// Sprites array contains valid sprites
	// SpriteNames contains an array of sprite names (parallel to Sprites)
	// or, an empty array if Sprite->GetName() is to be used.
	static UE_API void ExtractFlipbooksFromSprites(TMap<FString, TArray<UPaperSprite*> >& OutSpriteFlipbookMap, const TArray<UPaperSprite*>& Sprites, const TArray<FString>& SpriteNames);

private:

	static UE_API FString GetCleanerSpriteName(const FString& Name);
	static UE_API bool ExtractSpriteNumber(const FString& String, FString& BareString, int32& Number);

	FPaperFlipbookHelpers() {}
};

#undef UE_API
