// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

#define UE_API PAPER2DEDITOR_API

class FJsonValue;

//////////////////////////////////////////////////////////////////////////
// FPaperJSONHelpers

class FPaperJSONHelpers
{
public:
	// returns the value of Key in Item, or DefaultValue if the Key is missing or the wrong type
	static UE_API FString ReadString(TSharedPtr<class FJsonObject> Item, const FString& Key, const FString& DefaultValue);

	// Returns the object named Key or nullptr if it is missing or the wrong type
	static UE_API TSharedPtr<class FJsonObject> ReadObject(TSharedPtr<class FJsonObject> Item, const FString& Key);

	// Returns the array named Key or nullptr if it is missing or the wrong type
	static UE_API const TArray< TSharedPtr<FJsonValue> >& ReadArray(TSharedPtr<class FJsonObject> Item, const FString& Key);

	// Returns the bool named Key or bDefaultIfMissing if it is missing or the wrong type (note: no way to determine errors!)
	static UE_API bool ReadBoolean(const TSharedPtr<class FJsonObject> Item, const FString& Key, bool bDefaultIfMissing);

	// Returns true if the field named Key is a Number, populating Out_Value, or false if it is missing or the wrong type (Out_Value is set to 0.0f on failure)
	static UE_API bool ReadFloatNoDefault(const TSharedPtr<class FJsonObject> Item, const FString& Key, float& Out_Value);

	// Returns true if the field named Key is a Number, populating Out_Value, or false if it is missing or the wrong type (Out_Value is set to 0.0f on failure)
	static UE_API bool ReadFloatNoDefault(const TSharedPtr<class FJsonObject> Item, const FString& Key, double& Out_Value);

	// Returns true if the field named Key is a Number (truncating it to an integer), populating Out_Value, or false if it is missing or the wrong type (Out_Value is set to 0 on failure)
	static UE_API bool ReadIntegerNoDefault(const TSharedPtr<class FJsonObject> Item, const FString& Key, int32& Out_Value);

	// Returns true if the object named Key is a struct containing four integers (x,y,w,h), populating XY and WH with the values)
	static UE_API bool ReadRectangle(const TSharedPtr<class FJsonObject> Item, const FString& Key, FIntPoint& Out_XY, FIntPoint& Out_WH);

	// Returns true if the object named Key is a struct containing two floats (w,h), populating WH with the values)
	static UE_API bool ReadSize(const TSharedPtr<class FJsonObject> Item, const FString& Key, FVector2D& Out_WH);

	// Returns true if the object named Key is a struct containing two floats (x,y), populating WH with the values)
	static UE_API bool ReadXY(const TSharedPtr<class FJsonObject> Item, const FString& Key, FVector2D& Out_WH);

	// Returns true if the object named Key is a struct containing two floats (x,y), populating XY with the values)
	static UE_API bool ReadIntPoint(const TSharedPtr<class FJsonObject> Item, const FString& Key, FIntPoint& Out_XY);

private:
	FPaperJSONHelpers() {}
};

#undef UE_API
