// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PaperSpriteSheetImporter : ModuleRules
{
	public PaperSpriteSheetImporter(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Json",
				"Slate",
				"SlateCore",
				"Engine",
				"Paper2D",
				"UnrealEd",
				"Paper2DEditor",
				"AssetTools",
				"ContentBrowser",
				"ToolMenus",
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry"
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetRegistry"
			});
	}
}
