// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperJsonSpriteSheetImporter.h"
#include "PaperSpriteSheetImporterLog.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Serialization/JsonSerializer.h"
#include "EditorReimportHandler.h"
#include "PaperSprite.h"
#include "PaperJSONHelpers.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageTools.h"
#include "ObjectTools.h"
#include "PaperSpriteSheet.h"
#include "PaperImporterSettings.h"

TSharedPtr<FJsonObject> ParseJSON(const FString& FileContents, const FString& NameForErrors, bool bSilent)
{
	// Load the file up (JSON format)
	if (!FileContents.IsEmpty())
	{
		const TSharedRef< TJsonReader<> >& Reader = TJsonReaderFactory<>::Create(FileContents);

		TSharedPtr<FJsonObject> SpriteDescriptorObject;
		if (FJsonSerializer::Deserialize(Reader, SpriteDescriptorObject) && SpriteDescriptorObject.IsValid())
		{
			// File was loaded and deserialized OK!
			return SpriteDescriptorObject;
		}
		else
		{
			if (!bSilent)
			{
				UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to parse sprite descriptor file '%ls'.  Error: '%ls'", *NameForErrors, *Reader->GetErrorMessage());
			}
			return nullptr;
		}
	}
	else
	{
		if (!bSilent)
		{
			UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Sprite descriptor file '%ls' was empty.  This sprite cannot be imported.", *NameForErrors);
		}
		return nullptr;
	}
}

TSharedPtr<FJsonObject> ParseJSON(FArchive* const Stream, const FString& NameForErrors, bool bSilent)
{
	const TSharedRef< TJsonReader<> >& Reader = TJsonReaderFactory<>::Create(Stream);

	TSharedPtr<FJsonObject> SpriteDescriptorObject;
	if (FJsonSerializer::Deserialize(Reader, SpriteDescriptorObject) && SpriteDescriptorObject.IsValid())
	{
		// File was loaded and deserialized OK!
		return SpriteDescriptorObject;
	}
	else
	{
		if (!bSilent)
		{
			UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to parse sprite descriptor file '%ls'.  Error: '%ls'", *NameForErrors, *Reader->GetErrorMessage());
		}
		return nullptr;
	}
}

bool ParseMetaBlock(const FString& NameForErrors, TSharedPtr<FJsonObject>& SpriteDescriptorObject, FString& OutImage, bool bSilent)
{
	bool bLoadedSuccessfully = true;
	TSharedPtr<FJsonObject> MetaBlock = FPaperJSONHelpers::ReadObject(SpriteDescriptorObject, TEXT("meta"));
	if (MetaBlock.IsValid())
	{
		// Example contents:
		//   "app": "Adobe Flash CS6",
		//   "version": "12.0.0.481",        (ignored)
		//   "image": "MySprite.png",
		//   "format": "RGBA8888",           (ignored)
		//   "size": {"w":2048,"h":2048},    (ignored)
		//   "scale": "1"                    (ignored)

		const FString AppName = FPaperJSONHelpers::ReadString(MetaBlock, TEXT("app"), TEXT(""));
		OutImage = FPaperJSONHelpers::ReadString(MetaBlock, TEXT("image"), TEXT(""));

		const FString FlashPrefix(TEXT("Adobe Flash"));
		const FString TexturePackerPrefix(TEXT("http://www.codeandweb.com/texturepacker"));

		if (AppName.StartsWith(FlashPrefix) || AppName.StartsWith(TexturePackerPrefix))
		{
			// Cool, we (mostly) know how to handle these sorts of files!
			if (!bSilent)
			{
				UE_LOGF(LogPaperSpriteSheetImporter, Log, "Parsing sprite sheet exported from '%ls'", *AppName);
			}
		}
		else if (!AppName.IsEmpty())
		{
			// It's got an app tag inside a meta block, so we'll take a crack at it
			if (!bSilent)
			{
				UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Unexpected 'app' named '%ls' while parsing sprite descriptor file '%ls'.  Parsing will continue but the format may not be fully supported", *AppName, *NameForErrors);
			}
		}
		else
		{
			// Probably not a sprite sheet
			if (!bSilent)
			{
				UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to parse sprite descriptor file '%ls'.  Expected 'app' key indicating the exporter (might not be a sprite sheet)", *NameForErrors);
			}
			bLoadedSuccessfully = false;
		}

		if (OutImage.IsEmpty())
		{
			if (!bSilent)
			{
				UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to parse sprite descriptor file '%ls'.  Expected valid 'image' tag", *NameForErrors);
			}
			bLoadedSuccessfully = false;
		}
	}
	else
	{
		if (!bSilent)
		{
			UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to parse sprite descriptor file '%ls'.  Missing meta block", *NameForErrors);
		}
		bLoadedSuccessfully = false;
	}
	return bLoadedSuccessfully;
}

static bool ParseFrame(TSharedPtr<FJsonObject>& FrameData, FSpriteFrame& OutFrame)
{
	bool bReadFrameSuccessfully = true;
	// An example frame:
	//   "frame": {"x":210,"y":10,"w":190,"h":223},
	//   "rotated": false,
	//   "trimmed": true,
	//   "spriteSourceSize": {"x":0,"y":11,"w":216,"h":240},
	//   "sourceSize": {"w":216,"h":240},	
	//   "pivot": {"x":0.5,"y":0.5}			[optional]

	OutFrame.bRotated = FPaperJSONHelpers::ReadBoolean(FrameData, TEXT("rotated"), false);
	OutFrame.bTrimmed = FPaperJSONHelpers::ReadBoolean(FrameData, TEXT("trimmed"), false);

	bReadFrameSuccessfully = bReadFrameSuccessfully && FPaperJSONHelpers::ReadRectangle(FrameData, "frame", /*out*/ OutFrame.SpritePosInSheet, /*out*/ OutFrame.SpriteSizeInSheet);

	if (OutFrame.bTrimmed)
	{
		bReadFrameSuccessfully = bReadFrameSuccessfully && FPaperJSONHelpers::ReadSize(FrameData, "sourceSize", /*out*/ OutFrame.ImageSourceSize);
		bReadFrameSuccessfully = bReadFrameSuccessfully && FPaperJSONHelpers::ReadRectangle(FrameData, "spriteSourceSize", /*out*/ OutFrame.SpriteSourcePos, /*out*/ OutFrame.SpriteSourceSize);
	}
	else
	{
		OutFrame.SpriteSourcePos = FIntPoint::ZeroValue;
		OutFrame.SpriteSourceSize = OutFrame.SpriteSizeInSheet;
		OutFrame.ImageSourceSize = OutFrame.SpriteSizeInSheet;
	}

	if (!FPaperJSONHelpers::ReadXY(FrameData, "pivot", /*out*/ OutFrame.Pivot))
	{
		OutFrame.Pivot.X = 0.5f;
		OutFrame.Pivot.Y = 0.5f;
	}

	// A few more prerequisites to sort out before rotation can be enabled
	if (OutFrame.bRotated)
	{
		// Sprite Source Pos is from top left, our pivot when rotated is bottom left
		OutFrame.SpriteSourcePos.Y = OutFrame.ImageSourceSize.Y - OutFrame.SpriteSourcePos.Y - OutFrame.SpriteSizeInSheet.Y;

		// We rotate final sprite geometry 90 deg CCW to fixup
		// These need to be swapped to be valid in texture space.
		Swap(OutFrame.SpriteSizeInSheet.X, OutFrame.SpriteSizeInSheet.Y);
		Swap(OutFrame.ImageSourceSize.X, OutFrame.ImageSourceSize.Y);

		Swap(OutFrame.SpriteSourcePos.X, OutFrame.SpriteSourcePos.Y);
		Swap(OutFrame.SpriteSourceSize.X, OutFrame.SpriteSourceSize.Y);
	}

	return bReadFrameSuccessfully;
}

static bool ParseFramesFromSpriteHash(TSharedPtr<FJsonObject> ObjectBlock, TArray<FSpriteFrame>& OutSpriteFrames, TSet<FName>& FrameNames)
{
	GWarn->BeginSlowTask(NSLOCTEXT("Paper2D", "PaperJsonImporterFactory_ParsingSpriteFrame", "Parsing Sprite Frame"), true, true);
	bool bLoadedSuccessfully = true;

	// Parse all of the frames
	int32 FrameCount = 0;
	for (auto FrameIt = ObjectBlock->Values.CreateIterator(); FrameIt; ++FrameIt)
	{
		GWarn->StatusUpdate(FrameCount, ObjectBlock->Values.Num(), NSLOCTEXT("Paper2D", "PaperJsonImporterFactory_ParsingSpriteFrames", "Parsing Sprite Frames"));

		bool bReadFrameSuccessfully = true;

		FSpriteFrame Frame;
		Frame.FrameName = FName(FrameIt.Key());

		if (FrameNames.Contains(Frame.FrameName))
		{
			bReadFrameSuccessfully = false;
		}
		else
		{
			FrameNames.Add(Frame.FrameName);
		}

		TSharedPtr<FJsonValue> FrameDataAsValue = FrameIt.Value();
		TSharedPtr<FJsonObject> FrameData;
		if (FrameDataAsValue->Type == EJson::Object)
		{
			FrameData = FrameDataAsValue->AsObject();
			bReadFrameSuccessfully = bReadFrameSuccessfully && ParseFrame(FrameData, /*out*/Frame);
		}
		else
		{
			bReadFrameSuccessfully = false;
		}

		if (bReadFrameSuccessfully)
		{
			OutSpriteFrames.Add(Frame);
		}
		else
		{
			UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Frame %ls is in an unexpected format", *Frame.FrameName.ToString());
			bLoadedSuccessfully = false;
		}

		FrameCount++;
	}

	GWarn->EndSlowTask();
	return bLoadedSuccessfully;
}

static bool ParseFramesFromSpriteArray(const TArray<TSharedPtr<FJsonValue>>& ArrayBlock, TArray<FSpriteFrame>& OutSpriteFrames, TSet<FName>& FrameNames)
{
	GWarn->BeginSlowTask(NSLOCTEXT("Paper2D", "PaperJsonImporterFactory_ParsingSpriteFrame", "Parsing Sprite Frame"), true, true);
	bool bLoadedSuccessfully = true;

	// Parse all of the frames
	for (int32 FrameCount = 0; FrameCount < ArrayBlock.Num(); ++FrameCount)
	{
		GWarn->StatusUpdate(FrameCount, ArrayBlock.Num(), NSLOCTEXT("Paper2D", "PaperJsonImporterFactory_ParsingSpriteFrames", "Parsing Sprite Frames"));
		bool bReadFrameSuccessfully = true;
		FSpriteFrame Frame;

		const TSharedPtr<FJsonValue>& FrameDataAsValue = ArrayBlock[FrameCount];
		if (FrameDataAsValue->Type == EJson::Object)
		{
			TSharedPtr<FJsonObject> FrameData;
			FrameData = FrameDataAsValue->AsObject();

			FString FrameFilename = FPaperJSONHelpers::ReadString(FrameData, TEXT("filename"), TEXT(""));
			if (!FrameFilename.IsEmpty())
			{
				Frame.FrameName = FName(*FrameFilename); // case insensitive
				if (FrameNames.Contains(Frame.FrameName))
				{
					bReadFrameSuccessfully = false;
				}
				else
				{
					FrameNames.Add(Frame.FrameName);
				}

				bReadFrameSuccessfully = bReadFrameSuccessfully && ParseFrame(FrameData, /*out*/Frame);
			}
			else
			{
				bReadFrameSuccessfully = false;
			}
		}
		else
		{
			bReadFrameSuccessfully = false;
		}

		if (bReadFrameSuccessfully)
		{
			OutSpriteFrames.Add(Frame);
		}
		else
		{
			UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Frame %ls is in an unexpected format", *Frame.FrameName.ToString());
			bLoadedSuccessfully = false;
		}
	}

	GWarn->EndSlowTask();
	return bLoadedSuccessfully;
}

static ESpritePivotMode::Type GetBestPivotType(FVector2D JsonPivot)
{
	// Not assuming layout of ESpritePivotMode
	if (JsonPivot.X == 0 && JsonPivot.Y == 0)
	{
		return ESpritePivotMode::Top_Left;
	}
	else if (JsonPivot.X == 0.5f && JsonPivot.Y == 0)
	{
		return ESpritePivotMode::Top_Center;
	}
	else if (JsonPivot.X == 1.0f && JsonPivot.Y == 0)
	{
		return ESpritePivotMode::Top_Right;
	}
	else if (JsonPivot.X == 0 && JsonPivot.Y == 0.5f)
	{
		return ESpritePivotMode::Center_Left;
	}
	else if (JsonPivot.X == 0.5f && JsonPivot.Y == 0.5f)
	{
		return ESpritePivotMode::Center_Center;
	}
	else if (JsonPivot.X == 1.0f && JsonPivot.Y == 0.5f)
	{
		return ESpritePivotMode::Center_Right;
	}
	else if (JsonPivot.X == 0 && JsonPivot.Y == 1.0f)
	{
		return ESpritePivotMode::Bottom_Left;
	}
	else if (JsonPivot.X == 0.5f && JsonPivot.Y == 1.0f)
	{
		return ESpritePivotMode::Bottom_Center;
	}
	else if (JsonPivot.X == 1.0f && JsonPivot.Y == 1.0f)
	{
		return ESpritePivotMode::Bottom_Right;
	}
	else
	{
		return ESpritePivotMode::Custom;
	}
}

////////////////////////////////////////////
// FPaperJsonSpriteSheetImporter

FPaperJsonSpriteSheetImporter::FPaperJsonSpriteSheetImporter()
	: ImageTexture(nullptr)
	, NormalMapTexture(nullptr)
	, bIsReimporting(false)
	, ExistingBaseTexture(nullptr)
	, ExistingNormalMapTexture(nullptr)
{
}

void FPaperJsonSpriteSheetImporter::SetReimportData(const TArray<FString>& ExistingSpriteNames, const TArray< TSoftObjectPtr<class UPaperSprite> >& ExistingSpriteSoftPtrs)
{
	check(ExistingSpriteNames.Num() == ExistingSpriteSoftPtrs.Num());
	if (ExistingSpriteNames.Num() == ExistingSpriteSoftPtrs.Num())
	{
		for (int i = 0; i < ExistingSpriteSoftPtrs.Num(); ++i)
		{
			const TSoftObjectPtr<class UPaperSprite> SpriteSoftPtr = ExistingSpriteSoftPtrs[i];
			UPaperSprite* LoadedSprite = SpriteSoftPtr.LoadSynchronous();
			if (LoadedSprite != nullptr)
			{
				ExistingSprites.Add(ExistingSpriteNames[i], LoadedSprite);
			}
		}
	}
	bIsReimporting = true;
}

bool FPaperJsonSpriteSheetImporter::Import(TSharedPtr<FJsonObject> SpriteDescriptorObject, const FString& NameForErrors, bool bSilent)
{
	bool bLoadedSuccessfully = ParseMetaBlock(NameForErrors, SpriteDescriptorObject, /*out*/ ImageName, bSilent);
	if (bLoadedSuccessfully)
	{
		TSharedPtr<FJsonObject> ObjectFrameBlock = FPaperJSONHelpers::ReadObject(SpriteDescriptorObject, TEXT("frames"));
		TSet<FName> FrameNames;

		if (ObjectFrameBlock.IsValid())
		{
			bLoadedSuccessfully = ParseFramesFromSpriteHash(ObjectFrameBlock, /*out*/Frames, /*inout*/FrameNames);
		}
		else
		{
			// Try loading as an array
			TArray<TSharedPtr<FJsonValue>> ArrayBlock = FPaperJSONHelpers::ReadArray(SpriteDescriptorObject, TEXT("frames"));
			if (ArrayBlock.Num() > 0)
			{
				bLoadedSuccessfully = ParseFramesFromSpriteArray(ArrayBlock, /*out*/Frames, /*inout*/FrameNames);
			}
			else
			{
				if (!bSilent)
				{
					UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to parse sprite descriptor file '%ls'.  Missing frames block", *NameForErrors);
				}
				bLoadedSuccessfully = false;
			}
		}

		if (bLoadedSuccessfully && (Frames.Num() == 0))
		{
			if (!bSilent)
			{
				UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to parse sprite descriptor file '%ls'.  No frames loaded", *NameForErrors);
			}
			bLoadedSuccessfully = false;
		}
	}
	return bLoadedSuccessfully;
}

bool FPaperJsonSpriteSheetImporter::CanImportJSON(const FString& FileContents)
{
	TSharedPtr<FJsonObject> SpriteDescriptorObject = ParseJSON(FileContents, FString(), /*bSilent=*/ true);
	if (SpriteDescriptorObject.IsValid())
	{
		FString Unused;
		return ParseMetaBlock(FString(), SpriteDescriptorObject, /*out*/ Unused, /*bSilent=*/ true);
	}
	
	return false;
}

bool FPaperJsonSpriteSheetImporter::ImportFromString(const FString& FileContents, const FString& NameForErrors, bool bSilent)
{
	TSharedPtr<FJsonObject> SpriteDescriptorObject = ParseJSON(FileContents, NameForErrors, bSilent);
	return SpriteDescriptorObject.IsValid() && Import(SpriteDescriptorObject, NameForErrors, bSilent);
}

bool FPaperJsonSpriteSheetImporter::ImportFromArchive(FArchive* Archive, const FString& NameForErrors, bool bSilent)
{
	TSharedPtr<FJsonObject> SpriteDescriptorObject = ParseJSON(Archive, NameForErrors, bSilent);
	return SpriteDescriptorObject.IsValid() && Import(SpriteDescriptorObject, NameForErrors, bSilent);
}

bool FPaperJsonSpriteSheetImporter::ImportTextures(const FString& LongPackagePath, const FString& SourcePath)
{
	bool bLoadedSuccessfully = true;
	const FString TargetSubPath = LongPackagePath / TEXT("Textures");

	// Load the base texture
	const FString SourceSheetImageFilename = FPaths::Combine(*SourcePath, *ImageName);
	ImageTexture = ImportOrReimportTexture((bIsReimporting && (ExistingBaseTextureName == ImageName)) ? ExistingBaseTexture : nullptr, SourceSheetImageFilename, TargetSubPath);
	if (ImageTexture == nullptr)
	{
		UE_LOGF(LogPaperSpriteSheetImporter, Warning, "Failed to import sprite sheet image '%ls'.", *SourceSheetImageFilename);
		bLoadedSuccessfully = false;
	}

	// Try reimporting the normal map
	// Note: We are checking to see if the *base* texture has been renamed, since the .JSON doesn't actually store a name for the normal map.
	// If the base name has changed, we start from scratch for the normal map too, rather than reimport it even if the old computed one still exists
	if (bIsReimporting && (ExistingBaseTextureName == ImageName) && (ExistingNormalMapTexture != nullptr))
	{
		if (FReimportManager::Instance()->Reimport(ExistingNormalMapTexture, /*bAskForNewFileIfMissing=*/ true))
		{
			NormalMapTexture = ExistingNormalMapTexture;
			ComputedNormalMapName = ExistingNormalMapTextureName;
		}
	}

	// If we weren't reimporting (or failed the reimport), try scanning for a normal map (which may not exist, and that is not an error)
	if (NormalMapTexture == nullptr)
	{
		const UPaperImporterSettings* ImporterSettings = GetDefault<UPaperImporterSettings>();

		// Create a list of names to test of the form [ImageName-[BaseMapSuffix]][NormalMapSuffix] or [ImageName][NormalMapSuffix], preferring the former
		const FString ImageNameNoExtension = FPaths::GetBaseFilename(ImageName);
		const FString ImageTypeExtension = FPaths::GetExtension(ImageName, /*bIncludeDot=*/ true);
		const FString NormalMapNameNoSuffix = ImporterSettings->RemoveSuffixFromBaseMapName(ImageNameNoExtension);

		TArray<FString> NamesToTest;
		ImporterSettings->GenerateNormalMapNamesToTest(NormalMapNameNoSuffix, /*inout*/ NamesToTest);
		ImporterSettings->GenerateNormalMapNamesToTest(ImageNameNoExtension, /*inout*/ NamesToTest);

		// Test each name for a file we can try to import
		for (const FString& NameToTestNoExtension : NamesToTest)
		{
			const FString NameToTest = NameToTestNoExtension + ImageTypeExtension;
			const FString NormalMapSourceImageFilename = FPaths::Combine(*SourcePath, *NameToTest);

			if (FPaths::FileExists(NormalMapSourceImageFilename))
			{
				NormalMapTexture = ImportTexture(NormalMapSourceImageFilename, TargetSubPath);
				if (NormalMapTexture != nullptr)
				{
					ComputedNormalMapName = NameToTest;
				}
				break;
			}
		}
	}

	return bLoadedSuccessfully;
}

UTexture2D* FPaperJsonSpriteSheetImporter::ImportOrReimportTexture(UTexture2D* ExistingTexture, const FString& TextureSourcePath, const FString& DestinationAssetFolder)
{
	UTexture2D* ResultTexture = nullptr;

	// Try reimporting if we have an existing texture
	if (ExistingTexture != nullptr)
	{
		if (FReimportManager::Instance()->Reimport(ExistingTexture, /*bAskForNewFileIfMissing=*/ true))
		{
			ResultTexture = ExistingTexture;
		}
	}

	// If that fails, import the original textures
	if (ResultTexture == nullptr)
	{
		ResultTexture = ImportTexture(TextureSourcePath, DestinationAssetFolder);
	}

	return ResultTexture;
}

UTexture2D* FPaperJsonSpriteSheetImporter::ImportTexture(const FString& TextureSourcePath, const FString& DestinationAssetFolder)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	TArray<FString> TextureFileNames;
	TextureFileNames.Add(TextureSourcePath);
	TArray<UObject*> ImportedSheets = AssetToolsModule.Get().ImportAssets(TextureFileNames, DestinationAssetFolder);

	UTexture2D* ImportedTexture = (ImportedSheets.Num() > 0) ? Cast<UTexture2D>(ImportedSheets[0]) : nullptr;

	if (ImportedTexture != nullptr)
	{
		// Change the compression settings
		GetDefault<UPaperImporterSettings>()->ApplyTextureSettings(ImportedTexture);
	}

	return ImportedTexture;
}

UPaperSprite* FPaperJsonSpriteSheetImporter::FindExistingSprite(const FString& Name)
{
	return ExistingSprites.FindRef(Name);
}

bool FPaperJsonSpriteSheetImporter::PerformImport(const FString& LongPackagePath, EObjectFlags Flags, UPaperSpriteSheet* SpriteSheet)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	GWarn->BeginSlowTask(NSLOCTEXT("Paper2D", "PaperJsonImporterFactory_ImportingSpriteFrame", "Importing Sprite Frame"), true, true);
	for (int32 FrameIndex = 0; FrameIndex < Frames.Num(); ++FrameIndex)
	{
		GWarn->StatusUpdate(FrameIndex, Frames.Num(), NSLOCTEXT("Paper2D", "PaperJsonImporterFactory_ImportingSpriteFrames", "Importing Sprite Frames"));

		// Check for user canceling the import
		if (GWarn->ReceivedUserCancel())
		{
			break;
		}

		const FSpriteFrame& Frame = Frames[FrameIndex];

		// Create a package for the frame
		const FString TargetSubPath = LongPackagePath + TEXT("/Frames");

		UObject* OuterForFrame = nullptr; // @TODO: Use this if we don't want them to be individual assets - Flipbook;

		// Create a frame in the package
		UPaperSprite* TargetSprite = nullptr;
		
		if (bIsReimporting)
		{
			TargetSprite = FindExistingSprite(Frame.FrameName.ToString());
		}
		
		if (TargetSprite == nullptr)
		{
			const FString SanitizedFrameName = ObjectTools::SanitizeObjectName(Frame.FrameName.ToString());
			const FString TentativePackagePath = UPackageTools::SanitizePackageName(TargetSubPath + TEXT("/") + SanitizedFrameName);
			FString DefaultSuffix;
			FString AssetName;
			FString PackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(TentativePackagePath, DefaultSuffix, /*out*/ PackageName, /*out*/ AssetName);

			// Create a unique package name and asset name for the frame
			OuterForFrame = CreatePackage(*PackageName);

			// Create the asset
			TargetSprite = NewObject<UPaperSprite>(OuterForFrame, *AssetName, Flags);
			FAssetRegistryModule::AssetCreated(TargetSprite);
		}

		TargetSprite->Modify();
		FSpriteAssetInitParameters SpriteInitParams;
		SpriteInitParams.Texture = ImageTexture;

		if (NormalMapTexture != nullptr)
		{
			// Put the normal map into the additional textures array and ask for a lit material instead of unlit
			SpriteInitParams.AdditionalTextures.Add(NormalMapTexture);
		}

		SpriteInitParams.Offset = Frame.SpritePosInSheet;
		SpriteInitParams.Dimension = Frame.SpriteSizeInSheet;

		GetDefault<UPaperImporterSettings>()->ApplySettingsForSpriteInit(/*inout*/ SpriteInitParams, (NormalMapTexture != nullptr) ? ESpriteInitMaterialLightingMode::ForceLit : ESpriteInitMaterialLightingMode::Automatic);

		TargetSprite->InitializeSprite(SpriteInitParams, false);

		TargetSprite->SetRotated(Frame.bRotated, false);
		TargetSprite->SetTrim(Frame.bTrimmed, Frame.SpriteSourcePos, Frame.ImageSourceSize, false);

		// Set up pivot on object based on Texture Packer json
		ESpritePivotMode::Type PivotType = GetBestPivotType(Frame.Pivot);
		FVector2D TextureSpacePivotPoint = FVector2D::ZeroVector;
		if (PivotType == ESpritePivotMode::Custom)
		{
			TextureSpacePivotPoint.X = Frame.SpritePosInSheet.X - Frame.SpriteSourcePos.X + Frame.ImageSourceSize.X * Frame.Pivot.X;
			TextureSpacePivotPoint.Y = Frame.SpritePosInSheet.Y - Frame.SpriteSourcePos.Y + Frame.ImageSourceSize.Y * Frame.Pivot.Y;
		}
		TargetSprite->SetPivotMode(PivotType, TextureSpacePivotPoint, false);

		TargetSprite->RebuildData();

		// Create the entry in the animation
		SpriteSheet->SpriteNames.Add(Frame.FrameName.ToString());
		SpriteSheet->Sprites.Add(TargetSprite);
		
		TargetSprite->PostEditChange();
	}

	SpriteSheet->TextureName = ImageName;
	SpriteSheet->Texture = ImageTexture;
	SpriteSheet->NormalMapTextureName = ComputedNormalMapName;
	SpriteSheet->NormalMapTexture = NormalMapTexture;

	GWarn->EndSlowTask();
	return true;
}
