// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"

class FJsonObject;
class UPaperSprite;
class UPaperSpriteSheet;
class UTexture2D;

//////////////////////////////////////////////////////////////////////////
// FSpriteFrame

// Represents one parsed frame in a sprite sheet
struct FSpriteFrame
{
	FName FrameName;

	FIntPoint SpritePosInSheet;
	FIntPoint SpriteSizeInSheet;

	FIntPoint SpriteSourcePos;
	FIntPoint SpriteSourceSize;

	FVector2D ImageSourceSize;

	FVector2D Pivot;

	bool bTrimmed;
	bool bRotated;
};

//////////////////////////////////////////////////////////////////////////
// FJsonPaperSpriteSheetImporter

// Parses a json from FileContents and imports / reimports a spritesheet
class FPaperJsonSpriteSheetImporter
{
public:
	FPaperJsonSpriteSheetImporter();

	static bool CanImportJSON(const FString& FileContents);

	bool ImportFromString(const FString& FileContents, const FString& NameForErrors, bool bSilent);
	bool ImportFromArchive(FArchive* Archive, const FString& NameForErrors, bool bSilent);

	bool ImportTextures(const FString& LongPackagePath, const FString& SourcePath);

	bool PerformImport(const FString& LongPackagePath, EObjectFlags Flags, UPaperSpriteSheet* SpriteSheet);

	void SetReimportData(const TArray<FString>& ExistingSpriteNames, const TArray< TSoftObjectPtr<class UPaperSprite> >& ExistingSpriteSoftPtrs);

	static UTexture2D* ImportOrReimportTexture(UTexture2D* ExistingTexture, const FString& TextureSourcePath, const FString& DestinationAssetFolder);
	static UTexture2D* ImportTexture(const FString& TextureSourcePath, const FString& DestinationAssetFolder);

protected:
	bool Import(TSharedPtr<FJsonObject> SpriteDescriptorObject, const FString& NameForErrors, bool bSilent);
	UPaperSprite* FindExistingSprite(const FString& Name);

protected:
	TArray<FSpriteFrame> Frames;

	FString ImageName;
	UTexture2D* ImageTexture;

	FString ComputedNormalMapName;
	UTexture2D* NormalMapTexture;

public:
	bool bIsReimporting;

	// The name of the default or diffuse texture during a previous import
	FString ExistingBaseTextureName;

	// The asset that was created for ExistingBaseTextureName during a previous import
	UTexture2D* ExistingBaseTexture;

	// The name of the normal map texture during a previous import (if any)
	FString ExistingNormalMapTextureName;

	// The asset that was created for ExistingNormalMapTextureName  during a previous import (if any)
	UTexture2D* ExistingNormalMapTexture;

	// Map of a sprite name (as seen in the importer) -> UPaperSprite
	TMap<FString, UPaperSprite*> ExistingSprites;
};

