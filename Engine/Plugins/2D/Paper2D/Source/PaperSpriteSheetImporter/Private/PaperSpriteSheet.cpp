// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteSheet.h"
#include "EditorFramework/AssetImportData.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteSheet)

UPaperSpriteSheet::UPaperSpriteSheet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPaperSpriteSheet::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
	Super::PostInitProperties();
}

#if WITH_EDITORONLY_DATA
void UPaperSpriteSheet::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	Super::GetAssetRegistryTags(Context);
}
void UPaperSpriteSheet::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_ASSET_IMPORT_DATA_AS_JSON && !AssetImportData)
	{
		// AssetImportData should always be valid
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
}
#endif

