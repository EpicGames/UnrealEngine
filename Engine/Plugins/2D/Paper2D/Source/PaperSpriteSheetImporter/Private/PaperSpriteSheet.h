// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PaperSpriteSheet.generated.h"

class UTexture2D;

UCLASS(BlueprintType, meta = (DisplayThumbnail = "true"))
class UPaperSpriteSheet : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	// The names of sprites during import
	UPROPERTY(VisibleAnywhere, Category=Data)
	TArray<FString> SpriteNames;

	UPROPERTY(VisibleAnywhere, Category=Data)
	TArray< TSoftObjectPtr<class UPaperSprite> > Sprites;

	// The name of the default or diffuse texture during import
	UPROPERTY(VisibleAnywhere, Category=Data)
	FString TextureName;

	// The asset that was created for TextureName
	UPROPERTY(VisibleAnywhere, Category=Data)
	TObjectPtr<UTexture2D> Texture;

	// The name of the normal map texture during import (if any)
	UPROPERTY(VisibleAnywhere, Category=Data)
	FString NormalMapTextureName;

	// The asset that was created for NormalMapTextureName (if any)
	UPROPERTY(VisibleAnywhere, Category=Data)
	TObjectPtr<UTexture2D> NormalMapTexture;

#if WITH_EDITORONLY_DATA
	// Import data for this 
	UPROPERTY(VisibleAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	// UObject interface
	virtual void PostInitProperties() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface
#endif
};
