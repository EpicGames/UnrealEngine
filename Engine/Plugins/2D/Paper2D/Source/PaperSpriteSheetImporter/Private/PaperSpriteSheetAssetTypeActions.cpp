// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteSheetAssetTypeActions.h"
#include "Misc/PackageName.h"
#include "Misc/FeedbackContext.h"
#include "EditorFramework/AssetImportData.h"
#include "PaperSprite.h"
#include "PaperSpriteSheet.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "PaperFlipbookFactory.h"
#include "PackageTools.h"
#include "PaperFlipbookHelpers.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteSheetImportAssetTypeActions

FText FPaperSpriteSheetAssetTypeActions::GetName() const
{
	return LOCTEXT("FSpriteSheetAssetTypeActionsName", "Sprite Sheet");
}

FColor FPaperSpriteSheetAssetTypeActions::GetTypeColor() const
{
	return FColor::Cyan;
}

UClass* FPaperSpriteSheetAssetTypeActions::GetSupportedClass() const
{
	return UPaperSpriteSheet::StaticClass();
}

uint32 FPaperSpriteSheetAssetTypeActions::GetCategories()
{
	return EAssetTypeCategories::Misc;
}

bool FPaperSpriteSheetAssetTypeActions::IsImportedAsset() const
{
	return true;
}

void FPaperSpriteSheetAssetTypeActions::GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const
{
	for (auto& Asset : TypeAssets)
	{
		const auto SpriteSheet = CastChecked<UPaperSpriteSheet>(Asset);
		if (SpriteSheet->AssetImportData)
		{
			SpriteSheet->AssetImportData->ExtractFilenames(OutSourceFilePaths);
		}
	}
}

void FPaperSpriteSheetAssetTypeActions::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	auto SpriteSheetImports = GetTypedWeakObjectPtrs<UPaperSpriteSheet>(InObjects);

	Section.AddMenuEntry(
		"SpriteSheet_CreateFlipbooks",
		LOCTEXT("SpriteSheet_CreateFlipbooks", "Create Flipbooks"),
		LOCTEXT("SpriteSheet_CreateFlipbooksTooltip", "Creates flipbooks from sprites in this sprite sheet."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PaperFlipbook"),
		FUIAction(
		FExecuteAction::CreateSP(this, &FPaperSpriteSheetAssetTypeActions::ExecuteCreateFlipbooks, SpriteSheetImports),
		FCanExecuteAction()
		)
	);
}

//////////////////////////////////////////////////////////////////////////

void FPaperSpriteSheetAssetTypeActions::ExecuteCreateFlipbooks(TArray<TWeakObjectPtr<UPaperSpriteSheet>> Objects)
{	
	for (int SpriteSheetIndex = 0; SpriteSheetIndex < Objects.Num(); ++SpriteSheetIndex)
	{
		UPaperSpriteSheet* SpriteSheet = Objects[SpriteSheetIndex].Get();
		if (SpriteSheet != nullptr)
		{
			const FString PackagePath = FPackageName::GetLongPackagePath(SpriteSheet->GetOutermost()->GetPathName());

			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			
			check(SpriteSheet->SpriteNames.Num() == SpriteSheet->Sprites.Num());
			bool useSpriteNames = (SpriteSheet->SpriteNames.Num() == SpriteSheet->Sprites.Num());

			// Create a list of sprites and sprite names to feed into paper flipbook helpers
			TMap<FString, TArray<UPaperSprite*> > SpriteFlipbookMap;

			{
				TArray<UPaperSprite*> Sprites;
				TArray<FString> SpriteNames;

				for (int SpriteIndex = 0; SpriteIndex < SpriteSheet->Sprites.Num(); ++SpriteIndex)
				{
					TSoftObjectPtr<class UPaperSprite> SpriteSoftPtr = SpriteSheet->Sprites[SpriteIndex];
					UPaperSprite* Sprite = SpriteSoftPtr.LoadSynchronous();
					if (Sprite != nullptr)
					{
						const FString SpriteName = useSpriteNames ? SpriteSheet->SpriteNames[SpriteIndex] : Sprite->GetName();
						Sprites.Add(Sprite);
						SpriteNames.Add(SpriteName);
					}
				}

				FPaperFlipbookHelpers::ExtractFlipbooksFromSprites(/*out*/ SpriteFlipbookMap, Sprites, SpriteNames);
			}

			// Create one flipbook for every grouped flipbook name
			if (SpriteFlipbookMap.Num() > 0)
			{
				UPaperFlipbookFactory* FlipbookFactory = NewObject<UPaperFlipbookFactory>();

				GWarn->BeginSlowTask(NSLOCTEXT("Paper2D", "Paper2D_CreateFlipbooks", "Creating flipbooks from selection"), true, true);

				int Progress = 0;
				int TotalProgress = SpriteFlipbookMap.Num();
				TArray<UObject*> ObjectsToSync;
				for (auto It : SpriteFlipbookMap)
				{
					GWarn->UpdateProgress(Progress++, TotalProgress);

					const FString& FlipbookName = It.Key;
					TArray<UPaperSprite*>& Sprites = It.Value;

					const FString TentativePackagePath = UPackageTools::SanitizePackageName(PackagePath + TEXT("/") + FlipbookName);
					FString DefaultSuffix;
					FString AssetName;
					FString PackageName;
					AssetToolsModule.Get().CreateUniqueAssetName(TentativePackagePath, DefaultSuffix, /*out*/ PackageName, /*out*/ AssetName);

					FlipbookFactory->KeyFrames.Empty();
					for (int32 SpriteIndex = 0; SpriteIndex < Sprites.Num(); ++SpriteIndex)
					{
						FPaperFlipbookKeyFrame* KeyFrame = new (FlipbookFactory->KeyFrames) FPaperFlipbookKeyFrame();
						KeyFrame->Sprite = Sprites[SpriteIndex];
						KeyFrame->FrameRun = 1;
					}

					if (UObject* NewAsset = AssetToolsModule.Get().CreateAsset(AssetName, PackagePath, UPaperFlipbook::StaticClass(), FlipbookFactory))
					{
						ObjectsToSync.Add(NewAsset);
					}

					if (GWarn->ReceivedUserCancel())
					{
						break;
					}
				}

				GWarn->EndSlowTask();

				if (ObjectsToSync.Num() > 0)
				{
					ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

