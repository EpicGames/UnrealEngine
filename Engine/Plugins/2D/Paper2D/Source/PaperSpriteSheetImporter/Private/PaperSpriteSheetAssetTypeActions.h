// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class UPaperSpriteSheet;

class FPaperSpriteSheetAssetTypeActions : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions interface
	virtual FText GetName() const override;
	virtual FColor GetTypeColor() const override;
	virtual UClass* GetSupportedClass() const override;
	virtual void GetActions(const TArray<UObject*>& InObjects, struct FToolMenuSection& Section) override;
	virtual uint32 GetCategories() override;
	virtual bool IsImportedAsset() const override;
	virtual void GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const override;
	// End of IAssetTypeActions interface

private:
	void ExecuteCreateFlipbooks(TArray<TWeakObjectPtr<UPaperSpriteSheet>> Objects);
};
