// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteSheetImportFactory.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "EditorFramework/AssetImportData.h"
#include "Editor.h"
#include "AssetToolsModule.h"
#include "PaperSpriteSheet.h"
#include "Subsystems/ImportSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteSheetImportFactory)

//////////////////////////////////////////////////////////////////////////
// UPaperSpriteSheetImportFactory

UPaperSpriteSheetImportFactory::UPaperSpriteSheetImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	//bEditAfterNew = true;
	SupportedClass = UPaperSpriteSheet::StaticClass();

	bEditorImport = true;
	bText = true;

	Formats.Add(TEXT("json;Spritesheet JSON file"));
	Formats.Add(TEXT("paper2dsprites;Spritesheet JSON file"));
}

FText UPaperSpriteSheetImportFactory::GetToolTip() const
{
	return NSLOCTEXT("Paper2D", "PaperJsonImporterFactoryDescription", "Sprite sheets exported from Adobe Flash or Texture Packer");
}

bool UPaperSpriteSheetImportFactory::FactoryCanImport(const FString& Filename)
{
	FString FileContent;
	if (FFileHelper::LoadFileToString(/*out*/ FileContent, *Filename))
	{
		return FPaperJsonSpriteSheetImporter::CanImportJSON(FileContent);
	}

	return false;
}

UObject* UPaperSpriteSheetImportFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	Flags |= RF_Transactional;

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	const FString FactoryCurrentFilename = UFactory::GetCurrentFilename();
	FString CurrentSourcePath;
	FString FilenameNoExtension;
	FString UnusedExtension;
	FPaths::Split(FactoryCurrentFilename, CurrentSourcePath, FilenameNoExtension, UnusedExtension);

	

	const FString LongPackagePath = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName());
	UPaperSpriteSheet* Result = nullptr;
	
	const FString NameForErrors(InName.ToString());
	const FString FileContent = FString::ConstructFromPtrSize(Buffer, BufferEnd - Buffer);
	
	if (Importer.ImportFromString(FileContent, NameForErrors, /*bSilent=*/ false) &&
		Importer.ImportTextures(LongPackagePath, CurrentSourcePath))
	{
		UPaperSpriteSheet* SpriteSheet = NewObject<UPaperSpriteSheet>(InParent, InName, Flags);
		if (Importer.PerformImport(LongPackagePath, Flags, SpriteSheet))
		{
			SpriteSheet->AssetImportData->Update(FactoryCurrentFilename);

			Result = SpriteSheet;
		}
	}
	
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Result);

	// Reset the importer to ensure that no leftover data can contaminate future imports.
	Importer = FPaperJsonSpriteSheetImporter();

	return Result;
}

