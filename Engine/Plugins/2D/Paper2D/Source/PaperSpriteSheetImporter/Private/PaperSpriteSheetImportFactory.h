// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "PaperJsonSpriteSheetImporter.h"
#include "PaperSpriteSheetImportFactory.generated.h"

// Imports a sprite sheet (and associated paper sprites and textures) from a JSON file exported from Adobe Flash CS6, Texture Packer, or other compatible tool
UCLASS()
class UPaperSpriteSheetImportFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual FText GetToolTip() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn) override;
	// End of UFactory interface

protected:
	// The actual import worker, which may be already configured by the more derived reimport factory by the time FactoryCreateText is called
	FPaperJsonSpriteSheetImporter Importer;
};

