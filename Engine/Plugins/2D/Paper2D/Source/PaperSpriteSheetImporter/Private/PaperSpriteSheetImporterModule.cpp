// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteSheetImporterLog.h"

#include "PaperSpriteSheetAssetTypeActions.h"

//////////////////////////////////////////////////////////////////////////
// FPaperSpriteSheetImporterModule

class FPaperSpriteSheetImporterModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
		// Register asset types
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		SpriteSheetImportAssetTypeActions = MakeShareable(new FPaperSpriteSheetAssetTypeActions);
		AssetTools.RegisterAssetTypeActions(SpriteSheetImportAssetTypeActions.ToSharedRef());
	}

	virtual void ShutdownModule() override
	{
		if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
		{
			IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			if (SpriteSheetImportAssetTypeActions.IsValid())
			{
				AssetTools.UnregisterAssetTypeActions(SpriteSheetImportAssetTypeActions.ToSharedRef());
			}
		}
	}

private:
	TSharedPtr<IAssetTypeActions> SpriteSheetImportAssetTypeActions;
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FPaperSpriteSheetImporterModule, PaperSpriteSheetImporter);
DEFINE_LOG_CATEGORY(LogPaperSpriteSheetImporter);
