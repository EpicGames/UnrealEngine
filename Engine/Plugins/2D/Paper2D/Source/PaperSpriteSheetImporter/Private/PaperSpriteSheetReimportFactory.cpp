// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriteSheetReimportFactory.h"
#include "Engine/Texture2D.h"
#include "PaperSpriteSheetImporterLog.h"
#include "HAL/FileManager.h"
#include "EditorFramework/AssetImportData.h"
#include "PaperSpriteSheet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperSpriteSheetReimportFactory)

#define LOCTEXT_NAMESPACE "PaperJsonImporter"

//////////////////////////////////////////////////////////////////////////
// UPaperSpriteSheetReimportFactory

UPaperSpriteSheetReimportFactory::UPaperSpriteSheetReimportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPaperSpriteSheet::StaticClass();

	bCreateNew = false;
}

bool UPaperSpriteSheetReimportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UPaperSpriteSheet* SpriteSheet = Cast<UPaperSpriteSheet>(Obj);
	if (SpriteSheet && SpriteSheet->AssetImportData)
	{
		SpriteSheet->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
	return false;
}

void UPaperSpriteSheetReimportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UPaperSpriteSheet* SpriteSheet = Cast<UPaperSpriteSheet>(Obj);
	if (SpriteSheet && ensure(NewReimportPaths.Num() == 1))
	{
		SpriteSheet->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UPaperSpriteSheetReimportFactory::Reimport(UObject* Obj)
{
	UPaperSpriteSheet* SpriteSheet = Cast<UPaperSpriteSheet>(Obj);
	if (!SpriteSheet)
	{
		return EReimportResult::Failed;
	}

	// Make sure file is valid and exists
	const FString Filename = SpriteSheet->AssetImportData->GetFirstFilename();
	if (!Filename.Len() || IFileManager::Get().FileSize(*Filename) == INDEX_NONE)
	{
		return EReimportResult::Failed;
	}

	// Configure the importer with the reimport settings
	Importer.SetReimportData(SpriteSheet->SpriteNames, SpriteSheet->Sprites);
	Importer.ExistingBaseTextureName = SpriteSheet->TextureName;
	Importer.ExistingBaseTexture = SpriteSheet->Texture;
	Importer.ExistingNormalMapTextureName = SpriteSheet->NormalMapTextureName;
	Importer.ExistingNormalMapTexture = SpriteSheet->NormalMapTexture;

	// Run the import again
	EReimportResult::Type Result = EReimportResult::Failed;
	bool OutCanceled = false;

	if (ImportObject(SpriteSheet->GetClass(), SpriteSheet->GetOuter(), *SpriteSheet->GetName(), RF_Public | RF_Standalone, Filename, nullptr, OutCanceled) != nullptr)
	{
		UE_LOGF(LogPaperSpriteSheetImporter, Log, "Imported successfully");

		SpriteSheet->AssetImportData->Update(Filename);
		
		// Try to find the outer package so we can dirty it up
		if (SpriteSheet->GetOuter())
		{
			SpriteSheet->GetOuter()->MarkPackageDirty();
		}
		else
		{
			SpriteSheet->MarkPackageDirty();
		}
		Result = EReimportResult::Succeeded;
	}
	else
	{
		if (OutCanceled)
		{
			UE_LOGF(LogPaperSpriteSheetImporter, Warning, "-- import canceled");
		}
		else
		{
			UE_LOGF(LogPaperSpriteSheetImporter, Warning, "-- import failed");
		}

		Result = EReimportResult::Failed;
	}

	return Result;
}

int32 UPaperSpriteSheetReimportFactory::GetPriority() const
{
	return ImportPriority;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

