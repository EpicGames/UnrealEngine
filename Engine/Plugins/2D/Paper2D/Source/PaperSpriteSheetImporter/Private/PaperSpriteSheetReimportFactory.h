// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "PaperSpriteSheetImportFactory.h"
#include "PaperSpriteSheetReimportFactory.generated.h"

// Reimports a UPaperSpriteSheet asset
UCLASS()
class UPaperSpriteSheetReimportFactory : public UPaperSpriteSheetImportFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface
};
