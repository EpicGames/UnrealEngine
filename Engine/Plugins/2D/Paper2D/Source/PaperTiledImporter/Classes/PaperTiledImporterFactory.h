// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "Factories/Factory.h"
#include "PaperTiledImporterFactory.generated.h"

class UPaperTileMap;
class UTexture2D;
namespace ETileMapProjectionMode { enum Type : int; }
struct FPaperTileInfo;
struct FSpriteGeometryCollection;

class FJsonObject;
class FJsonValue;
class UPaperTileSet;
struct FTileMapFromTiled;

enum class ETiledLayerType : uint8
{
	TileLayer,
	ObjectGroup,
	ImageLayer
};

enum class ETiledOrientation : uint8
{
	Unknown,
	Orthogonal,
	Isometric,
	Staggered,
	Hexagonal
};

enum class ETiledObjectLayerDrawOrder : uint8
{
	TopDown,
	Index
};

enum class ETiledStaggerAxis : uint8
{
	X,
	Y
};

enum class ETiledStaggerIndex : uint8
{
	Odd,
	Even
};

enum class ETiledRenderOrder : uint8
{
	RightDown,
	RightUp,
	LeftDown,
	LeftUp
};

enum class ETiledObjectType : uint8
{
	Box,
	Ellipse,
	Polygon,
	Polyline,
	PlacedTile
};

// Imports a tile map (and associated textures & tile sets) exported from Tiled (http://www.mapeditor.org/)
UCLASS()
class UPaperTiledImporterFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual FText GetToolTip() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn) override;
	// End of UFactory interface

	// FReimportHandler interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	// End of FReimportHandler interface

protected:
	TSharedPtr<FJsonObject> ParseJSON(const FString& FileContents, const FString& NameForErrors, bool bSilent = false);

	void ParseGlobalInfoFromJSON(TSharedPtr<FJsonObject> Tree, struct FTileMapFromTiled& OutParsedInfo, const FString& NameForErrors, bool bSilent = false);

	static UObject* CreateNewAsset(UClass* AssetClass, const FString& TargetPath, const FString& DesiredName, EObjectFlags Flags);
	static UTexture2D* ImportTexture(const FString& SourceFilename, const FString& TargetSubPath);

	void FinalizeTileMap(FTileMapFromTiled& GlobalInfo, UPaperTileMap* TileMap);

	bool ConvertTileSets(FTileMapFromTiled& GlobalInfo, const FString& CurrentSourcePath, const FString& LongPackagePath, EObjectFlags Flags);
};

//////////////////////////////////////////////////////////////////////////
// FTiledStringPair

struct FTiledStringPair
{
	FString Key;
	FString Value;

public:
	FTiledStringPair()
	{
	}

	FTiledStringPair(const FString& InKey, const FString& InValue)
		: Key(InKey)
		, Value(InValue)
	{
	}

	static void ParsePropertyBag(TArray<FTiledStringPair>& OutProperties, TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FTiledObject - A single object/shape placed inside a layer or tile

// See https://github.com/bjorn/tiled/wiki/TMX-Map-Format#object for more information

struct FTiledObject
{
	// The type of the object
	ETiledObjectType TiledObjectType;

	// ID of the shape (always 0 for per-tile collision shapes)
	int32 ID;

	// Arbitrary user-specified name
	FString Name;

	// Arbitrary user-specified type
	FString UserType;

	// Is the shape currently visible?
	bool bVisible;

	// Position of the shape:
	//   Ellipse or Box: The center
	//   PlacedTile: Bottom left
	//   Polygon or Polyline: Position of first vertex (not their center, the Tiled editor always bakes down rotations for them!)
	double X;
	double Y;

	// Local-space width/height of the shape (Ellipse, Box)
	// Note: Set to (0,0) for Polygon, Polyline, PlacedTile
	double Width;
	double Height;

	// Rotation (in degrees)
	// Note: Always 0 for Polyline and Polygon
	// Note: Includes winding (can be < 0 or > 360)
	double RotationDegrees;

	// Arbitrary user specified Key-Value pairs
	TArray<FTiledStringPair> Properties;

	// Points for polygon/polyline shapes, relative to (X,Y), which seems to always be the first point in the polygon
	// Note: Only used when TiledObjectType is Polygon or Polyline
	TArray<FVector2D> Points;

	// The tile GID for placed tiles
	// Note: Only used when TiledObjectType is PlacedTile
	uint32 TileGID;

public:
	FTiledObject();

	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent = false);

	static bool ParsePointArray(TArray<FVector2D>& OutPoints, const TArray<TSharedPtr<FJsonValue>>& InArray, const FString& NameForErrors, bool bSilent);

	static void AddToSpriteGeometryCollection(const FVector2D& Offset, const TArray<FTiledObject>& InObjects, FSpriteGeometryCollection& InOutShapes);
};

//////////////////////////////////////////////////////////////////////////
// FTileLayerFromTiled - A layer, containing either tiles or objects

// See https://github.com/bjorn/tiled/wiki/TMX-Map-Format#layer for more information

struct FTileLayerFromTiled
{
	// Name of the layer
	FString Name;

	// Array of tiles (only used when LayerType is TileLayer)
	TArray<uint32> TileIndices;

	// Width and height in tiles
	int32 Width;
	int32 Height;

	// Color of the layer (only set for object layers, to help distinguish them)
	FColor Color;

	// Object draw order (only used for object layers)
	ETiledObjectLayerDrawOrder ObjectDrawOrder;

	// Saved layer opacity (only RGB are used, A is ignored)
	float Opacity;

	// Is the layer currently visible?
	bool bVisible;

	// Type of the layer
	ETiledLayerType LayerType;

	// Offset
	int32 OffsetX;
	int32 OffsetY;

	// Placed objects (only used when LayerType is ObjectGroup)
	TArray<FTiledObject> Objects;

	// Overlay image (only used when LayerType is ImageLayer)
	FString OverlayImagePath;

	// Arbitrary user specified Key-Value pairs
	TArray<FTiledStringPair> Properties;

public:
	FTileLayerFromTiled();

	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent = false);

	bool IsValid() const;
};

//////////////////////////////////////////////////////////////////////////
// FTiledTerrain

// See https://github.com/bjorn/tiled/wiki/TMX-Map-Format#terrain for more information

struct FTiledTerrain
{
	// The name of this terrain type
	FString TerrainName;

	// The index of the solid tile for this terrain (local  index to the tile set, not a GID!)
	uint32 SolidTileLocalIndex;

public:
	FTiledTerrain();

	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent = false);
};

//////////////////////////////////////////////////////////////////////////
// FTiledTileInfo

// See https://github.com/bjorn/tiled/wiki/TMX-Map-Format#tile for more information

struct FTiledTileInfo
{
	// The terrain indices (into the TerrainTypes array of the containing tile set, INDEX_NONE if invalid)
	// Order is top-left, top-right, bottom-left, bottom-right
	int32 TerrainIndices[4];

	// The probability of placement of this tile (0..1)
	// Note: Tiled doesn't allow editing of this value right now (at least as of 0.11.0), so it's of limited value/trustworthiness
	float Probability;

	//@TODO: image?

	// Collision shapes
	TArray<FTiledObject> Objects;

	// Arbitrary user specified Key-Value pairs
	TArray<FTiledStringPair> Properties;

public:
	FTiledTileInfo();

	bool ParseTileInfoFromJSON(int32 TileIndex, TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent = false);
};

//////////////////////////////////////////////////////////////////////////
// FTileSetFromTiled

// See https://github.com/bjorn/tiled/wiki/TMX-Map-Format#tileset for more information

struct FTileSetFromTiled
{
	int32 FirstGID;
	FString Name;

	// Source image path
	FString ImagePath;

	// Source image dimensions
	int32 ImageWidth;
	int32 ImageHeight;

	// Should we chroma-key remove certain pixels, making them transparent?
	bool bRemoveTransparentColor;

	// The color to remove if bRemoveTransparentColor is true
	FColor ImageTransparentColor;

	// Arbitrary user specified Key-Value pairs
	TArray<FTiledStringPair> Properties;

	// Terrain types
	TArray<FTiledTerrain> TerrainTypes;

	// Per-tile info (terrain membership, collision objects, properties, etc...)
	TMap<int32, FTiledTileInfo> PerTileData;

	// Offset used when drawing tiles from this tile set
	int32 TileOffsetX;
	int32 TileOffsetY;

	// The spacing to ignore around the outer edge of the source image (in pixels)
	int32 Margin;

	// The spacing between each tile in the source image (in pixels)
	int32 Spacing;

	int32 TileWidth;
	int32 TileHeight;

public:
	FTileSetFromTiled();

	void ParseTileSetFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent = false);

	bool IsValid() const;
};

//////////////////////////////////////////////////////////////////////////
// FTileMapFromTiled

// See https://github.com/bjorn/tiled/wiki/TMX-Map-Format#map for more information

struct FTileMapFromTiled
{
	// JSON export file version as defined by Tiled (1 is the only known version)
	int32 FileVersion;

	// Dimensions of the tile map (in tiles)
	int32 Width;
	int32 Height;

	// Dimensions of a tile (in pixels)
	int32 TileWidth;
	int32 TileHeight;

	// Projection mode of the tile map
	ETiledOrientation Orientation;

	// Side length (only used in Hexagonal projection mode)
	int32 HexSideLength;

	// Stagger axis (only used in Staggered and Hexagonal modes)
	ETiledStaggerAxis StaggerAxis;

	// Stagger index (only used in Staggered and Hexagonal modes)
	ETiledStaggerIndex StaggerIndex;

	// Render order
	ETiledRenderOrder RenderOrder;

	// Background color
	FColor BackgroundColor;

	// Set of source tile sets imported from Tiled
	TArray<FTileSetFromTiled> TileSets;

	// Set of destination tile set assets created by this import
	TArray<UPaperTileSet*> CreatedTileSetAssets;

	// Layers
	TArray<FTileLayerFromTiled> Layers;

	// Arbitrary user specified Key-Value pairs
	TArray<FTiledStringPair> Properties;

public:
	FPaperTileInfo ConvertTileGIDToPaper2D(uint32 GID) const;
	ETileMapProjectionMode::Type GetOrientationType() const;

	FTileMapFromTiled();
	bool IsValid() const;
};
