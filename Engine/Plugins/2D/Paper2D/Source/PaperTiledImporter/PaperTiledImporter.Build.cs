// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PaperTiledImporter : ModuleRules
{
	public PaperTiledImporter(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Json",
				"Paper2D",
				"UnrealEd",
				"Paper2DEditor"
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"AssetRegistry"
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"AssetRegistry"
			});
	}
}
