// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTiledImporterFactory.h"
#include "PaperTileLayer.h"
#include "PaperTiledImporterLog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "PaperTileMap.h"
#include "Serialization/JsonSerializer.h"
#include "Editor.h"
#include "Subsystems/ImportSubsystem.h"
#include "TileMapAssetImportData.h"
#include "PaperJSONHelpers.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PaperImporterSettings.h"
#include "PackageTools.h"
#include "PaperTileSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PaperTiledImporterFactory)

#define LOCTEXT_NAMESPACE "Paper2D"
#define TILED_IMPORT_ERROR(FormatString, ...) \
	if (!bSilent) { UE_LOG(LogPaperTiledImporter, Warning, FormatString, __VA_ARGS__); }
#define TILED_IMPORT_WARNING(FormatString, ...) \
	if (!bSilent) { UE_LOG(LogPaperTiledImporter, Warning, FormatString, __VA_ARGS__); }

//////////////////////////////////////////////////////////////////////////
// FRequiredScalarField

template<typename ScalarType>
struct FRequiredScalarField
{
	ScalarType& Value;
	const FString Key;
	ScalarType MinValue;

	FRequiredScalarField(ScalarType& InValue, const FString& InKey)
		: Value(InValue)
		, Key(InKey)
		, MinValue(1.0f)
	{
	}

	FRequiredScalarField(ScalarType& InValue, const FString& InKey, ScalarType InMinValue)
		: Value(InValue)
		, Key(InKey)
		, MinValue(InMinValue)
	{
	}
};

typedef FRequiredScalarField<int32> FRequiredIntField;
typedef FRequiredScalarField<double> FRequiredDoubleField;

//////////////////////////////////////////////////////////////////////////

template <typename ScalarType>
bool ParseScalarFields(FRequiredScalarField<ScalarType>* FieldArray, int32 ArrayCount, TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	for (int32 ArrayIndex = 0; ArrayIndex < ArrayCount; ++ArrayIndex)
	{
		const FRequiredScalarField<ScalarType>& Field = FieldArray[ArrayIndex];

		if (!Tree->TryGetNumberField(Field.Key, /*out*/ Field.Value))
		{
			TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Missing '%s' property"), *NameForErrors, *Field.Key);
			bSuccessfullyParsed = false;
			Field.Value = 0;
		}
		else
		{
			if (Field.Value < Field.MinValue)
			{
				TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' (%d but must be at least %d)"), *NameForErrors, *Field.Key, int(Field.Value), int(Field.MinValue));
				bSuccessfullyParsed = false;
				Field.Value = Field.MinValue;
			}
		}
	}

	return bSuccessfullyParsed;
}

bool ParseIntegerFields(FRequiredIntField* IntFieldArray, int32 ArrayCount, TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	return ParseScalarFields(IntFieldArray, ArrayCount, Tree, NameForErrors, bSilent);
}

//////////////////////////////////////////////////////////////////////////
// UPaperTiledImporterFactory

UPaperTiledImporterFactory::UPaperTiledImporterFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	//bEditAfterNew = true;
	SupportedClass = UPaperTileMap::StaticClass();

	bEditorImport = true;
	bText = true;

	Formats.Add(TEXT("json;Tiled JSON file"));
}

FText UPaperTiledImporterFactory::GetToolTip() const
{
	return LOCTEXT("PaperTiledImporterFactoryDescription", "Tile maps exported from Tiled");
}

bool UPaperTiledImporterFactory::FactoryCanImport(const FString& Filename)
{
	FString FileContent;
	if (FFileHelper::LoadFileToString(/*out*/ FileContent, *Filename))
	{
		TSharedPtr<FJsonObject> DescriptorObject = ParseJSON(FileContent, FString(), /*bSilent=*/ true);
		if (DescriptorObject.IsValid())
		{
			FTileMapFromTiled GlobalInfo;
			ParseGlobalInfoFromJSON(DescriptorObject, /*out*/ GlobalInfo, FString(), /*bSilent=*/ true);

			return GlobalInfo.IsValid();
		}
	}

	return false;
}

UObject* UPaperTiledImporterFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	Flags |= RF_Transactional;

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

 	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
 
 	bool bLoadedSuccessfully = true;
 
 	FString CurrentSourcePath;
 	FString FilenameNoExtension;
 	FString UnusedExtension;
 	FPaths::Split(UFactory::GetCurrentFilename(), CurrentSourcePath, FilenameNoExtension, UnusedExtension);
 
 	const FString LongPackagePath = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName());
 
 	const FString NameForErrors(InName.ToString());
 	const FString FileContent = FString::ConstructFromPtrSize(Buffer, BufferEnd - Buffer);
 	TSharedPtr<FJsonObject> DescriptorObject = ParseJSON(FileContent, NameForErrors);

 	UPaperTileMap* Result = nullptr;
 
	FTileMapFromTiled GlobalInfo;
	if (DescriptorObject.IsValid())
	{
		ParseGlobalInfoFromJSON(DescriptorObject, /*out*/ GlobalInfo, NameForErrors);
	}

	if (GlobalInfo.IsValid())
	{
		if (GlobalInfo.FileVersion != 1)
		{
			UE_LOGF(LogPaperTiledImporter, Warning, "JSON exported from Tiled in file '%ls' has an unknown version %d (expected version 1).  Parsing will continue but some things may not import correctly", *NameForErrors, GlobalInfo.FileVersion);
		}

		// Parse the global properties
		const TSharedPtr<FJsonObject>* PropertiesSubobject;
		if (DescriptorObject->TryGetObjectField(TEXT("properties"), /*out*/ PropertiesSubobject))
		{
			FTiledStringPair::ParsePropertyBag(/*out*/ GlobalInfo.Properties, *PropertiesSubobject, NameForErrors, /*bSilent=*/ false);
		}

		// Load the tile sets
		const TArray<TSharedPtr<FJsonValue>>* TileSetDescriptors;
		if (DescriptorObject->TryGetArrayField(TEXT("tilesets"), /*out*/ TileSetDescriptors))
		{
			for (TSharedPtr<FJsonValue> TileSetDescriptor : *TileSetDescriptors)
			{
				FTileSetFromTiled& TileSet = *new (GlobalInfo.TileSets) FTileSetFromTiled();
				TileSet.ParseTileSetFromJSON(TileSetDescriptor->AsObject(), NameForErrors);
				bLoadedSuccessfully = bLoadedSuccessfully && TileSet.IsValid();
			}
		}
		else
		{
			UE_LOGF(LogPaperTiledImporter, Warning, "JSON exported from Tiled in file '%ls' has no tile sets.", *NameForErrors);
			bLoadedSuccessfully = false;
		}

		// Load the layers
		const TArray<TSharedPtr<FJsonValue>>* LayerDescriptors;
		if (DescriptorObject->TryGetArrayField(TEXT("layers"), /*out*/ LayerDescriptors))
		{
			for (TSharedPtr<FJsonValue> LayerDescriptor : *LayerDescriptors)
			{
				FTileLayerFromTiled& TileLayer = *new (GlobalInfo.Layers) FTileLayerFromTiled();
				TileLayer.ParseFromJSON(LayerDescriptor->AsObject(), NameForErrors);
				bLoadedSuccessfully = bLoadedSuccessfully && TileLayer.IsValid();
			}
		}
		else
		{
			UE_LOGF(LogPaperTiledImporter, Warning, "JSON exported from Tiled in file '%ls' has no layers.", *NameForErrors);
			bLoadedSuccessfully = false;
		}

		// Create the new tile map asset and import basic/global data
		Result = NewObject<UPaperTileMap>(InParent, InName, Flags);

		Result->Modify();
		Result->MapWidth = GlobalInfo.Width;
		Result->MapHeight = GlobalInfo.Height;
		Result->TileWidth = GlobalInfo.TileWidth;
		Result->TileHeight = GlobalInfo.TileHeight;
		Result->SeparationPerTileX = 0.0f;
		Result->SeparationPerTileY = 0.0f;
		Result->SeparationPerLayer = 1.0f;
		Result->ProjectionMode = GlobalInfo.GetOrientationType();
		Result->BackgroundColor = GlobalInfo.BackgroundColor;
		Result->HexSideLength = GlobalInfo.HexSideLength;

		if (GlobalInfo.Orientation == ETiledOrientation::Hexagonal)
		{
			Result->TileHeight += GlobalInfo.HexSideLength;
		}

		// Create the tile sets
		const bool bConvertedTileSetsSuccessfully = ConvertTileSets(GlobalInfo, CurrentSourcePath, LongPackagePath, Flags);
		bLoadedSuccessfully = bLoadedSuccessfully && bConvertedTileSetsSuccessfully;

		// Create the layers
		for (int32 LayerIndex = GlobalInfo.Layers.Num() - 1; LayerIndex >= 0; --LayerIndex)
		{
			const FTileLayerFromTiled& LayerData = GlobalInfo.Layers[LayerIndex];

			if (LayerData.IsValid())
			{
				UPaperTileLayer* NewLayer = NewObject<UPaperTileLayer>(Result);
				NewLayer->SetFlags(RF_Transactional);

				NewLayer->LayerName = FText::FromString(LayerData.Name);
				NewLayer->SetShouldRenderInEditor(LayerData.bVisible);

				FLinearColor LayerColor = FLinearColor::White;
				LayerColor.A = FMath::Clamp<float>(LayerData.Opacity, 0.0f, 1.0f);
				NewLayer->SetLayerColor(LayerColor);

				//@TODO: No support for Objects (and thus Color, ObjectDrawOrder), Properties, OffsetX, or OffsetY

				NewLayer->DestructiveAllocateMap(LayerData.Width, LayerData.Height);

				int32 SourceIndex = 0;
				for (int32 Y = 0; Y < LayerData.Height; ++Y)
				{
					for (int32 X = 0; X < LayerData.Width; ++X)
					{
						const uint32 SourceTileGID = LayerData.TileIndices[SourceIndex++];
						const FPaperTileInfo CellContents = GlobalInfo.ConvertTileGIDToPaper2D(SourceTileGID);
						NewLayer->SetCell(X, Y, CellContents);
					}
				}

				Result->TileLayers.Add(NewLayer);
			}
		}

		// Finalize the tile map, including analyzing the tile set textures to determine a good material
		FinalizeTileMap(GlobalInfo, Result);

		Result->PostEditChange();
	}
 	else
 	{
 		// Failed to parse the JSON
 		bLoadedSuccessfully = false;
 	}

	if (Result != nullptr)
	{
		// Store the current file path and timestamp for re-import purposes
		UTileMapAssetImportData* ImportData = UTileMapAssetImportData::GetImportDataForTileMap(Result);
		ImportData->Update(CurrentFilename);

		//GlobalInfo.TileSets
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Result);

	return Result;
}

TSharedPtr<FJsonObject> UPaperTiledImporterFactory::ParseJSON(const FString& FileContents, const FString& NameForErrors, bool bSilent)
{
	// Load the file up (JSON format)
	if (!FileContents.IsEmpty())
	{
		const TSharedRef< TJsonReader<> >& Reader = TJsonReaderFactory<>::Create(FileContents);

		TSharedPtr<FJsonObject> DescriptorObject;
		if (FJsonSerializer::Deserialize(Reader, /*out*/ DescriptorObject) && DescriptorObject.IsValid())
		{
			// File was loaded and deserialized OK!
			return DescriptorObject;
		}
		else
		{
			if (!bSilent)
			{
				//@TODO: PAPER2D: How to correctly surface import errors to the user?
				UE_LOGF(LogPaperTiledImporter, Warning, "Failed to parse tile map JSON file '%ls'.  Error: '%ls'", *NameForErrors, *Reader->GetErrorMessage());
			}
			return nullptr;
		}
	}
	else
	{
		if (!bSilent)
		{
			UE_LOGF(LogPaperTiledImporter, Warning, "Tile map JSON file '%ls' was empty.  This tile map cannot be imported.", *NameForErrors);
		}
		return nullptr;
	}
}

bool UPaperTiledImporterFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	if (UPaperTileMap* TileMap = Cast<UPaperTileMap>(Obj))
	{
		if (TileMap->AssetImportData != nullptr)
		{
			TileMap->AssetImportData->ExtractFilenames(OutFilenames);
		}
		else
		{
			OutFilenames.Add(FString());
		}
		return true;
	}
	return false;
}

void UPaperTiledImporterFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (UPaperTileMap* TileMap = Cast<UPaperTileMap>(Obj))
	{
		if (ensure(NewReimportPaths.Num() == 1))
		{
			UTileMapAssetImportData* ImportData = UTileMapAssetImportData::GetImportDataForTileMap(TileMap);
			ImportData->UpdateFilenameOnly(NewReimportPaths[0]);
		}
	}
}

EReimportResult::Type UPaperTiledImporterFactory::Reimport(UObject* Obj)
{
	if (UPaperTileMap* TileMap = Cast<UPaperTileMap>(Obj))
	{
		//@TODO: Not implemented yet
		ensureMsgf(false, TEXT("Tile map reimport is not implemented yet"));
	}
	return EReimportResult::Failed;
}

int32 UPaperTiledImporterFactory::GetPriority() const
{
	return ImportPriority;
}


UObject* UPaperTiledImporterFactory::CreateNewAsset(UClass* AssetClass, const FString& TargetPath, const FString& DesiredName, EObjectFlags Flags)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	// Create a unique package name and asset name for the frame
	const FString TentativePackagePath = UPackageTools::SanitizePackageName(TargetPath + TEXT("/") + DesiredName);
	FString DefaultSuffix;
	FString AssetName;
	FString PackageName;
	AssetToolsModule.Get().CreateUniqueAssetName(TentativePackagePath, DefaultSuffix, /*out*/ PackageName, /*out*/ AssetName);

	// Create a package for the asset
	UObject* OuterForAsset = CreatePackage(*PackageName);

	// Create a frame in the package
	UObject* NewAsset = NewObject<UObject>(OuterForAsset, AssetClass, *AssetName, Flags);
	FAssetRegistryModule::AssetCreated(NewAsset);

	return NewAsset;
}

UTexture2D* UPaperTiledImporterFactory::ImportTexture(const FString& SourceFilename, const FString& TargetSubPath)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	TArray<FString> FileNames;
	FileNames.Add(SourceFilename);

	//@TODO: Avoid the first compression, since we're going to recompress
	TArray<UObject*> ImportedAssets = AssetToolsModule.Get().ImportAssets(FileNames, TargetSubPath);
	UTexture2D* ImportedTexture = (ImportedAssets.Num() > 0) ? Cast<UTexture2D>(ImportedAssets[0]) : nullptr;

	if (ImportedTexture != nullptr)
	{
		// Change the compression settings
		GetDefault<UPaperImporterSettings>()->ApplyTextureSettings(ImportedTexture);
	}

	return ImportedTexture;
}

void UPaperTiledImporterFactory::ParseGlobalInfoFromJSON(TSharedPtr<FJsonObject> Tree, FTileMapFromTiled& OutParsedInfo, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Parse all of the required integer fields
	FRequiredIntField IntFields[] = {
		FRequiredIntField( OutParsedInfo.FileVersion, TEXT("version") ),
		FRequiredIntField( OutParsedInfo.Width, TEXT("width") ),
		FRequiredIntField( OutParsedInfo.Height, TEXT("height") ),
		FRequiredIntField( OutParsedInfo.TileWidth, TEXT("tilewidth") ),
		FRequiredIntField( OutParsedInfo.TileHeight, TEXT("tileheight") )
	};
	bSuccessfullyParsed = bSuccessfullyParsed && ParseIntegerFields(IntFields, UE_ARRAY_COUNT(IntFields), Tree, NameForErrors, bSilent);

	// Parse hexsidelength if present
	FRequiredIntField OptionalIntFields[] = {
		FRequiredIntField(OutParsedInfo.HexSideLength, TEXT("hexsidelength"), 0)
	};
	ParseIntegerFields(OptionalIntFields, UE_ARRAY_COUNT(OptionalIntFields), Tree, NameForErrors, /*bSilent=*/ true);

	// Parse StaggerAxis if present
	const FString StaggerAxisStr = FPaperJSONHelpers::ReadString(Tree, TEXT("staggeraxis"), FString());
	if (StaggerAxisStr == TEXT("x"))
	{
		OutParsedInfo.StaggerAxis = ETiledStaggerAxis::X;
	}
	else if (StaggerAxisStr == TEXT("y"))
	{
		OutParsedInfo.StaggerAxis = ETiledStaggerAxis::Y;
	}
	else if (!StaggerAxisStr.IsEmpty())
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' ('%s' but expected 'x' or 'y')"), *NameForErrors, TEXT("staggeraxis"), *StaggerAxisStr);
		bSuccessfullyParsed = false;
	}

	// Parse StaggerIndex if present
	const FString StaggerIndexStr = FPaperJSONHelpers::ReadString(Tree, TEXT("staggerindex"), FString());
	if (StaggerIndexStr == TEXT("even"))
	{
		OutParsedInfo.StaggerIndex = ETiledStaggerIndex::Even;
	}
	else if (StaggerIndexStr == TEXT("odd"))
	{
		OutParsedInfo.StaggerIndex = ETiledStaggerIndex::Odd;
	}
	else if (!StaggerIndexStr.IsEmpty())
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' ('%s' but expected 'x' or 'y')"), *NameForErrors, TEXT("staggerindex"), *StaggerIndexStr);
		bSuccessfullyParsed = false;
	}

	// Parse RenderOrder if present
	const FString RenderOrderStr = FPaperJSONHelpers::ReadString(Tree, TEXT("staggerindex"), FString());
	if (RenderOrderStr == TEXT("right-down"))
	{
		OutParsedInfo.RenderOrder = ETiledRenderOrder::RightDown;
	}
	else if (RenderOrderStr == TEXT("right-up"))
	{
		OutParsedInfo.RenderOrder = ETiledRenderOrder::RightUp;
	}
	else if (RenderOrderStr == TEXT("left-down"))
	{
		OutParsedInfo.RenderOrder = ETiledRenderOrder::LeftDown;
	}
	else if (RenderOrderStr == TEXT("left-up"))
	{
		OutParsedInfo.RenderOrder = ETiledRenderOrder::LeftUp;
	}
	else if (!RenderOrderStr.IsEmpty())
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' ('%s' but expected 'right-down', 'right-up', 'left-down', or 'left-up')"), *NameForErrors, TEXT("renderorder"), *RenderOrderStr);
		bSuccessfullyParsed = false;
	}

	// Parse BackgroundColor if present
	const FString ColorHexStr = FPaperJSONHelpers::ReadString(Tree, TEXT("backgroundcolor"), FString());
	if (!ColorHexStr.IsEmpty())
	{
		OutParsedInfo.BackgroundColor = FColor::FromHex(ColorHexStr);
	}

	// Parse the orientation
	const FString OrientationModeStr = FPaperJSONHelpers::ReadString(Tree, TEXT("orientation"), FString());
	if (OrientationModeStr == TEXT("orthogonal"))
	{
		OutParsedInfo.Orientation = ETiledOrientation::Orthogonal;
	}
	else if (OrientationModeStr == TEXT("isometric"))
	{
		OutParsedInfo.Orientation = ETiledOrientation::Isometric;
	}
	else if (OrientationModeStr == TEXT("staggered"))
	{
		OutParsedInfo.Orientation = ETiledOrientation::Staggered;
	}
	else if (OrientationModeStr == TEXT("hexagonal"))
	{
		OutParsedInfo.Orientation = ETiledOrientation::Hexagonal;
	}
	else
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' ('%s' but expected 'orthogonal', 'isometric', 'staggered', or 'hexagonal')"), *NameForErrors, TEXT("orientation"), *OrientationModeStr);
		bSuccessfullyParsed = false;
		OutParsedInfo.Orientation = ETiledOrientation::Unknown;
	}
}

//////////////////////////////////////////////////////////////////////////
// FTileMapFromTiled

FTileMapFromTiled::FTileMapFromTiled()
	: FileVersion(0)
	, Width(0)
	, Height(0)
	, TileWidth(0)
	, TileHeight(0)
	, Orientation(ETiledOrientation::Unknown)
	, HexSideLength(0)
	, StaggerAxis(ETiledStaggerAxis::Y)
	, StaggerIndex(ETiledStaggerIndex::Odd)
	, RenderOrder(ETiledRenderOrder::RightDown)
	, BackgroundColor(55, 55, 55)
{
}

bool FTileMapFromTiled::IsValid() const
{
	return (FileVersion != 0) && (Width > 0) && (Height > 0) && (TileWidth > 0) && (TileHeight > 0) && (Orientation != ETiledOrientation::Unknown);
}

FPaperTileInfo FTileMapFromTiled::ConvertTileGIDToPaper2D(uint32 GID) const
{
	// Split the GID into flip bits and tile index
	const uint32 Flags = GID >> 29;
	const int32 TileIndex = (int32)(GID & ~(7U << 29));

	FPaperTileInfo Result;

	for (int32 SetIndex = TileSets.Num() - 1; SetIndex >= 0; SetIndex--)
	{
		const int32 RelativeIndex = TileIndex - TileSets[SetIndex].FirstGID;
		if (RelativeIndex >= 0)
		{
			// We've found the source tile set and are done searching, but only import a non-null if that tile set imported successfully
			if (UPaperTileSet* Set = CreatedTileSetAssets[SetIndex])
			{
				Result.TileSet = Set;
				Result.PackedTileIndex = RelativeIndex;
				Result.SetFlagValue(EPaperTileFlags::FlipHorizontal, (Flags & 0x4) != 0);
				Result.SetFlagValue(EPaperTileFlags::FlipVertical, (Flags & 0x2) != 0);
				Result.SetFlagValue(EPaperTileFlags::FlipDiagonal, (Flags & 0x1) != 0);
			}
			break;
		}
	}

	return Result;
}

ETileMapProjectionMode::Type FTileMapFromTiled::GetOrientationType() const
{
	switch (Orientation)
	{
	case ETiledOrientation::Isometric:
		return ETileMapProjectionMode::IsometricDiamond;
	case ETiledOrientation::Staggered:
		return ETileMapProjectionMode::IsometricStaggered;
	case ETiledOrientation::Hexagonal:
		return ETileMapProjectionMode::HexagonalStaggered;
	case ETiledOrientation::Orthogonal:
	default:
		return ETileMapProjectionMode::Orthogonal;
	};
}

//////////////////////////////////////////////////////////////////////////
// FTileSetFromTiled

FTileSetFromTiled::FTileSetFromTiled()
	: FirstGID(INDEX_NONE)
	, ImageWidth(0)
	, ImageHeight(0)
	, bRemoveTransparentColor(false)
	, ImageTransparentColor(FColor::Magenta)
	, TileOffsetX(0)
	, TileOffsetY(0)
	, Margin(0)
	, Spacing(0)
	, TileWidth(0)
	, TileHeight(0)
{
}

void FTileSetFromTiled::ParseTileSetFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Parse all of the integer fields
	FRequiredIntField IntFields[] = {
		FRequiredIntField( FirstGID, TEXT("firstgid"), 1 ),
		FRequiredIntField( ImageWidth, TEXT("imagewidth"), 1 ),
		FRequiredIntField( ImageHeight, TEXT("imageheight"), 1 ),
		FRequiredIntField( Margin, TEXT("margin"), 0 ),
		FRequiredIntField( Spacing, TEXT("spacing"), 0 ),
		FRequiredIntField( TileWidth, TEXT("tilewidth"), 1 ),
		FRequiredIntField( TileHeight, TEXT("tileheight"), 1 )
	};

	bSuccessfullyParsed = bSuccessfullyParsed && ParseIntegerFields(IntFields, UE_ARRAY_COUNT(IntFields), Tree, NameForErrors, bSilent);

	// Parse the tile offset
	if (bSuccessfullyParsed)
	{
		FIntPoint TileOffsetTemp;

		if (Tree->HasField(TEXT("tileoffset")))
		{
			if (FPaperJSONHelpers::ReadIntPoint(Tree, TEXT("tileoffset"), /*out*/ TileOffsetTemp))
			{
				TileOffsetX = TileOffsetTemp.X;
				TileOffsetY = TileOffsetTemp.Y;
			}
			else
			{
				TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid or missing value for '%s'"), *NameForErrors, TEXT("tileoffset"));
				bSuccessfullyParsed = false;
			}
		}
	}

	// Parse the tile set name
	Name = FPaperJSONHelpers::ReadString(Tree, TEXT("name"), FString());
	if (Name.IsEmpty())
	{
		TILED_IMPORT_WARNING(TEXT("Expected a non-empty name for each tile set in '%s', generating a new name"), *NameForErrors);
		Name = FString::Printf(TEXT("TileSetStartingAt%d"), FirstGID);
	}

	// Parse the image path
	ImagePath = FPaperJSONHelpers::ReadString(Tree, TEXT("image"), FString());
	if (ImagePath.IsEmpty())
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' ('%s' but expected a path to an image)"), *NameForErrors, TEXT("image"), *ImagePath);
		bSuccessfullyParsed = false;
	}

	// Parse the transparent color if present
	const FString TransparentColorStr = FPaperJSONHelpers::ReadString(Tree, TEXT("transparentcolor"), FString());
	if (!TransparentColorStr.IsEmpty())
	{
		bRemoveTransparentColor = true;
		ImageTransparentColor = FColor::FromHex(TransparentColorStr);
	}

	// Parse the properties (if present)
	const TSharedPtr<FJsonObject>* PropertiesSubobject;
	if (Tree->TryGetObjectField(TEXT("properties"), /*out*/ PropertiesSubobject))
	{
		FTiledStringPair::ParsePropertyBag(/*out*/ Properties, *PropertiesSubobject, NameForErrors, bSilent);
	}

	// Parse the terrain types (if present)
	const TArray<TSharedPtr<FJsonValue>>* TerrainTypesArray;
	if (Tree->TryGetArrayField(TEXT("terrains"), /*out*/ TerrainTypesArray))
	{
		TerrainTypes.Reserve(TerrainTypesArray->Num());
		for (TSharedPtr<FJsonValue> TerrainTypeSrc : *TerrainTypesArray)
		{
			FTiledTerrain NewTerrainType;
			bSuccessfullyParsed = bSuccessfullyParsed && NewTerrainType.ParseFromJSON(TerrainTypeSrc->AsObject(), NameForErrors, bSilent);
			TerrainTypes.Add(NewTerrainType);
		}
	}

	// Parse the per-tile metadata if present (collision objects, terrain membership, etc...)
	const TSharedPtr<FJsonObject>* PerTileInfoSubobject;
	if (Tree->TryGetObjectField(TEXT("tiles"), /*out*/ PerTileInfoSubobject))
	{
		for (const auto& TileInfoSrcPair : (*PerTileInfoSubobject)->Values)
		{
			const int32 TileIndex = FCString::Atoi(*TileInfoSrcPair.Key);
			const TSharedPtr<FJsonObject> TileInfoSrc = TileInfoSrcPair.Value->AsObject();

			FTiledTileInfo& TileInfo = PerTileData.FindOrAdd(TileIndex);

			TileInfo.ParseTileInfoFromJSON(TileIndex, TileInfoSrc, NameForErrors, bSilent);
		}
	}

	// Parse the per-tile properties if present (stored separately to 'tiles' for reasons known only to the author of Tiled)
	const TSharedPtr<FJsonObject>* PerTilePropertiesSubobject;
	if (Tree->TryGetObjectField(TEXT("tiles"), /*out*/ PerTilePropertiesSubobject))
	{
		for (const auto& TilePropsSrcPair : (*PerTilePropertiesSubobject)->Values)
		{
			const int32 TileIndex = FCString::Atoi(*TilePropsSrcPair.Key);
			const TSharedPtr<FJsonObject> TilePropsSrc = TilePropsSrcPair.Value->AsObject();

			FTiledTileInfo& TileInfo = PerTileData.FindOrAdd(TileIndex);

			FTiledStringPair::ParsePropertyBag(/*out*/ TileInfo.Properties, TilePropsSrc, NameForErrors, bSilent);
		}
	}

	//@TODO: Should we parse ImageWidth and ImageHeight?
	// Are these useful if the tile map gets resized to avoid invalidating everything?
}

bool FTileSetFromTiled::IsValid() const
{
	return (TileWidth > 0) && (TileHeight > 0) && (FirstGID > 0);
}

//////////////////////////////////////////////////////////////////////////
// FTileLayerFromTiled

FTileLayerFromTiled::FTileLayerFromTiled()
	: Width(0)
	, Height(0)
	, Color(FColor::White)
	, ObjectDrawOrder(ETiledObjectLayerDrawOrder::TopDown)
	, Opacity(1.0f)
	, bVisible(true)
	, LayerType(ETiledLayerType::TileLayer)
	, OffsetX(0)
	, OffsetY(0)
{
}

bool FTileLayerFromTiled::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Parse all of the integer fields
	FRequiredIntField IntFields[] = {
		FRequiredIntField( Width, TEXT("width"), 0 ),
		FRequiredIntField( Height, TEXT("height"), 0 ),
		FRequiredIntField( OffsetX, TEXT("x"), 0 ),
		FRequiredIntField( OffsetY, TEXT("y"), 0 )
	};

	bSuccessfullyParsed = bSuccessfullyParsed && ParseIntegerFields(IntFields, UE_ARRAY_COUNT(IntFields), Tree, NameForErrors, bSilent);

	if (!Tree->TryGetBoolField(TEXT("visible"), /*out*/ bVisible))
	{
		bVisible = true;
	}

	if (!FPaperJSONHelpers::ReadFloatNoDefault(Tree, TEXT("opacity"), /*out*/ Opacity))
	{
		Opacity = 1.0f;
	}

	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Expected a layer name"), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	// Parse the layer type
	const FString LayerTypeStr = FPaperJSONHelpers::ReadString(Tree, TEXT("type"), FString());
	if (LayerTypeStr == TEXT("tilelayer"))
	{
		if ((Width < 1) || (Height < 1))
		{
			TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Tile layers should be at least 1x1"), *NameForErrors);
			bSuccessfullyParsed = false;
		}

		LayerType = ETiledLayerType::TileLayer;
	}
	else if (LayerTypeStr == TEXT("objectgroup"))
	{
		LayerType = ETiledLayerType::ObjectGroup;
	}
	else if (LayerTypeStr == TEXT("imagelayer"))
	{
		LayerType = ETiledLayerType::ImageLayer;
	}
	else
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' ('%s' but expected 'tilelayer' or 'objectgroup')"), *NameForErrors, TEXT("type"), *LayerTypeStr);
		bSuccessfullyParsed = false;
	}

	// Parse the object draw order (if present)
	const FString ObjectDrawOrderStr = FPaperJSONHelpers::ReadString(Tree, TEXT("draworder"), FString());
	if (ObjectDrawOrderStr == TEXT("index"))
	{
		ObjectDrawOrder = ETiledObjectLayerDrawOrder::Index;
	}
	else if (ObjectDrawOrderStr == TEXT("topdown"))
	{
		ObjectDrawOrder = ETiledObjectLayerDrawOrder::TopDown;
	}
	else if (!ObjectDrawOrderStr.IsEmpty())
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Invalid value for '%s' ('%s' but expected 'index' or 'topdown')"), *NameForErrors, TEXT("draworder"), *ObjectDrawOrderStr);
		bSuccessfullyParsed = false;
	}

	// Parse the property bag if present
	const TSharedPtr<FJsonObject>* PropertiesSubobject;
	if (Tree->TryGetObjectField(TEXT("properties"), /*out*/ PropertiesSubobject))
	{
		FTiledStringPair::ParsePropertyBag(/*out*/ Properties, *PropertiesSubobject, NameForErrors, bSilent);
	}

	// Parse the data specific to this layer type
	if (LayerType == ETiledLayerType::TileLayer)
	{
		const TArray<TSharedPtr<FJsonValue>>* DataArray;
		if (Tree->TryGetArrayField(TEXT("data"), /*out*/ DataArray))
		{
			TileIndices.Reserve(DataArray->Num());
			for (TSharedPtr<FJsonValue> TileEntry : *DataArray)
			{
				const double TileIndexAsDouble = TileEntry->AsNumber();
				uint32 TileID = (uint32)TileIndexAsDouble;
				TileIndices.Add(TileID);
			}
		}
		else
		{
			TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Missing tile data for layer '%s'"), *NameForErrors, *Name);
			bSuccessfullyParsed = false;
		}
	}
	else if (LayerType == ETiledLayerType::ObjectGroup)
	{
		const TArray<TSharedPtr<FJsonValue>>* ObjectArray;
		if (Tree->TryGetArrayField(TEXT("objects"), /*out*/ ObjectArray))
		{
			Objects.Reserve(ObjectArray->Num());
			for (TSharedPtr<FJsonValue> ObjectEntry : *ObjectArray)
			{
				FTiledObject NewObject;
				bSuccessfullyParsed = bSuccessfullyParsed && NewObject.ParseFromJSON(ObjectEntry->AsObject(), NameForErrors, bSilent);
				Objects.Add(NewObject);
			}
		}
		else
		{
			TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Missing object data for layer '%s'"), *NameForErrors, *Name);
			bSuccessfullyParsed = false;
		}
	}
	else if (LayerType == ETiledLayerType::ImageLayer)
	{
		OverlayImagePath = FPaperJSONHelpers::ReadString(Tree, TEXT("image"), FString());
	}
	else
	{
		if (bSuccessfullyParsed)
		{
			TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Unknown layer type for layer '%s'"), *NameForErrors, *Name);
			bSuccessfullyParsed = false;
		}
	}

	return bSuccessfullyParsed;
}

bool FTileLayerFromTiled::IsValid() const
{
	return (Width > 0) && (Height > 0) && (TileIndices.Num() == (Width*Height));
}

//////////////////////////////////////////////////////////////////////////
// FTiledTerrain

FTiledTerrain::FTiledTerrain()
	: SolidTileLocalIndex(0)
{
}

bool FTiledTerrain::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ TerrainName))
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Terrain entry is missing the 'name' field"), *NameForErrors);
		return false;
	}

	if (!Tree->TryGetNumberField(TEXT("tile"), /*out*/ SolidTileLocalIndex))
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Terrain entry is missing the 'tile' field"), *NameForErrors);
		return false;
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////
// FTiledTileInfo

FTiledTileInfo::FTiledTileInfo()
	: Probability(1.0f)
{
	for (int32 Index = 0; Index < 4; ++Index)
	{
		TerrainIndices[Index] = INDEX_NONE;
	}
}

bool FTiledTileInfo::ParseTileInfoFromJSON(int32 TileIndex, TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try reading the terrain membership array if present
	const TArray<TSharedPtr<FJsonValue>>* TerrainMembershipArray;
	if (Tree->TryGetArrayField(TEXT("terrain"), /*out*/ TerrainMembershipArray))
	{
		if (TerrainMembershipArray->Num() == 4)
		{
			for (int32 Index = 0; Index < 4; ++Index)
			{
				TSharedPtr<FJsonValue> MembershipIndex = (*TerrainMembershipArray)[Index];
				
				if (!MembershipIndex->TryGetNumber(TerrainIndices[Index]))
				{
					TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  The 'terrain' array for tile %d should contain 4 indices into the terrain array"), *NameForErrors, TileIndex);
					bSuccessfullyParsed = false;
				}
			}
		}
		else
		{
			TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  The 'terrain' array for tile %d should contain 4 entries but it contained %d entries"), *NameForErrors, TileIndex, TerrainMembershipArray->Num());
			bSuccessfullyParsed = false;
		}
	}

	// Try reading the probability if present
	double DoubleProbability;
	if (Tree->TryGetNumberField(TEXT("probability"), /*out*/ DoubleProbability))
	{
		Probability = FMath::Clamp<float>((float)DoubleProbability, 0.0f, 1.0f);
	}

	// Try reading the per-tile collision data if present
	// Note: This is really an entire fake objectgroup layer, but only the objects array matters; Tiled doesn't even provide a way to edit the rest of the data.
	const TSharedPtr<FJsonObject>* ObjectGroupSubobject;
	if (Tree->TryGetObjectField(TEXT("objectgroup"), /*out*/ ObjectGroupSubobject))
	{
		const TArray<TSharedPtr<FJsonValue>>* ObjectArray;
		if ((*ObjectGroupSubobject)->TryGetArrayField(TEXT("objects"), /*out*/ ObjectArray))
		{
			Objects.Reserve(ObjectArray->Num());
			for (TSharedPtr<FJsonValue> ObjectEntry : *ObjectArray)
			{
				FTiledObject NewObject;
				bSuccessfullyParsed = bSuccessfullyParsed && NewObject.ParseFromJSON(ObjectEntry->AsObject(), NameForErrors, bSilent);
				Objects.Add(NewObject);
			}
		}
		else
		{
			TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Expected an 'objects' entry inside 'objectgroup' for tile %d"), *NameForErrors, TileIndex);
			bSuccessfullyParsed = false;
		}
	}

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FTiledStringPair

void FTiledStringPair::ParsePropertyBag(TArray<FTiledStringPair>& OutProperties, TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	OutProperties.Reserve(Tree->Values.Num());
	for (auto KV : Tree->Values)
	{
		OutProperties.Add(FTiledStringPair(FString(KV.Key), KV.Value->AsString()));
	}
}

//////////////////////////////////////////////////////////////////////////
// FTiledObject

FTiledObject::FTiledObject()
	: TiledObjectType(ETiledObjectType::Box)
	, ID(0)
	, bVisible(true)
	, X(0.0)
	, Y(0.0)
	, Width(0.0)
	, Height(0.0)
	, RotationDegrees(0.0)
	, TileGID(0)
{
}

bool FTiledObject::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Parse all of the integer fields
	FRequiredDoubleField FloatFields[] = {
		FRequiredDoubleField(Width, TEXT("width"), 0.0),
		FRequiredDoubleField(Height, TEXT("height"), 0.0),
		FRequiredDoubleField(X, TEXT("x"), -MAX_FLT),
		FRequiredDoubleField(Y, TEXT("y"), -MAX_FLT),
		FRequiredDoubleField(RotationDegrees, TEXT("rotation"), -MAX_FLT)
	};

	bSuccessfullyParsed = bSuccessfullyParsed && ParseScalarFields(FloatFields, UE_ARRAY_COUNT(FloatFields), Tree, NameForErrors, bSilent);

	if (!Tree->TryGetBoolField(TEXT("visible"), /*out*/ bVisible))
	{
		bVisible = true;
	}

	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Expected an object name"), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	if (!Tree->TryGetStringField(TEXT("type"), /*out*/ UserType))
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Expected an object type"), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	if (!Tree->TryGetNumberField(TEXT("id"), /*out*/ ID))
	{
		TILED_IMPORT_ERROR(TEXT("Failed to parse '%s'.  Expected an object ID"), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	const TSharedPtr<FJsonObject>* PropertiesSubobject;
	if (Tree->TryGetObjectField(TEXT("properties"), /*out*/ PropertiesSubobject))
	{
		FTiledStringPair::ParsePropertyBag(/*out*/ Properties, *PropertiesSubobject, NameForErrors, bSilent);
	}

	// Determine the object type
	if (Tree->TryGetNumberField(TEXT("gid"), /*out*/ TileGID))
	{
		TiledObjectType = ETiledObjectType::PlacedTile;
	}
	else if (Tree->HasField(TEXT("ellipse")))
	{
		TiledObjectType = ETiledObjectType::Ellipse;
	}
	else
	{
		const TArray<TSharedPtr<FJsonValue>>* PointsArray;
		if (Tree->TryGetArrayField(TEXT("polygon"), /*out*/ PointsArray))
		{
			TiledObjectType = ETiledObjectType::Polygon;
			bSuccessfullyParsed = bSuccessfullyParsed && ParsePointArray(/*out*/ Points, *PointsArray, NameForErrors, bSilent);
		}
		else if (Tree->TryGetArrayField(TEXT("polyline"), /*out*/ PointsArray))
		{
			TiledObjectType = ETiledObjectType::Polyline;
			bSuccessfullyParsed = bSuccessfullyParsed && ParsePointArray(/*out*/ Points, *PointsArray, NameForErrors, bSilent);
		}
		else
		{
			TiledObjectType = ETiledObjectType::Box;
		}
	}


	return bSuccessfullyParsed;
}

bool FTiledObject::ParsePointArray(TArray<FVector2D>& OutPoints, const TArray<TSharedPtr<FJsonValue>>& InArray, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	OutPoints.Reserve(InArray.Num());
	for (TSharedPtr<FJsonValue> ArrayElement : InArray)
	{
		double X = 0.0;
		double Y = 0.0;

		FRequiredDoubleField FloatFields[] = {
			FRequiredDoubleField(X, TEXT("x"), -MAX_FLT),
			FRequiredDoubleField(Y, TEXT("y"), -MAX_FLT)
		};

		bSuccessfullyParsed = bSuccessfullyParsed && ParseScalarFields(FloatFields, UE_ARRAY_COUNT(FloatFields), ArrayElement->AsObject(), NameForErrors, bSilent);

		OutPoints.Add(FVector2D(X, Y));
	}

	return bSuccessfullyParsed;
}

void FTiledObject::AddToSpriteGeometryCollection(const FVector2D& Offset, const TArray<FTiledObject>& InObjects, FSpriteGeometryCollection& InOutShapes)
{
	for (const FTiledObject& SourceObject : InObjects)
	{
		const FVector2D SourcePos = Offset + FVector2D(SourceObject.X, SourceObject.Y);
		const float SmallerWidthOrHeight = FMath::Min(SourceObject.Width, SourceObject.Height);

		bool bCreatedShape = false;
		switch (SourceObject.TiledObjectType)
		{
		case ETiledObjectType::Box:
			InOutShapes.AddRectangleShape(SourcePos, FVector2D(SourceObject.Width, SourceObject.Height));
			bCreatedShape = true;
			break;
		case ETiledObjectType::Ellipse:
			InOutShapes.AddRectangleShape(SourcePos, FVector2D(SmallerWidthOrHeight, SmallerWidthOrHeight));
			bCreatedShape = true;
			break;
		case ETiledObjectType::Polygon:
			{
				FSpriteGeometryShape NewShape;
				NewShape.ShapeType = ESpriteShapeType::Polygon;
				NewShape.BoxPosition = SourcePos;
				NewShape.Vertices = SourceObject.Points;
				InOutShapes.Shapes.Add(NewShape);
				bCreatedShape = true;
		}
			break;
		case ETiledObjectType::PlacedTile:
			UE_LOGF(LogPaperTiledImporter, Warning, "Ignoring Tiled Object of type PlacedTile");
			break;
		case ETiledObjectType::Polyline:
			UE_LOGF(LogPaperTiledImporter, Warning, "Ignoring Tiled Object of type Polyline");
			break;
		default:
			ensureMsgf(false, TEXT("Unknown enumerant in ETiledObjectType"));
			break;
		}

		if (bCreatedShape)
		{
			const float RotationUnwound = FMath::Fmod(SourceObject.RotationDegrees, 360.0);
			InOutShapes.Shapes.Last().Rotation = RotationUnwound;
		}
	}

	InOutShapes.ConditionGeometry();
}

void UPaperTiledImporterFactory::FinalizeTileMap(FTileMapFromTiled& GlobalInfo, UPaperTileMap* TileMap)
{
	const UPaperImporterSettings* ImporterSettings = GetDefault<UPaperImporterSettings>();

	// Bind our selected tile set to the first tile set that was imported so something is already picked
	UPaperTileSet* DefaultTileSet = (GlobalInfo.CreatedTileSetAssets.Num() > 0) ? GlobalInfo.CreatedTileSetAssets[0] : nullptr;
	TileMap->SelectedTileSet = DefaultTileSet;

	// Initialize the scale
	TileMap->PixelsPerUnrealUnit = ImporterSettings->GetDefaultPixelsPerUnrealUnit();
	
	// Analyze the tile set textures (anything with translucent wins; failing that use masked)
	ESpriteInitMaterialType BestMaterial = ESpriteInitMaterialType::Masked;
	if (ImporterSettings->ShouldPickBestMaterialWhenCreatingTileMaps())
	{
		BestMaterial = ESpriteInitMaterialType::Automatic;
		for (UPaperTileSet* TileSet : GlobalInfo.CreatedTileSetAssets)
		{
			if ((TileSet != nullptr) && (TileSet->GetTileSheetTexture() != nullptr))
			{
				ESpriteInitMaterialType TileSheetMaterial = ImporterSettings->AnalyzeTextureForDesiredMaterialType(TileSet->GetTileSheetTexture(), FIntPoint::ZeroValue, TileSet->GetTileSheetAuthoredSize());
				
				switch (TileSheetMaterial)
				{
				case ESpriteInitMaterialType::Opaque:
				case ESpriteInitMaterialType::Masked:
					BestMaterial = ((BestMaterial == ESpriteInitMaterialType::Automatic) || (BestMaterial == ESpriteInitMaterialType::Opaque)) ? TileSheetMaterial : BestMaterial;
					break;
				case ESpriteInitMaterialType::Translucent:
					BestMaterial = TileSheetMaterial;
					break;
				}
			}
		}
	}

	if (BestMaterial == ESpriteInitMaterialType::Automatic)
	{
		// Fall back to masked if we wanted automatic and couldn't analyze things
		BestMaterial = ESpriteInitMaterialType::Masked;
	}

	if (BestMaterial != ESpriteInitMaterialType::LeaveAsIs)
	{
		const bool bUseLitMaterial = false;
		TileMap->Material = ImporterSettings->GetDefaultMaterial(BestMaterial, bUseLitMaterial);
	}
}

bool UPaperTiledImporterFactory::ConvertTileSets(FTileMapFromTiled& GlobalInfo, const FString& CurrentSourcePath, const FString& LongPackagePath, EObjectFlags Flags)
{
	bool bLoadedSuccessfully = true;

	for (const FTileSetFromTiled& TileSetData : GlobalInfo.TileSets)
	{
		if (TileSetData.IsValid())
		{
			const FString TargetTileSetPath = LongPackagePath;
			const FString TargetTexturePath = LongPackagePath / TEXT("Textures");

			UPaperTileSet* TileSetAsset = CastChecked<UPaperTileSet>(CreateNewAsset(UPaperTileSet::StaticClass(), TargetTileSetPath, TileSetData.Name, Flags));
			TileSetAsset->Modify();

			TileSetAsset->SetTileSize(FIntPoint(TileSetData.TileWidth, TileSetData.TileHeight));
			TileSetAsset->SetMargin(FIntMargin(TileSetData.Margin));
			TileSetAsset->SetPerTileSpacing(FIntPoint(TileSetData.Spacing, TileSetData.Spacing));
			TileSetAsset->SetDrawingOffset(FIntPoint(TileSetData.TileOffsetX, TileSetData.TileOffsetY));

			// Import the texture
			const FString SourceImageFilename = FPaths::Combine(*CurrentSourcePath, *TileSetData.ImagePath);

			if (UTexture2D* ImportedTileSheetTexture = ImportTexture(SourceImageFilename, TargetTexturePath))
			{
				TileSetAsset->SetTileSheetTexture(ImportedTileSheetTexture);
			}
			else
			{
				UE_LOGF(LogPaperTiledImporter, Warning, "Failed to import tile set image '%ls' referenced from tile set '%ls'.", *TileSetData.ImagePath, *TileSetData.Name);
				bLoadedSuccessfully = false;
			}

			// Make the tile set allocate space for the per-tile data
			FPropertyChangedEvent InteractiveRebuildTileSet(nullptr, EPropertyChangeType::Interactive);
			TileSetAsset->PostEditChangeProperty(InteractiveRebuildTileSet);

			// Copy across terrain information
			const int32 MaxTerrainTypes = 0xFE;
			const uint8 NoTerrainMembershipIndex = 0xFF;
			if (TileSetData.TerrainTypes.Num() > MaxTerrainTypes)
			{
				UE_LOGF(LogPaperTiledImporter, Warning, "Tile set '%ls' contains more than %d terrain types, ones above this will be ignored.", *TileSetData.Name, MaxTerrainTypes);
			}
			const int32 NumTerrainsToCopy = FMath::Min<int32>(TileSetData.TerrainTypes.Num(), MaxTerrainTypes);
			for (int32 TerrainIndex = 0; TerrainIndex < NumTerrainsToCopy; ++TerrainIndex)
			{
				const FTiledTerrain& SourceTerrain = TileSetData.TerrainTypes[TerrainIndex];

				FPaperTileSetTerrain DestTerrain;
				DestTerrain.TerrainName = SourceTerrain.TerrainName;
				DestTerrain.CenterTileIndex = SourceTerrain.SolidTileLocalIndex;

				TileSetAsset->AddTerrainDescription(DestTerrain);
			}

			// Copy across per-tile metadata
			const int32 NumTilesCreated = TileSetAsset->GetTileCount();
			for (const auto& KV : TileSetData.PerTileData)
			{
				const int32 TileIndex = KV.Key;
				const FTiledTileInfo& SourceTileData = KV.Value;

				if (FPaperTileMetadata* TargetTileData = TileSetAsset->GetMutableTileMetadata(TileIndex))
				{
					// Convert collision geometry
					FTiledObject::AddToSpriteGeometryCollection(FVector2D::ZeroVector, SourceTileData.Objects, TargetTileData->CollisionData);

					// Convert terrain memberhsip
					for (int32 Index = 0; Index < 4; ++Index)
					{
						const int32 SourceTerrainIndex = SourceTileData.TerrainIndices[Index];

						const uint8 DestTerrainIndex = ((SourceTerrainIndex >= 0) && (SourceTerrainIndex < NumTerrainsToCopy)) ? (uint8)SourceTerrainIndex : NoTerrainMembershipIndex;
						TargetTileData->TerrainMembership[Index] = DestTerrainIndex;
					}

					//@TODO: Convert metadata?
				}
			}

			// Update anyone who might be using the tile set (in case we're reimporting)
			FPropertyChangedEvent FinalRebuildTileSet(nullptr, EPropertyChangeType::ValueSet);
			TileSetAsset->PostEditChangeProperty(FinalRebuildTileSet);

			// Save off that we created the asset
			GlobalInfo.CreatedTileSetAssets.Add(TileSetAsset);
		}
		else
		{
			GlobalInfo.CreatedTileSetAssets.Add(nullptr);
		}
	}

	return bLoadedSuccessfully;
}

//////////////////////////////////////////////////////////////////////////

#undef TILED_IMPORT_ERROR
#undef LOCTEXT_NAMESPACE

