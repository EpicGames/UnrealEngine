// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperTiledImporterLog.h"
#include "Modules/ModuleManager.h"

//////////////////////////////////////////////////////////////////////////
// FPaperTiledImporterModule

class FPaperTiledImporterModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FPaperTiledImporterModule, PaperTiledImporter);
DEFINE_LOG_CATEGORY(LogPaperTiledImporter);
