// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlanarConstraintSnapPolicy.h"

//////////////////////////////////////////////////////////////////////////
// FPlanarConstraintSnapPolicy

FPlanarConstraintSnapPolicy::FPlanarConstraintSnapPolicy()
{
	// Default to snapping to Y=0 in the XZ plane
	SnapPlane = FPlane(0.0f, 1.0f, 0.0f, 0.0f);
	bIsEnabled = false;

	//@TODO: Load settings from config
}

void FPlanarConstraintSnapPolicy::SnapScale(FVector& Point, const FVector& GridBase)
{
	// Do nothing
}

void FPlanarConstraintSnapPolicy::SnapPointToGrid(FVector& Point, const FVector& GridBase)
{
	if (bIsEnabled)
	{
		// Snap to within the plane
		Point = Point - (SnapPlane.PlaneDot(Point) * SnapPlane);
	}
}

void FPlanarConstraintSnapPolicy::SnapRotatorToGrid(FRotator& Rotation)
{
	// Do nothing
}

void FPlanarConstraintSnapPolicy::ClearSnappingHelpers(bool bClearImmediately)
{
}

void FPlanarConstraintSnapPolicy::DrawSnappingHelpers(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
}

bool FPlanarConstraintSnapPolicy::IsTranslationSnappingEnabled() const
{
	return IsEnabled();
}

bool FPlanarConstraintSnapPolicy::IsRotationSnappingEnabled() const
{
	return false; // rotation snapping not supported
}

bool FPlanarConstraintSnapPolicy::IsScaleSnappingEnabled() const
{
	return false; // scale snapping not supported
}

bool FPlanarConstraintSnapPolicy::IsEnabled() const
{
	return bIsEnabled;
}

void FPlanarConstraintSnapPolicy::ToggleEnabled()
{
	bIsEnabled = !bIsEnabled;
	//@TODO: Persist to config
}
