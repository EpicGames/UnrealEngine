// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISnappingPolicy.h"
#include "Math/Plane.h"

class FPrimitiveDrawInterface;
class FSceneView;

//////////////////////////////////////////////////////////////////////////
// FPlanarConstraintSnapPolicy

class FPlanarConstraintSnapPolicy : public ISnappingPolicy
{
public:
	FPlane SnapPlane;

	virtual bool IsEnabled() const;
	void ToggleEnabled();

private:
	bool bIsEnabled;
public:
	FPlanarConstraintSnapPolicy();

	// ISnappingPolicy interface
	virtual void SnapScale(FVector& Point, const FVector& GridBase) override;
	virtual void SnapPointToGrid(FVector& Point, const FVector& GridBase) override;
	virtual void SnapRotatorToGrid(FRotator& Rotation) override;
	virtual void ClearSnappingHelpers(bool bClearImmediately) override;
	virtual void DrawSnappingHelpers(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual bool IsTranslationSnappingEnabled() const override;
	virtual bool IsRotationSnappingEnabled() const override;
	virtual bool IsScaleSnappingEnabled() const override;
	// End of ISnappingPolicy interface
};
