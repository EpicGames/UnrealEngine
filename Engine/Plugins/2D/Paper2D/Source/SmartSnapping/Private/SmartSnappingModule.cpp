// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportSnappingModule.h"
#include "PlanarConstraintSnapPolicy.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"

#define LOCTEXT_NAMESPACE "SmartSnapping"

//////////////////////////////////////////////////////////////////////////
// FSmartSnappingModule

class FSmartSnappingModule : public FDefaultModuleImpl
{
public:
	TSharedPtr<FPlanarConstraintSnapPolicy> PlanarPolicy;
	FLevelEditorModule::FLevelEditorMenuExtender ViewMenuExtender;
	FDelegateHandle ViewMenuExtenderHandle;

	virtual void StartupModule() override
	{
		if (!IsRunningCommandlet())
		{
			// Create and register the snapping policy
			PlanarPolicy = MakeShareable(new FPlanarConstraintSnapPolicy);

			IViewportSnappingModule& SnappingModule = FModuleManager::LoadModuleChecked<IViewportSnappingModule>("ViewportSnapping");
			SnappingModule.RegisterSnappingPolicy(PlanarPolicy);

			// Register the extension with the level editor
			{
				// TODO: This should extend the viewport toolbar menu "LevelEditor.ViewportToolbar.Show" using the
				// ToolMenus system instead of using legacy menu extenders.
				ViewMenuExtender = FLevelEditorModule::FLevelEditorMenuExtender::CreateRaw(this, &FSmartSnappingModule::OnExtendLevelEditorViewMenu);

				FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
				auto& MenuExtenders = LevelEditor.GetAllLevelEditorToolbarViewMenuExtenders();
				MenuExtenders.Add(ViewMenuExtender);
				ViewMenuExtenderHandle = MenuExtenders.Last().GetHandle();
			}
		}
	}

	virtual void ShutdownModule() override
	{
		if (UObjectInitialized() && !IsRunningCommandlet())
		{
			// Unregister the level editor extensions
			{
				FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
				LevelEditor.GetAllLevelEditorToolbarViewMenuExtenders().RemoveAll([this](const FLevelEditorModule::FLevelEditorMenuExtender& Extender) { return Extender.GetHandle() == ViewMenuExtenderHandle; });
			}
			
			//

			// Unregister the snapping policy
			IViewportSnappingModule& SnappingModule = FModuleManager::LoadModuleChecked<IViewportSnappingModule>("ViewportSnapping");
			SnappingModule.UnregisterSnappingPolicy(PlanarPolicy);
			PlanarPolicy.Reset();
		}
	}

	void CreateSnappingOptionsMenu(FMenuBuilder& Builder)
	{
		FUIAction Action_TogglePlanarSnap(
			FExecuteAction::CreateRaw(PlanarPolicy.Get(), &FPlanarConstraintSnapPolicy::ToggleEnabled),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(PlanarPolicy.Get(), &FPlanarConstraintSnapPolicy::IsEnabled));
		Builder.AddMenuEntry(
			LOCTEXT("View_Extension_PlanarSnap_Enable", "Planar"),
			LOCTEXT(
				"View_Extension_PlanarSnap_Tooltip", "If Enabled, actors will snap to the nearest location on the constraint plane (NOTE: Only works correctly in perspective views right now!)"
			),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.SnapPlanar"),
			Action_TogglePlanarSnap,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		// 		Builder.AddWidget(SNew(STextBlock).Text(TEXT("Depth value goes here")),
		// 			LOCTEXT("View_Extension_PlanarSnap_Depth", "Distance from plane"));
		// 		FUIAction Action_TogglePlanarSnap(
		// 			FExecuteAction::CreateSP(PlanarPolicy, &FPlanarConstraintSnapPolicy::ToggleEnabled),
		// 			FCanExecuteAction(),
		// 			FIsActionChecked::CreateSP(PlanarPolicy, &FPlanarConstraintSnapPolicy::IsEnabled));
	}

	TSharedRef<FExtender> OnExtendLevelEditorViewMenu(const TSharedRef<FUICommandList> CommandList)
	{
		TSharedRef<FExtender> Extender(new FExtender());

		Extender->AddMenuExtension(
			"Snapping",
			EExtensionHook::After,
			nullptr,
			FMenuExtensionDelegate::CreateRaw(this, &FSmartSnappingModule::CreateSnappingOptionsMenu));
	
		return Extender;
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FSmartSnappingModule, SmartSnapping);

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
