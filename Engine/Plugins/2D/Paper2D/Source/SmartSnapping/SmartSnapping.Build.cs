// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SmartSnapping : ModuleRules
{
	public SmartSnapping(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore",
				"LevelEditor",
				"ViewportSnapping"
			});
	}
}
