// Copyright Epic Games, Inc. All Rights Reserved.

#include "PaperSpriterImportData.h"
#include "UObject/AssetRegistryTagsContext.h"

//////////////////////////////////////////////////////////////////////////
// UPaperSpriterImportData

UPaperSpriterImportData::UPaperSpriterImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UPaperSpriterImportData::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData != nullptr)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	Super::GetAssetRegistryTags(Context);
}
