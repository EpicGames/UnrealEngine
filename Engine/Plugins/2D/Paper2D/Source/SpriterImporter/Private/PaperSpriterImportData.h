// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpriterDataModel.h" //@TODO: For debug only
#include "PaperSpriterImportData.generated.h"

// This is the 'hub' asset that tracks other imported assets for a rigged sprite character exported from Spriter
UCLASS(Experimental)
class UPaperSpriterImportData : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	//@TODO: For debug only
	UPROPERTY(VisibleAnywhere, Category = Spriter)
	FSpriterSCON ImportedData;

// 	// Should be a TMap...
// 	TArray<FString> FileToTextureList;
// 	TSoftObjectPtr<class UTexture2D> ImportedTextureList;
// 	TSoftObjectPtr<class UPaperSprite> ImportedSpriteList;

// 	// The names of sprites during import
// 	UPROPERTY(VisibleAnywhere, Category=Data)
// 	TArray<FString> SpriteNames;
// 
// 	UPROPERTY(VisibleAnywhere, Category = Data)
// 	TArray< TSoftObjectPtr<class UPaperSprite> > Sprites;
// 
// 	// The name of the texture during import
// 	UPROPERTY(VisibleAnywhere, Category = Data)
// 	FString TextureName;
// 
// 	UPROPERTY(VisibleAnywhere, Category = Data)
// 	UTexture2D* Texture;

	// Import data for this 
	UPROPERTY(VisibleAnywhere, Instanced, Category=ImportSettings)
	class UAssetImportData* AssetImportData;

	/** Override to ensure we write out the asset import data */
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
};
