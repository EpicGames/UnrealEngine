// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriterDataModel.h"
#include "SpriterImporterLog.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "PaperJSONHelpers.h"


#define LOCTEXT_NAMESPACE "SpriterImporter"

//////////////////////////////////////////////////////////////////////////
// FSpriterAuditTools

#define UE_AUDIT_SPRITER_IMPORT 1

#if UE_AUDIT_SPRITER_IMPORT

// This class will help to catch unknown/newly introduced properties in the future
class FSpriterAuditTools
{
public:
	TSet<FString> KnownSpatialInfoKeys;
	TSet<FString> KnownFileKeys;
	TSet<FString> KnownFolderKeys;
	TSet<FString> KnownMapInstructionKeys;

	TSet<FString> KnownTagLineKeyTagKeys;
	TSet<FString> KnownTagLineKeyKeys;
	TSet<FString> KnownTagLineKeys;
	TSet<FString> KnownValLineKeyKeys;
	TSet<FString> KnownValLineKeys;
	TSet<FString> KnownMetaKeys;

	TSet<FString> KnownRefKeys;
	TSet<FString> KnownObjectRefKeys;
	TSet<FString> KnownMainlineKeyKeys;
	TSet<FString> KnownBasicTimelineKeyKeys;
	TSet<FString> KnownTimelineBoneKeyKeys;
	TSet<FString> KnownTimelineObjectKeyKeys;
	TSet<FString> KnownTimelineKeys;
	TSet<FString> KnownEventLineKeyKeys;
	TSet<FString> KnownEventLineKeys;
	TSet<FString> KnownAnimationKeys;
	TSet<FString> KnownCharacterMapKeys;
	TSet<FString> KnownVariableDefinitionKeys;
	TSet<FString> KnownObjInfoKeys;
	TSet<FString> KnownEntityKeys;
	TSet<FString> KnownSCONTagListKeys;
	TSet<FString> KnownSCONKeys;


	static FSpriterAuditTools& Get()
	{
		static FSpriterAuditTools StaticInstance;
		return StaticInstance;
	}

	static void AuditKeys(const TSet<FString>& TestSet, const TSharedPtr<FJsonObject> Tree, const FString& ContextString)
	{
		for (const auto& KVP : Tree->Values)
		{
			if (!TestSet.Contains(KVP.Key))
			{
				const bool bSilent = false;
				SPRITER_IMPORT_WARNING(TEXT("Unexpected field '%s' in context '%s'.  Parsing will continue but not all information is being imported."), *KVP.Key, *ContextString);
				static int32 A = 0;
				++A;
			}
		}
	}

private:
	FSpriterAuditTools()
	{
		KnownSpatialInfoKeys.Add(TEXT("x"));
		KnownSpatialInfoKeys.Add(TEXT("y"));
		KnownSpatialInfoKeys.Add(TEXT("angle"));
		KnownSpatialInfoKeys.Add(TEXT("scale_x"));
		KnownSpatialInfoKeys.Add(TEXT("scale_y"));
		KnownSpatialInfoKeys.Add(TEXT("r"));
		KnownSpatialInfoKeys.Add(TEXT("g"));
		KnownSpatialInfoKeys.Add(TEXT("b"));
		KnownSpatialInfoKeys.Add(TEXT("a"));

		KnownFileKeys.Add(TEXT("name"));
		KnownFileKeys.Add(TEXT("pivot_x"));
		KnownFileKeys.Add(TEXT("pivot_y"));
		KnownFileKeys.Add(TEXT("width"));
		KnownFileKeys.Add(TEXT("height"));
		KnownFileKeys.Add(TEXT("type"));
		KnownFileKeys.Add(TEXT("id")); // Known but being ignored

		KnownFolderKeys.Add(TEXT("name"));
		KnownFolderKeys.Add(TEXT("file"));
		KnownFolderKeys.Add(TEXT("id")); // Known but being ignored

		KnownMapInstructionKeys.Add(TEXT("file"));
		KnownMapInstructionKeys.Add(TEXT("folder"));
		KnownMapInstructionKeys.Add(TEXT("target_file"));
		KnownMapInstructionKeys.Add(TEXT("target_folder"));

		KnownTagLineKeyTagKeys.Add(TEXT("t"));
		KnownTagLineKeyTagKeys.Add(TEXT("id")); // Known but being ignored

		KnownTagLineKeyKeys.Add(TEXT("time"));
		KnownTagLineKeyKeys.Add(TEXT("tag"));
		KnownTagLineKeyKeys.Add(TEXT("id")); // Known but being ignored

		KnownTagLineKeys.Add(TEXT("key"));

		KnownValLineKeyKeys.Add(TEXT("time"));
		KnownValLineKeyKeys.Add(TEXT("val"));
		KnownValLineKeyKeys.Add(TEXT("id")); // Known but being ignored

		KnownValLineKeys.Add(TEXT("def"));
		KnownValLineKeys.Add(TEXT("key"));
		KnownValLineKeys.Add(TEXT("id")); // Known but being ignored

		KnownMetaKeys.Add(TEXT("tagline"));
		KnownMetaKeys.Add(TEXT("valline"));

		KnownRefKeys.Add(TEXT("key"));
		KnownRefKeys.Add(TEXT("parent"));
		KnownRefKeys.Add(TEXT("timeline"));
		KnownRefKeys.Add(TEXT("id")); // Known but being ignored

		KnownObjectRefKeys.Append(KnownRefKeys);
		KnownObjectRefKeys.Add(TEXT("z_index"));

		KnownMainlineKeyKeys.Add(TEXT("time"));
		KnownMainlineKeyKeys.Add(TEXT("bone_ref"));
		KnownMainlineKeyKeys.Add(TEXT("object_ref"));
		KnownMainlineKeyKeys.Add(TEXT("curve_type"));
		KnownMainlineKeyKeys.Add(TEXT("id")); // Known but being ignored

		KnownBasicTimelineKeyKeys.Add(TEXT("time"));
		KnownBasicTimelineKeyKeys.Add(TEXT("curve_type"));
		KnownBasicTimelineKeyKeys.Add(TEXT("c1"));
		KnownBasicTimelineKeyKeys.Add(TEXT("c2"));
		KnownBasicTimelineKeyKeys.Add(TEXT("spin"));
		KnownBasicTimelineKeyKeys.Add(TEXT("id")); // Known but being ignored
		KnownBasicTimelineKeyKeys.Add(TEXT("object"));
		KnownBasicTimelineKeyKeys.Add(TEXT("bone"));

		KnownTimelineBoneKeyKeys.Append(KnownSpatialInfoKeys);

		KnownTimelineObjectKeyKeys.Append(KnownSpatialInfoKeys);
		KnownTimelineObjectKeyKeys.Add(TEXT("file"));
		KnownTimelineObjectKeyKeys.Add(TEXT("folder"));
		KnownTimelineObjectKeyKeys.Add(TEXT("pivot_x"));
		KnownTimelineObjectKeyKeys.Add(TEXT("pivot_y"));

		KnownTimelineKeys.Add(TEXT("name"));
		KnownTimelineKeys.Add(TEXT("object_type"));
		KnownTimelineKeys.Add(TEXT("obj"));
		KnownTimelineKeys.Add(TEXT("key"));
		KnownTimelineKeys.Add(TEXT("meta"));
		KnownTimelineKeys.Add(TEXT("id")); // Known but being ignored

		KnownEventLineKeyKeys.Add(TEXT("time"));
		KnownEventLineKeyKeys.Add(TEXT("id")); // Known but being ignored

		KnownEventLineKeys.Add(TEXT("name"));
		KnownEventLineKeys.Add(TEXT("obj"));
		KnownEventLineKeys.Add(TEXT("key"));
		KnownEventLineKeys.Add(TEXT("id")); // Known but being ignored

		KnownAnimationKeys.Add(TEXT("name"));
		KnownAnimationKeys.Add(TEXT("length"));
		KnownAnimationKeys.Add(TEXT("interval"));
		KnownAnimationKeys.Add(TEXT("mainline"));
		KnownAnimationKeys.Add(TEXT("looping"));
		KnownAnimationKeys.Add(TEXT("timeline"));
		//KnownAnimationKeys.Add(TEXT("soundline")); //@TODO: Not supported yet
		KnownAnimationKeys.Add(TEXT("eventline"));
		KnownAnimationKeys.Add(TEXT("gline")); //@TODO: Not supported yet
		KnownAnimationKeys.Add(TEXT("meta"));
		KnownAnimationKeys.Add(TEXT("id")); // Known but being ignored

		KnownCharacterMapKeys.Add(TEXT("name"));
		KnownCharacterMapKeys.Add(TEXT("map"));
		KnownCharacterMapKeys.Add(TEXT("id")); // Known but being ignored

		KnownVariableDefinitionKeys.Add(TEXT("name"));
		KnownVariableDefinitionKeys.Add(TEXT("default"));
		KnownVariableDefinitionKeys.Add(TEXT("type"));
		KnownVariableDefinitionKeys.Add(TEXT("id")); // Known but being ignored

		KnownObjInfoKeys.Add(TEXT("name"));
		KnownObjInfoKeys.Add(TEXT("type"));
		KnownObjInfoKeys.Add(TEXT("w"));
		KnownObjInfoKeys.Add(TEXT("h"));
		KnownObjInfoKeys.Add(TEXT("pivot_x"));
		KnownObjInfoKeys.Add(TEXT("pivot_y"));
		KnownObjInfoKeys.Add(TEXT("id")); // Known but being ignored
		KnownObjInfoKeys.Add(TEXT("frames")); //@TODO: Not supported yet
		KnownObjInfoKeys.Add(TEXT("var_defs"));

		KnownEntityKeys.Add(TEXT("name"));
		KnownEntityKeys.Add(TEXT("obj_info"));
		KnownEntityKeys.Add(TEXT("animation"));
		KnownEntityKeys.Add(TEXT("character_map"));
		KnownEntityKeys.Add(TEXT("id")); // Known but being ignored
		KnownEntityKeys.Add(TEXT("var_defs"));

		KnownSCONTagListKeys.Add(TEXT("name"));
		KnownSCONTagListKeys.Add(TEXT("id")); // Known but being ignored

		KnownSCONKeys.Add(TEXT("scon_version"));
		KnownSCONKeys.Add(TEXT("generator"));
		KnownSCONKeys.Add(TEXT("generator_version"));
		KnownSCONKeys.Add(TEXT("entity"));
		KnownSCONKeys.Add(TEXT("folder"));
		KnownSCONKeys.Add(TEXT("tag_list"));
	}
};

#define UE_DO_SPRITER_AUDIT(KeySet, Object, Message) FSpriterAuditTools::AuditKeys(FSpriterAuditTools::Get().KeySet, Object, Message);

#else

#define UE_DO_SPRITER_AUDIT(KeySet, Object, Message)

#endif

//////////////////////////////////////////////////////////////////////////
// FSpriterEnumHelper

ESpriterObjectType FSpriterEnumHelper::StringToObjectType(const FString& InString)
{
	if (InString == TEXT("sprite"))
	{
		return ESpriterObjectType::Sprite;
	}
	else if (InString == TEXT("bone"))
	{
		return ESpriterObjectType::Bone;
	}
	else if (InString == TEXT("box"))
	{
		return ESpriterObjectType::Box;
	}
	else if (InString == TEXT("point"))
	{
		return ESpriterObjectType::Point;
	}
	else if (InString == TEXT("sound"))
	{
		return ESpriterObjectType::Sound;
	}
	else if (InString == TEXT("entity"))
	{
		return ESpriterObjectType::Entity;
	}
	else if (InString == TEXT("variable"))
	{
		return ESpriterObjectType::Variable;
	}
	else if (InString == TEXT("event"))
	{
		return ESpriterObjectType::Event;
	}
	else
	{
		return ESpriterObjectType::INVALID;
	}
}

ESpriterCurveType FSpriterEnumHelper::StringToCurveType(const FString& InString)
{
	if (InString == TEXT("linear"))
	{
		return ESpriterCurveType::Linear;
	}
	else if (InString == TEXT("instant"))
	{
		return ESpriterCurveType::Instant;
	}
	else if (InString == TEXT("quadratic"))
	{
		return ESpriterCurveType::Quadratic;
	}
	else if (InString == TEXT("cubic"))
	{
		return ESpriterCurveType::Cubic;
	}
	else
	{
		return ESpriterCurveType::INVALID;
	}
}

ESpriterVariableType FSpriterEnumHelper::StringToVariableType(const FString& InString)
{
	if (InString == TEXT("float"))
	{
		return ESpriterVariableType::Float;
	}
	else if (InString == TEXT("int"))
	{
		return ESpriterVariableType::Integer;
	}
	else if (InString == TEXT("string"))
	{
		return ESpriterVariableType::String;
	}
	else
	{
		return ESpriterVariableType::INVALID;
	}
}

ESpriterFileType FSpriterEnumHelper::StringToFileType(const FString& InString)
{
	if (InString == TEXT("sprite"))
	{
		return ESpriterFileType::Sprite;
	}
	else if (InString == TEXT("sound"))
	{
		return ESpriterFileType::Sound;
	}
	else
	{
		return ESpriterFileType::INVALID;
	}
}

//////////////////////////////////////////////////////////////////////////
// FSpriterSpatialInfo

FSpriterSpatialInfo::FSpriterSpatialInfo()
	: X(0.0f)
	, Y(0.0f)
	, AngleInDegrees(0.0f)
	, ScaleX(1.0f)
	, ScaleY(1.0f)
	, Color(FLinearColor::White)
{
}

bool FSpriterSpatialInfo::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	Tree->TryGetNumberField(TEXT("x"), /*out*/ X);
	Tree->TryGetNumberField(TEXT("y"), /*out*/ Y);
	Tree->TryGetNumberField(TEXT("angle"), /*out*/ AngleInDegrees);
	Tree->TryGetNumberField(TEXT("scale_x"), /*out*/ ScaleX);
	Tree->TryGetNumberField(TEXT("scale_y"), /*out*/ ScaleY);

	double DR = 1.0;
	double DG = 1.0;
	double DB = 1.0;
	double DA = 1.0;
	Tree->TryGetNumberField(TEXT("r"), /*out*/ DR);
	Tree->TryGetNumberField(TEXT("g"), /*out*/ DG);
	Tree->TryGetNumberField(TEXT("b"), /*out*/ DB);
	Tree->TryGetNumberField(TEXT("a"), /*out*/ DA);
	Color = FLinearColor(DR, DG, DB, DA);

	return true;
}

FTransform FSpriterSpatialInfo::ConvertToTransform() const
{
	FTransform Result;
	Result.SetTranslation((X * PaperAxisX) + (Y * PaperAxisY));
	Result.SetRotation(FRotator(AngleInDegrees, 0.0f, 0.0f).Quaternion());
	Result.SetScale3D((ScaleX * PaperAxisX) + (ScaleY * PaperAxisY) + (PaperAxisZ * 1.0f));

	return Result;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterFile

FSpriterFile::FSpriterFile()
	: PivotX(0.0f)
	, PivotY(1.0f)
	, Width(0)
	, Height(0)
	, FileType(ESpriterFileType::INVALID)
{
}

bool FSpriterFile::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Parse the name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the file object of '%s'."), *NameForErrors);
		Name = TEXT("(missing file name)");
		bSuccessfullyParsed = false;
	}

	// Optionally parse the type property
	FString FileTypeAsString;
	if (Tree->TryGetStringField(TEXT("type"), /*out*/ FileTypeAsString))
	{
		FileType = FSpriterEnumHelper::StringToFileType(FileTypeAsString);
		if (FileType == ESpriterFileType::INVALID)
		{
			SPRITER_IMPORT_ERROR(TEXT("Unknown value '%s' for 'type' in file '%s' in '%s'."), *FileTypeAsString, *Name, *NameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else
	{
		// Defaults to sprite
		FileType = ESpriterFileType::Sprite;
	}

	Tree->TryGetNumberField(TEXT("pivot_x"), /*out*/ PivotX);
	Tree->TryGetNumberField(TEXT("pivot_y"), /*out*/ PivotY);
	Tree->TryGetNumberField(TEXT("width"), /*out*/ Width);
	Tree->TryGetNumberField(TEXT("height"), /*out*/ Height);

	UE_DO_SPRITER_AUDIT(KnownFileKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterFolder

FSpriterFolder::FSpriterFolder()
{
}

bool FSpriterFolder::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the folder name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the folder object of '%s'."), *NameForErrors);
		Name = TEXT("(missing folder name)");
		bSuccessfullyParsed = false;
	}

	// Try parsing the list of files
	const TArray<TSharedPtr<FJsonValue>>* FileDescriptors;
	if (Tree->TryGetArrayField(TEXT("file"), /*out*/ FileDescriptors))
	{
		const FString LocalNameForErrors = FString::Printf(TEXT("%s folder '%s'"), *NameForErrors, *Name);
		for (TSharedPtr<FJsonValue> FileDescriptor : *FileDescriptors)
		{
			FSpriterFile& File = *new (Files) FSpriterFile();
			const bool bParsedFileOK = File.ParseFromJSON(FileDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedFileOK;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("JSON exported from Spriter in file '%s' has no entities."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownFolderKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterMapInstruction

FSpriterMapInstruction::FSpriterMapInstruction()
	: Folder(0)
	, File(0)
	, TargetFolder(INDEX_NONE)
	, TargetFile(INDEX_NONE)
{
}

bool FSpriterMapInstruction::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// These two are required
	if (!Tree->TryGetNumberField(TEXT("file"), /*out*/ File))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'file' field in the map object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	if (!Tree->TryGetNumberField(TEXT("folder"), /*out*/ Folder))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'folder' field in the map object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}
	
	// These two are optional
	Tree->TryGetNumberField(TEXT("target_file"), /*out*/ TargetFile);
	Tree->TryGetNumberField(TEXT("target_folder"), /*out*/ TargetFolder);

	UE_DO_SPRITER_AUDIT(KnownMapInstructionKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterTagLineKey

FSpriterTagLineKey::FSpriterTagLineKey()
	: TimeInMS(0)
{
}

bool FSpriterTagLineKey::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Read the time of the key (in milliseconds)
	if (!Tree->TryGetNumberField(TEXT("time"), /*out*/ TimeInMS))
	{
		// Assume 0 when missing?
		TimeInMS = 0;
	}

	// Parse the tag array
	const TArray<TSharedPtr<FJsonValue>>* TagDescriptors;
	if (Tree->TryGetArrayField(TEXT("tag"), /*out*/ TagDescriptors))
	{
		for (const TSharedPtr<FJsonValue> TagDescriptorUntyped : *TagDescriptors)
		{
			const TSharedPtr<FJsonObject> TagDescriptor = TagDescriptorUntyped->AsObject();

			int32 NewTagIndex = INDEX_NONE;
			if (TagDescriptor->TryGetNumberField(TEXT("t"), /*out*/ NewTagIndex))
			{
				Tags.Add(NewTagIndex);
			}
			else
			{
				SPRITER_IMPORT_ERROR(TEXT("Expected a 't' field in the objects inside the tags array of '%s'."), *NameForErrors);
				bSuccessfullyParsed = false;
			}

			UE_DO_SPRITER_AUDIT(KnownTagLineKeyTagKeys, TagDescriptor, NameForErrors);
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'tag' field in the tag line key '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownTagLineKeyKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterTagLine

FSpriterTagLine::FSpriterTagLine()
{
}

bool FSpriterTagLine::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Parse the key array
	const TArray<TSharedPtr<FJsonValue>>* KeyDescriptors;
	if (Tree->TryGetArrayField(TEXT("key"), /*out*/ KeyDescriptors))
	{
		for (const TSharedPtr<FJsonValue> KeyDescriptor : *KeyDescriptors)
		{
			FSpriterTagLineKey& Key = *new (Keys) FSpriterTagLineKey();
			const bool bParsedKeySuccessfully = Key.ParseFromJSON(KeyDescriptor->AsObject(), NameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedKeySuccessfully;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'key' field in the tag line '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownTagLineKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterValLineKey

FSpriterValLineKey::FSpriterValLineKey()
	: TimeInMS(0)
	, bReadAsNumber(false)
	, ValueAsNumber(0.0)
{
}

bool FSpriterValLineKey::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Read the time of the key (in milliseconds)
	if (!Tree->TryGetNumberField(TEXT("time"), /*out*/ TimeInMS))
	{
		// Assume 0 when missing?
		TimeInMS = 0;
	}

	const TSharedPtr<FJsonValue> ValField = Tree->TryGetField(TEXT("val"));
	if (ValField.IsValid())
	{
		if (ValField->Type == EJson::String)
		{
			bReadAsNumber = false;
			ValueAsString = ValField->AsString();
		}
		else if (ValField->Type == EJson::Number)
		{
			bReadAsNumber = true;
			ValueAsNumber = ValField->AsNumber();
		}
		else
		{
			SPRITER_IMPORT_ERROR(TEXT("Expected the 'val' field to be a string or number in the val line key of '%s'."), *NameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'val' field in the val line key of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownValLineKeyKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterValLine

FSpriterValLine::FSpriterValLine()
	: DefinitionIndex(INDEX_NONE)
{
}

bool FSpriterValLine::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Read the definition index
	if (!Tree->TryGetNumberField(TEXT("def"), /*out*/ DefinitionIndex))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'def' field in the val line of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	// Parse the key array
	const TArray<TSharedPtr<FJsonValue>>* KeyDescriptors;
	if (Tree->TryGetArrayField(TEXT("key"), /*out*/ KeyDescriptors))
	{
		for (const TSharedPtr<FJsonValue> KeyDescriptor : *KeyDescriptors)
		{
			FSpriterValLineKey& Key = *new (Keys) FSpriterValLineKey();
			const bool bParsedKeySuccessfully = Key.ParseFromJSON(KeyDescriptor->AsObject(), NameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedKeySuccessfully;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'key' field in the val line '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownValLineKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterMeta

FSpriterMeta::FSpriterMeta()
{
}

bool FSpriterMeta::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Parse the tagline array (optional)
	const TArray<TSharedPtr<FJsonValue>>* TagLineDescriptors;
	if (Tree->TryGetArrayField(TEXT("tagline"), /*out*/ TagLineDescriptors))
	{
		for (const TSharedPtr<FJsonValue> TagLineDescriptor : *TagLineDescriptors)
		{
			FSpriterTagLine& TagLine = *new (TagLines) FSpriterTagLine();
			const bool bParsedTagLineSuccessfully = TagLine.ParseFromJSON(TagLineDescriptor->AsObject(), NameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedTagLineSuccessfully;
		}
	}

	// Parse the valline array (optional)
	const TArray<TSharedPtr<FJsonValue>>* ValLineDescriptors;
	if (Tree->TryGetArrayField(TEXT("valline"), /*out*/ ValLineDescriptors))
	{
		for (const TSharedPtr<FJsonValue> ValLineDescriptor : *ValLineDescriptors)
		{
			FSpriterValLine& ValLine = *new (ValLines) FSpriterValLine();
			const bool bParsedValLineProperly = ValLine.ParseFromJSON(ValLineDescriptor->AsObject(), NameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedValLineProperly;
		}
	}

	UE_DO_SPRITER_AUDIT(KnownMetaKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterRef

FSpriterRefCommon::FSpriterRefCommon()
	: ParentIndex(INDEX_NONE)
	, TimelineIndex(INDEX_NONE)
	, KeyIndex(INDEX_NONE)
{
}

bool FSpriterRefCommon::ParseCommonFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	if (!Tree->TryGetNumberField(TEXT("key"), /*out*/ KeyIndex))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'key' field in the ref object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	if (!Tree->TryGetNumberField(TEXT("parent"), /*out*/ ParentIndex))
	{
		ParentIndex = INDEX_NONE;
	}

	if (!Tree->TryGetNumberField(TEXT("timeline"), /*out*/ TimelineIndex))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'timeline' field in the ref object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterRef

bool FSpriterRef::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	const bool bSuccessfullyParsed = ParseCommonFromJSON(Tree, NameForErrors, bSilent);

	UE_DO_SPRITER_AUDIT(KnownRefKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterObjectRef

FSpriterObjectRef::FSpriterObjectRef()
	: ZIndex(0)
{
}

bool FSpriterObjectRef::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = ParseCommonFromJSON(Tree, NameForErrors, bSilent);

	if (!Tree->TryGetNumberField(TEXT("z_index"), /*out*/ ZIndex))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'z_index' field in the object ref object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownObjectRefKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterMainlineKey

FSpriterMainlineKey::FSpriterMainlineKey()
	: TimeInMS(INDEX_NONE)
	, CurveType(ESpriterCurveType::INVALID)
{
}

bool FSpriterMainlineKey::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Read the time of the key (in milliseconds)
	if (!Tree->TryGetNumberField(TEXT("time"), /*out*/ TimeInMS))
	{
		// Assume 0 when missing?
		TimeInMS = 0;
	}

	// Parse the bone_ref array
	const TArray<TSharedPtr<FJsonValue>>* BoneRefDescriptors;
	if (Tree->TryGetArrayField(TEXT("bone_ref"), /*out*/ BoneRefDescriptors))
	{
		for (TSharedPtr<FJsonValue> BoneRefDescriptor : *BoneRefDescriptors)
		{
			FSpriterRef& BoneRef = *new (BoneRefs) FSpriterRef();
			const bool bParsedBoneRefOK = BoneRef.ParseFromJSON(BoneRefDescriptor->AsObject(), NameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedBoneRefOK;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'bone_ref' field in '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	// Optionally parse the curve_type property
	FString CurveTypeAsString;
	if (Tree->TryGetStringField(TEXT("curve_type"), /*out*/ CurveTypeAsString))
	{
		CurveType = FSpriterEnumHelper::StringToCurveType(CurveTypeAsString);
		if (CurveType == ESpriterCurveType::INVALID)
		{
			SPRITER_IMPORT_ERROR(TEXT("Unknown value '%s' for 'curve_type' in '%s'."), *CurveTypeAsString, *NameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else
	{
		// Defaults to linear
		CurveType = ESpriterCurveType::Linear;
	}

	// Parse the object_ref array
	const TArray<TSharedPtr<FJsonValue>>* ObjectRefDescriptors;
	if (Tree->TryGetArrayField(TEXT("object_ref"), /*out*/ ObjectRefDescriptors))
	{
		for (TSharedPtr<FJsonValue> ObjectRefDescriptor : *ObjectRefDescriptors)
		{
			FSpriterObjectRef& ObjectRef = *new (ObjectRefs) FSpriterObjectRef();
			const bool bParsedObjectRefOK = ObjectRef.ParseFromJSON(ObjectRefDescriptor->AsObject(), NameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedObjectRefOK;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'object_ref' field in '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownMainlineKeyKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterTimelineKey

FSpriterTimelineKey::FSpriterTimelineKey()
	: TimeInMS(INDEX_NONE)
	, CurveType(ESpriterCurveType::INVALID)
	, C1(0.0)
	, C2(0.0)
	, Spin(1)
{
}

bool FSpriterTimelineKey::ParseBasicsFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Read the time of the key (in milliseconds)
	if (!Tree->TryGetNumberField(TEXT("time"), /*out*/ TimeInMS))
	{
		// Assume 0 when missing?
		TimeInMS = 0;
	}

	// Optionally parse the curve_type property
	FString CurveTypeAsString;
	if (Tree->TryGetStringField(TEXT("curve_type"), /*out*/ CurveTypeAsString))
	{
		CurveType = FSpriterEnumHelper::StringToCurveType(CurveTypeAsString);
		if (CurveType == ESpriterCurveType::INVALID)
		{
			SPRITER_IMPORT_ERROR(TEXT("Unknown value '%s' for 'curve_type' in '%s'."), *CurveTypeAsString, *NameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else
	{
		// Defaults to linear
		CurveType = ESpriterCurveType::Linear;
	}

	// Optionally parse c1 and c2
	Tree->TryGetNumberField(TEXT("c1"), /*out*/ C1);
	if ((C1 < 0.0) || (C1 > 1.0))
	{
		SPRITER_IMPORT_ERROR(TEXT("Unexpected value '%f' for 'c1' in '%s' (expected 0..1)."), C1, *NameForErrors);
		bSuccessfullyParsed = false;
	}

	Tree->TryGetNumberField(TEXT("c2"), /*out*/ C2);
	if ((C2 < 0.0) || (C2 > 1.0))
	{
		SPRITER_IMPORT_ERROR(TEXT("Unexpected value '%f' for 'c2' in '%s' (expected 0..1)."), C2, *NameForErrors);
		bSuccessfullyParsed = false;
	}

	// Optionally parse the spin
	Tree->TryGetNumberField(TEXT("spin"), /*out*/ Spin);
	if ((Spin != 1) && (Spin != -1) & (Spin != 0))
	{
		SPRITER_IMPORT_ERROR(TEXT("Unknown value '%d' for 'spin' in '%s' (expected -1, 0, or 1)."), Spin, *NameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownBasicTimelineKeyKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterFatTimelineKey

FSpriterFatTimelineKey::FSpriterFatTimelineKey()
	: Folder(INDEX_NONE)
	, File(INDEX_NONE)
	, bUseDefaultPivot(true)
	, PivotX(0.0f)
	, PivotY(1.0f)
{
}

bool FSpriterFatTimelineKey::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent, const ESpriterObjectType ObjectType)
{
	// Parse the common stuff shared for all object types
	bool bSuccessfullyParsed = ParseBasicsFromJSON(Tree, NameForErrors, bSilent);

	if (ObjectType == ESpriterObjectType::Bone)
	{
		// Parse the bone child
		const TSharedPtr<FJsonObject>* BoneDescriptor;
		if (Tree->TryGetObjectField(TEXT("bone"), /*out*/ BoneDescriptor))
		{
			const bool bParsedBoneOK = ParseBoneFromJSON(*BoneDescriptor, NameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed & bParsedBoneOK;
		}
		else
		{
			SPRITER_IMPORT_ERROR(TEXT("Expected a 'bone' field in '%s'."), *NameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else
	{
		// Parse the object child
		const TSharedPtr<FJsonObject>* ObjectDescriptor;
		if (Tree->TryGetObjectField(TEXT("object"), /*out*/ ObjectDescriptor))
		{
			const bool bParsedObjectOK = ParseObjectFromJSON(*ObjectDescriptor, NameForErrors, bSilent, ObjectType);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedObjectOK;
		}
		else
		{
			SPRITER_IMPORT_ERROR(TEXT("Expected a 'object' field in '%s'."), *NameForErrors);
			bSuccessfullyParsed = false;
		}
	}

	return bSuccessfullyParsed;
}

bool FSpriterFatTimelineKey::ParseBoneFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	const bool bSuccessfullyParsed = Info.ParseFromJSON(Tree, NameForErrors, bSilent);

	UE_DO_SPRITER_AUDIT(KnownTimelineBoneKeyKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

bool FSpriterFatTimelineKey::ParseObjectFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent, const ESpriterObjectType ObjectType)
{
	const bool bSuccessfullyParsed = Info.ParseFromJSON(Tree, NameForErrors, bSilent);

	Tree->TryGetNumberField(TEXT("file"), /*out*/ File);
	Tree->TryGetNumberField(TEXT("folder"), /*out*/ Folder);

	const bool bHasPivotX = Tree->TryGetNumberField(TEXT("pivot_x"), /*out*/ PivotX);
	const bool bHasPivotY = Tree->TryGetNumberField(TEXT("pivot_y"), /*out*/ PivotY);
	bUseDefaultPivot = !bHasPivotX && !bHasPivotY;

	UE_DO_SPRITER_AUDIT(KnownTimelineObjectKeyKeys, Tree, NameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterTimeline

FSpriterTimeline::FSpriterTimeline()
	: ObjectIndex(INDEX_NONE)
	, ObjectType(ESpriterObjectType::INVALID)
{
}

bool FSpriterTimeline::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the timeline name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the timeline object of '%s'."), *NameForErrors);
		Name = TEXT("(missing timeline name)");
		bSuccessfullyParsed = false;
	}
	const FString LocalNameForErrors = FString::Printf(TEXT("%s timeline '%s'"), *NameForErrors, *Name);

	// Optionally parse the object_type property
	FString ObjectTypeAsString;
	if (Tree->TryGetStringField(TEXT("object_type"), /*out*/ ObjectTypeAsString))
	{
		ObjectType = FSpriterEnumHelper::StringToObjectType(ObjectTypeAsString);
		if (ObjectType == ESpriterObjectType::INVALID)
		{
			SPRITER_IMPORT_ERROR(TEXT("Unknown value '%s' for 'object_type' in '%s'."), *ObjectTypeAsString, *LocalNameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else
	{
		// Defaults to sprite
		ObjectType = ESpriterObjectType::Sprite;
	}

	// Optionally parse the obj property
	if (!Tree->TryGetNumberField(TEXT("obj"), /*out*/ ObjectIndex))
	{
		ObjectIndex = INDEX_NONE;
	}

	// Parse the key array
	const TArray<TSharedPtr<FJsonValue>>* TimelineKeyDescriptors;
	if (Tree->TryGetArrayField(TEXT("key"), /*out*/ TimelineKeyDescriptors))
	{
		for (TSharedPtr<FJsonValue> TimelineKeyDescriptor : *TimelineKeyDescriptors)
		{
			FSpriterFatTimelineKey& Key = *new (Keys) FSpriterFatTimelineKey();
			const bool bParsedKeyOK = Key.ParseFromJSON(TimelineKeyDescriptor->AsObject(), LocalNameForErrors, bSilent, ObjectType);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedKeyOK;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'key' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Read the meta block (optional)
	const TSharedPtr<FJsonObject>* MetaDescriptor;
	if (Tree->TryGetObjectField(TEXT("meta"), /*out*/ MetaDescriptor))
	{
		const bool bParsedMetadataOK = Metadata.ParseFromJSON(*MetaDescriptor, NameForErrors, bSilent);
		bSuccessfullyParsed = bSuccessfullyParsed && bParsedMetadataOK;
	}

	UE_DO_SPRITER_AUDIT(KnownTimelineKeys, Tree, LocalNameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterEventLineKey

FSpriterEventLineKey::FSpriterEventLineKey()
	: TimeInMS(0)
{
}

bool FSpriterEventLineKey::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bParsedSuccessfully = true;

	// Read the time of the key (in milliseconds)
	if (!Tree->TryGetNumberField(TEXT("time"), /*out*/ TimeInMS))
	{
		// Assume 0 when missing?
		TimeInMS = 0;
	}

	UE_DO_SPRITER_AUDIT(KnownEventLineKeyKeys, Tree, NameForErrors);

	return bParsedSuccessfully;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterEventLine

FSpriterEventLine::FSpriterEventLine()
	: ObjectIndex(INDEX_NONE)
{
}

bool FSpriterEventLine::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the event line name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the event line of '%s'."), *NameForErrors);
		Name = TEXT("(missing event line name)");
		bSuccessfullyParsed = false;
	}
	const FString LocalNameForErrors = FString::Printf(TEXT("%s event line '%s'"), *NameForErrors, *Name);

	// Parse the object index
	if (!Tree->TryGetNumberField(TEXT("obj"), /*out*/ ObjectIndex))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'obj' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Parse the key array
	const TArray<TSharedPtr<FJsonValue>>* KeyDescriptors;
	if (Tree->TryGetArrayField(TEXT("key"), /*out*/ KeyDescriptors))
	{
		for (const TSharedPtr<FJsonValue> KeyDescriptor : *KeyDescriptors)
		{
			FSpriterEventLineKey& Key = *new (Keys) FSpriterEventLineKey();
			const bool bParsedKeySuccessfully = Key.ParseFromJSON(KeyDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedKeySuccessfully;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'key' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterAnimation

FSpriterAnimation::FSpriterAnimation()
	: LengthInMS(INDEX_NONE)
	, IntervalInMS(INDEX_NONE)
	, bIsLooping(true)
{
}

bool FSpriterAnimation::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the animation name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the animation object of '%s'."), *NameForErrors);
		Name = TEXT("(missing animation name)");
		bSuccessfullyParsed = false;
	}
	const FString LocalNameForErrors = FString::Printf(TEXT("%s animation '%s'"), *NameForErrors, *Name);

	// Read the length of the animation (in milliseconds)
	if (!Tree->TryGetNumberField(TEXT("length"), /*out*/ LengthInMS))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'length' field in the animation object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	// Read the interval of the animation (in milliseconds - I think this is probably optional (it's not mentioned in the reference))
	Tree->TryGetNumberField(TEXT("interval"), /*out*/ IntervalInMS);

	// Read the mainline
	const TSharedPtr<FJsonObject>* MainlineDescriptor;
	if (Tree->TryGetObjectField(TEXT("mainline"), /*out*/ MainlineDescriptor))
	{
		// Parse the keys array inside of the mainline object
		const TArray<TSharedPtr<FJsonValue>>* KeyDescriptors;
		if ((*MainlineDescriptor)->TryGetArrayField(TEXT("key"), /*out*/ KeyDescriptors))
		{
			for (TSharedPtr<FJsonValue> KeyDescriptor : *KeyDescriptors)
			{
				FSpriterMainlineKey& Key = *new (MainlineKeys) FSpriterMainlineKey();
				const bool bParsedKeyOK = Key.ParseFromJSON(KeyDescriptor->AsObject(), LocalNameForErrors, bSilent);
				bSuccessfullyParsed = bSuccessfullyParsed && bParsedKeyOK;
			}
		}
		else
		{
			SPRITER_IMPORT_ERROR(TEXT("Expected a 'key' field in the 'mainline' object in '%s'."), *LocalNameForErrors);
			bSuccessfullyParsed = false;
		}

		//@TODO: Should we do a sub-audit in the mainline object here?
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'mainline' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Read the looping flag
	if (!Tree->TryGetBoolField(TEXT("looping"), /*out*/ bIsLooping))
	{
		// Default to looping
		bIsLooping = true;
	}

	// Parse the timeline array
	const TArray<TSharedPtr<FJsonValue>>* TimelineDescriptors;
	if (Tree->TryGetArrayField(TEXT("timeline"), /*out*/ TimelineDescriptors))
	{
		for (TSharedPtr<FJsonValue> TimelineDescriptor : *TimelineDescriptors)
		{
			FSpriterTimeline& Timeline = *new (Timelines) FSpriterTimeline();
			const bool bParsedTimelineSuccessfully = Timeline.ParseFromJSON(TimelineDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedTimelineSuccessfully;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'timeline' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Read the eventline array (optional)
	const TArray<TSharedPtr<FJsonValue>>* EventLineDescriptors;
	if (Tree->TryGetArrayField(TEXT("eventline"), /*out*/ EventLineDescriptors))
	{
		for (const TSharedPtr<FJsonValue> EventLineDescriptor : *EventLineDescriptors)
		{
			FSpriterEventLine& EventLine = *new (EventLines) FSpriterEventLine();
			const bool bParsedEventLineOK = EventLine.ParseFromJSON(EventLineDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedEventLineOK;
		}
	}

	// Read the meta block (optional)
	const TSharedPtr<FJsonObject>* MetaDescriptor;
	if (Tree->TryGetObjectField(TEXT("meta"), /*out*/ MetaDescriptor))
	{
		const bool bParsedMetadataOK = Metadata.ParseFromJSON(*MetaDescriptor, NameForErrors, bSilent);
		bSuccessfullyParsed = bSuccessfullyParsed && bParsedMetadataOK;
	}

	//@TODO: Figure out what "gline": [], is

	UE_DO_SPRITER_AUDIT(KnownAnimationKeys, Tree, LocalNameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterCharacterMap

FSpriterCharacterMap::FSpriterCharacterMap()
{
}

bool FSpriterCharacterMap::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the character map name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the character_map object of '%s'."), *NameForErrors);
		Name = TEXT("(missing character_map name)");
		bSuccessfullyParsed = false;
	}
	const FString LocalNameForErrors = FString::Printf(TEXT("%s character map '%s'"), *NameForErrors, *Name);

	// Parse the map array
	const TArray<TSharedPtr<FJsonValue>>* MapDescriptors;
	if (Tree->TryGetArrayField(TEXT("map"), /*out*/ MapDescriptors))
	{
		for (TSharedPtr<FJsonValue> MapDescriptor : *MapDescriptors)
		{
			FSpriterMapInstruction& Map = *new (Maps) FSpriterMapInstruction();
			const bool bParsedMapInstructionSuccessfully = Map.ParseFromJSON(MapDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedMapInstructionSuccessfully;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'map' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownCharacterMapKeys, Tree, LocalNameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterVariableDefinition

FSpriterVariableDefinition::FSpriterVariableDefinition()
	: VariableType(ESpriterVariableType::INVALID)
	, DefaultValueNumber(0.0)
{
}

bool FSpriterVariableDefinition::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the variable name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the variable defintion of '%s'."), *NameForErrors);
		Name = TEXT("(missing variable name)");
		bSuccessfullyParsed = false;
	}
	const FString LocalNameForErrors = FString::Printf(TEXT("%s variable '%s'"), *NameForErrors, *Name);

	// Parse the type property
	FString VariableTypeAsString;
	if (Tree->TryGetStringField(TEXT("type"), /*out*/ VariableTypeAsString))
	{
		VariableType = FSpriterEnumHelper::StringToVariableType(VariableTypeAsString);
	}
	if (VariableType == ESpriterVariableType::INVALID)
	{
		SPRITER_IMPORT_ERROR(TEXT("Unknown value '%s' for 'type' in '%s'."), *VariableTypeAsString, *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Parse the default value
	if ((VariableType == ESpriterVariableType::Float) || (VariableType == ESpriterVariableType::Integer))
	{
		if (!Tree->TryGetNumberField(TEXT("default"), /*out*/ DefaultValueNumber))
		{
			SPRITER_IMPORT_ERROR(TEXT("Expected a number field named 'default' in '%s'."), *LocalNameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else if (VariableType == ESpriterVariableType::String)
	{
		if (!Tree->TryGetStringField(TEXT("default"), /*out*/ DefaultValueString))
		{
			SPRITER_IMPORT_ERROR(TEXT("Expected a string field named 'default' in '%s'."), *LocalNameForErrors);
			bSuccessfullyParsed = false;
		}
	}
	else if (VariableType != ESpriterVariableType::INVALID)
	{
		SPRITER_IMPORT_ERROR(TEXT("No handling for 'default' in '%s' for an unknown variable type (update this when a new type is added)."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownVariableDefinitionKeys, Tree, LocalNameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterObjectInfo

FSpriterObjectInfo::FSpriterObjectInfo()
	: Width(0.0)
	, Height(0.0)
	, PivotX(0.0)
	, PivotY(0.0)
	, ObjectType(ESpriterObjectType::INVALID)
{
}

bool FSpriterObjectInfo::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the object name
	FString ObjectNameAsString;
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ ObjectNameAsString))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the object of '%s'."), *NameForErrors);
		ObjectNameAsString = TEXT("(missing object name)");
		bSuccessfullyParsed = false;
	}
	ObjectName = *ObjectNameAsString;
	const FString LocalNameForErrors = FString::Printf(TEXT("%s object '%s'"), *NameForErrors, *ObjectNameAsString);

	// Parse the type property
	FString ObjectTypeAsString;
	if (Tree->TryGetStringField(TEXT("type"), /*out*/ ObjectTypeAsString))
	{
		ObjectType = FSpriterEnumHelper::StringToObjectType(ObjectTypeAsString);
	}
	if (ObjectType == ESpriterObjectType::INVALID)
	{
		SPRITER_IMPORT_ERROR(TEXT("Unknown value '%s' for 'type' in '%s'."), *ObjectTypeAsString, *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Optionally parse the width and height properties
	Tree->TryGetNumberField(TEXT("w"), /*out*/ Width);
	Tree->TryGetNumberField(TEXT("h"), /*out*/ Height);

	// Optionally parse the pivot properties
	Tree->TryGetNumberField(TEXT("pivot_x"), /*out*/ PivotX);
	Tree->TryGetNumberField(TEXT("pivot_y"), /*out*/ PivotY);

	//@TODO: Parse the frames[] field of an 'event' type (once I see one that isn't empty...)

	// Parse the var_defs array (optional; can be missing)
	const TArray<TSharedPtr<FJsonValue>>* VariableDefinitionDescriptors;
	if (Tree->TryGetArrayField(TEXT("var_defs"), /*out*/ VariableDefinitionDescriptors))
	{
		for (TSharedPtr<FJsonValue> VariableDefinitionDescriptor : *VariableDefinitionDescriptors)
		{
			FSpriterVariableDefinition& VariableDef = *new (VariableDefinitions)FSpriterVariableDefinition();
			const bool bParsedVariableDefOK = VariableDef.ParseFromJSON(VariableDefinitionDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedVariableDefOK;
		}
	}

	UE_DO_SPRITER_AUDIT(KnownObjInfoKeys, Tree, LocalNameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterEntity

FSpriterEntity::FSpriterEntity()
{
}

bool FSpriterEntity::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent)
{
	bool bSuccessfullyParsed = true;

	// Try parsing the entity name
	if (!Tree->TryGetStringField(TEXT("name"), /*out*/ Name))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the entity object of '%s'."), *NameForErrors);
		Name = TEXT("(missing entity name)");
		bSuccessfullyParsed = false;
	}
	const FString LocalNameForErrors = FString::Printf(TEXT("%s entity '%s'"), *NameForErrors, *Name);

	// Parse the obj_info array
	const TArray<TSharedPtr<FJsonValue>>* ObjectDescriptors;
	if (Tree->TryGetArrayField(TEXT("obj_info"), /*out*/ ObjectDescriptors))
	{
		for (TSharedPtr<FJsonValue> ObjectDescriptor : *ObjectDescriptors)
		{
			FSpriterObjectInfo& ObjectInfo = *new (Objects) FSpriterObjectInfo();
			const bool bParsedObjectInfoOK = ObjectInfo.ParseFromJSON(ObjectDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedObjectInfoOK;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'obj_info' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Parse the var_defs array (optional; can be missing)
	const TArray<TSharedPtr<FJsonValue>>* VariableDefinitionDescriptors;
	if (Tree->TryGetArrayField(TEXT("var_defs"), /*out*/ VariableDefinitionDescriptors))
	{
		for (TSharedPtr<FJsonValue> VariableDefinitionDescriptor : *VariableDefinitionDescriptors)
		{
			FSpriterVariableDefinition& VariableDef = *new (VariableDefinitions) FSpriterVariableDefinition();
			const bool bParsedVariableDefOK = VariableDef.ParseFromJSON(VariableDefinitionDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedVariableDefOK;
		}
	}

	// Parse the animation array
	const TArray<TSharedPtr<FJsonValue>>* AnimationDescriptors;
	if (Tree->TryGetArrayField(TEXT("animation"), /*out*/ AnimationDescriptors))
	{
		for (TSharedPtr<FJsonValue> AnimationDescriptor : *AnimationDescriptors)
		{
			FSpriterAnimation& Animation = *new (Animations) FSpriterAnimation();
			const bool bParsedAnimationOK = Animation.ParseFromJSON(AnimationDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedAnimationOK;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'animation' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	// Parse the character_map array
	const TArray<TSharedPtr<FJsonValue>>* CharacterMapDescriptors;
	if (Tree->TryGetArrayField(TEXT("character_map"), /*out*/ CharacterMapDescriptors))
	{
		for (TSharedPtr<FJsonValue> CharacterMapDescriptor : *CharacterMapDescriptors)
		{
			FSpriterCharacterMap& CharacterMap = *new (CharacterMaps) FSpriterCharacterMap();
			const bool bParsedCharacterMapSuccessfully = CharacterMap.ParseFromJSON(CharacterMapDescriptor->AsObject(), LocalNameForErrors, bSilent);
			bSuccessfullyParsed = bSuccessfullyParsed && bParsedCharacterMapSuccessfully;
		}
	}
	else
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'character_map' field in '%s'."), *LocalNameForErrors);
		bSuccessfullyParsed = false;
	}

	UE_DO_SPRITER_AUDIT(KnownEntityKeys, Tree, LocalNameForErrors);

	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////
// FSpriterSCON

FSpriterSCON::FSpriterSCON()
	: bSuccessfullyParsed(false)
{
}

void FSpriterSCON::ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent, bool bPreparseOnly)
{
	bSuccessfullyParsed = true;

	// Try parsing the SCON version
	if (!Tree->TryGetStringField(TEXT("scon_version"), /*out*/ SconVersion))
	{
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'scon_version' field in the top level object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	// Try parsing the generator and generator version strings
	if (!Tree->TryGetStringField(TEXT("generator"), /*out*/ Generator))
	{
		// No good, probably isn't the right kind of file
		Generator = FString();
		SPRITER_IMPORT_ERROR(TEXT("Expected a 'generator' field in the top level object of '%s'."), *NameForErrors);
		bSuccessfullyParsed = false;
	}

	if (!Tree->TryGetStringField(TEXT("generator_version"), /*out*/ GeneratorVersion))
	{
		GeneratorVersion = TEXT("(missing generator_version)");
	}

	// Validate the SCON version
	const FString ExpectedSCONVersion(TEXT("1.0"));
	if (SconVersion != ExpectedSCONVersion)
	{
		// Not 100% we can handle it but we'll try
		SPRITER_IMPORT_WARNING(TEXT("Unknown 'scon_version' '%s' (expected '%s') SCON file '%s'.  Parsing will continue but the format may not be fully supported"), *SconVersion, *ExpectedSCONVersion, *NameForErrors);
	}

	// Validate the generator
	const FString BrashMonkeySpriterGenerator(TEXT("BrashMonkey Spriter"));
	if (Generator.StartsWith(BrashMonkeySpriterGenerator))
	{
		// Cool, we (mostly) know how to handle these sorts of files!
		if (!bSilent)
		{
			UE_LOGF(LogSpriterImporter, Log, "Parsing Spriter character SCON v%ls exported from '%ls' '%ls'", *SconVersion, *Generator, *GeneratorVersion);
		}
	}
	else if (!Generator.IsEmpty())
	{
		// Not 100% we can handle it but we'll try
		SPRITER_IMPORT_WARNING(TEXT("Unexpected 'generator' named '%s' '%s' while parsing SCON v%s file '%s'.  Parsing will continue but the format may not be fully supported"), *Generator, *GeneratorVersion, *SconVersion, *NameForErrors);
	}
	
	// Load the rest of the data if we're doing a full parse
	if (!bPreparseOnly)
	{
		// Parse the entities array
		const TArray<TSharedPtr<FJsonValue>>* EntityDescriptors;
		if (Tree->TryGetArrayField(TEXT("entity"), /*out*/ EntityDescriptors))
		{
			for (TSharedPtr<FJsonValue> EntityDescriptor : *EntityDescriptors)
			{
				FSpriterEntity& Entity = *new (Entities) FSpriterEntity();
				const bool bParsedEntityOK = Entity.ParseFromJSON(EntityDescriptor->AsObject(), NameForErrors, bSilent);
				bSuccessfullyParsed = bSuccessfullyParsed && bParsedEntityOK;
			}
		}
		else
		{
			SPRITER_IMPORT_ERROR(TEXT("JSON exported from Spriter in file '%s' has no entities."), *NameForErrors);
			bSuccessfullyParsed = false;
		}

		// Parse the folders array
		const TArray<TSharedPtr<FJsonValue>>* FolderDescriptors;
		if (Tree->TryGetArrayField(TEXT("folder"), /*out*/ FolderDescriptors))
		{
			for (TSharedPtr<FJsonValue> FolderDescriptor : *FolderDescriptors)
			{
				FSpriterFolder& Folder = *new (Folders) FSpriterFolder();
				const bool bParsedFolderOK = Folder.ParseFromJSON(FolderDescriptor->AsObject(), NameForErrors, bSilent);
				bSuccessfullyParsed = bSuccessfullyParsed && bParsedFolderOK;
			}
		}
		else
		{
			SPRITER_IMPORT_ERROR(TEXT("JSON exported from Spriter in file '%s' has no folders."), *NameForErrors);
			bSuccessfullyParsed = false;
		}

		// Parse the tag list array (optional)
		const TArray<TSharedPtr<FJsonValue>>* TagListDescriptors;
		if (Tree->TryGetArrayField(TEXT("tag_list"), /*out*/ TagListDescriptors))
		{
			for (const TSharedPtr<FJsonValue> TagListDescriptorUntyped : *TagListDescriptors)
			{
				const TSharedPtr<FJsonObject> TagListDescriptor = TagListDescriptorUntyped->AsObject();

				FString NewTag;
				if (TagListDescriptor->TryGetStringField(TEXT("name"), /*out*/ NewTag))
				{
					Tags.Add(NewTag);
				}
				else
				{
					SPRITER_IMPORT_ERROR(TEXT("Expected a 'name' field in the tag object in file '%s'."), *NameForErrors);
					bSuccessfullyParsed = false;
				}

				UE_DO_SPRITER_AUDIT(KnownSCONTagListKeys, TagListDescriptor, NameForErrors);
			}
		}
	}

	UE_DO_SPRITER_AUDIT(KnownSCONKeys, Tree, NameForErrors);
}

bool FSpriterSCON::IsValid() const
{
	return bSuccessfullyParsed;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE