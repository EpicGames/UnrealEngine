// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "SpriterDataModel.generated.h"

// This file contains the definition of various Spriter data types
// Reference http://www.brashmonkey.com/ScmlDocs/ScmlReference.html and a bunch of exported files

//////////////////////////////////////////////////////////////////////////

UENUM()
enum class ESpriterObjectType : uint8
{
	INVALID,
	/** Default when not specified */
	Sprite,
	Bone,
	Box,
	Point,
	Sound, //?
	Entity, //?
	Variable, //?
	Event
};

UENUM()
enum class ESpriterCurveType : uint8
{
	INVALID,
	Instant,
	/** Default when not specified */
	Linear,
	Quadratic,
	Cubic
};

UENUM()
enum class ESpriterVariableType : uint8
{
	INVALID,
	Float,
	Integer,
	String
};

UENUM()
enum class ESpriterFileType : uint8
{
	INVALID,
	Sprite,
	Sound
};

struct FSpriterEnumHelper
{
public:
	static ESpriterObjectType StringToObjectType(const FString& InString);
	static ESpriterCurveType StringToCurveType(const FString& InString);
	static ESpriterVariableType StringToVariableType(const FString& InString);
	static ESpriterFileType StringToFileType(const FString& InString);

private:
	FSpriterEnumHelper() {}
};

//////////////////////////////////////////////////////////////////////////
// FSpriterSpatialInfo

USTRUCT()
struct FSpriterSpatialInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double X;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double Y;

	// Angle (in degrees)
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double AngleInDegrees;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double ScaleX;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double ScaleY;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FLinearColor Color;

public:
	FSpriterSpatialInfo();

	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);

	FTransform ConvertToTransform() const;
};

//////////////////////////////////////////////////////////////////////////
// FSpriterFile

USTRUCT()
struct FSpriterFile
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double PivotX;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double PivotY;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 Width;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 Height;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	ESpriterFileType FileType;

public:
	FSpriterFile();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterFolder

USTRUCT()
struct FSpriterFolder
{
public:
	GENERATED_USTRUCT_BODY()
		
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Name;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterFile> Files;

public:
	FSpriterFolder();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterMapInstruction

USTRUCT()
struct FSpriterMapInstruction
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 Folder;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 File;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TargetFolder;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TargetFile;

public:
	FSpriterMapInstruction();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterTagLineKey

USTRUCT()
struct FSpriterTagLineKey
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TimeInMS;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<int32> Tags;
	
public:
	FSpriterTagLineKey();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterTagLine

USTRUCT()
struct FSpriterTagLine
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterTagLineKey> Keys;

public:
 	FSpriterTagLine();
 	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterValLineKey

USTRUCT()
struct FSpriterValLineKey
{
public:
	GENERATED_USTRUCT_BODY()
		
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TimeInMS;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	bool bReadAsNumber;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double ValueAsNumber;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString ValueAsString;

public:
	FSpriterValLineKey();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterValLine

USTRUCT()
struct FSpriterValLine
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterValLineKey> Keys;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 DefinitionIndex;

public:
	FSpriterValLine();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterMeta

USTRUCT()
struct FSpriterMeta
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterTagLine> TagLines;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterValLine> ValLines;

public:
	FSpriterMeta();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterRefCommon

USTRUCT()
struct FSpriterRefCommon
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 ParentIndex;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TimelineIndex;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 KeyIndex;

public:
	FSpriterRefCommon();
	bool ParseCommonFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterRef

USTRUCT()
struct FSpriterRef : public FSpriterRefCommon
{
public:
	GENERATED_USTRUCT_BODY()

	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterObjectRef

USTRUCT()
struct FSpriterObjectRef : public FSpriterRefCommon
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 ZIndex;

public:
	FSpriterObjectRef();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterMainlineKey

USTRUCT()
struct FSpriterMainlineKey
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TimeInMS;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterRef> BoneRefs;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterObjectRef> ObjectRefs;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	ESpriterCurveType CurveType;

public:
	FSpriterMainlineKey();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterTimelineKey

USTRUCT()
struct FSpriterTimelineKey
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TimeInMS;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	ESpriterCurveType CurveType;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double C1;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double C2;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 Spin;

public:
	FSpriterTimelineKey();
	bool ParseBasicsFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterFatTimelineKey

USTRUCT()
struct FSpriterFatTimelineKey : public FSpriterTimelineKey
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FSpriterSpatialInfo Info;

	// Only valid in 'object' keys (Sprite, Point, etc...)
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 Folder;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 File;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double PivotX;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double PivotY;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	bool bUseDefaultPivot;

	// Overrides linear!
public:
	FSpriterFatTimelineKey();

	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent, const ESpriterObjectType ObjectType);
	bool ParseBoneFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
	bool ParseObjectFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent, const ESpriterObjectType ObjectType);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterTimeline

USTRUCT()
struct FSpriterTimeline
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category="Spriter")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 ObjectIndex;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	ESpriterObjectType ObjectType;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterFatTimelineKey> Keys;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FSpriterMeta Metadata;

public:
	FSpriterTimeline();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterEventLineKey

USTRUCT()
struct FSpriterEventLineKey
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 TimeInMS;

public:
	FSpriterEventLineKey();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterEventLine

USTRUCT()
struct FSpriterEventLine
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterEventLineKey> Keys;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 ObjectIndex;

public:
	FSpriterEventLine();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterAnimation

USTRUCT()
struct FSpriterAnimation
{
public:
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 LengthInMS;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	int32 IntervalInMS;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	bool bIsLooping;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FSpriterMeta Metadata;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterMainlineKey> MainlineKeys;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterTimeline> Timelines;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterEventLine> EventLines;

public:
	FSpriterAnimation();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterCharacterMap

USTRUCT()
struct FSpriterCharacterMap
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Name;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterMapInstruction> Maps;

public:
	FSpriterCharacterMap();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterVariableDefinition

USTRUCT()
struct FSpriterVariableDefinition
{
public:
	GENERATED_USTRUCT_BODY()
		
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	ESpriterVariableType VariableType;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double DefaultValueNumber;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString DefaultValueString;

public:
	FSpriterVariableDefinition();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterObjectInfo

USTRUCT()
struct FSpriterObjectInfo
{
public:
	GENERATED_USTRUCT_BODY()
		
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FName ObjectName;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double Width;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double Height;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double PivotX;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	double PivotY;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	ESpriterObjectType ObjectType;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterVariableDefinition> VariableDefinitions;

public:
	FSpriterObjectInfo();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterEntity

USTRUCT()
struct FSpriterEntity
{
public:
	GENERATED_USTRUCT_BODY()
		
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Name;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterCharacterMap> CharacterMaps;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterAnimation> Animations;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterObjectInfo> Objects;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterVariableDefinition> VariableDefinitions;

public:
	FSpriterEntity();
	bool ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent);
};

//////////////////////////////////////////////////////////////////////////
// FSpriterSCON

USTRUCT()
struct FSpriterSCON
{
public:
	GENERATED_USTRUCT_BODY()
		
	//"generator" : "BrashMonkey Spriter",
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString Generator;

	//"generator_version" : "r2",
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString GeneratorVersion;

	//"scon_version" : "1.0"
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	FString SconVersion;

	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterFolder> Folders;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FSpriterEntity> Entities;
	
	UPROPERTY(VisibleAnywhere, Category = "Spriter")
	TArray<FString> Tags;

	bool bSuccessfullyParsed;

public:
	FSpriterSCON();

	void ParseFromJSON(TSharedPtr<FJsonObject> Tree, const FString& NameForErrors, bool bSilent, bool bPreparseOnly);

	bool IsValid() const;
};

