// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpriterImporterFactory.h"
#include "SpriterImporterLog.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "PaperJSONHelpers.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageTools.h"
#include "PaperSpriterImportData.h"
#include "PaperImporterSettings.h"

#include "PaperSprite.h"

#define LOCTEXT_NAMESPACE "SpriterImporter"

//////////////////////////////////////////////////////////////////////////
// FBoneHierarchyBuilder

struct FBoneHierarchyBuilder
{
	TArray<FName> RootBones;
	TArray<FName> AllBones;
	TArray<int32> ParentIndices;
	TArray<FTransform> Transforms;

	bool bCreatedDummyRoot;
	
	FBoneHierarchyBuilder()
		: bCreatedDummyRoot(false)
	{
	}

	void ProcessBone(FName BoneName, int32 ExpectedParentIndex, const FTransform& InitialTransform, const FString& AnimationNameForErrors, int32 TimeForErrors);
	void ProcessHierarchy(const FSpriterEntity& Entity);
	void CopyToRefSkeleton(FReferenceSkeleton& RefSkeleton);
};

void FBoneHierarchyBuilder::CopyToRefSkeleton(FReferenceSkeleton& RefSkeleton)
{
	for (int32 SourceBoneIndex = 0; SourceBoneIndex < AllBones.Num(); ++SourceBoneIndex)
	{
		const FName BoneName(AllBones[SourceBoneIndex]);
		const int32 ParentIndex(ParentIndices[SourceBoneIndex]);
		const FMeshBoneInfo BoneInfo(BoneName, BoneName.ToString(), ParentIndex);
		const FTransform& Transform = Transforms[SourceBoneIndex];
		RefSkeleton.Add(BoneInfo, Transform);
	}
}

void FBoneHierarchyBuilder::ProcessBone(FName BoneName, int32 ExpectedParentIndex, const FTransform& InitialTransform, const FString& AnimationNameForErrors, int32 TimeForErrors)
{
	const int32 BoneNameAlreadyInsertedIndex = AllBones.Find(BoneName);
	if (BoneNameAlreadyInsertedIndex == INDEX_NONE)
	{
		if (ExpectedParentIndex == INDEX_NONE)
		{
			RootBones.Add(BoneName);
		}
		AllBones.Add(BoneName);
		ParentIndices.Add(ExpectedParentIndex);
		Transforms.Add(InitialTransform);
	}
	else
	{
		// Verify that the hierarchy hasn't changed
		if (ParentIndices[BoneNameAlreadyInsertedIndex] != ExpectedParentIndex)
		{
			UE_LOGF(LogInit, Warning, "Bone hierarchy (for bone '%ls') in animation '%ls' was changed at time %d ms.  This change will be ignored and the animation will not play properly.", *BoneName.ToString(), *AnimationNameForErrors, TimeForErrors);
		}
	}
}

void FBoneHierarchyBuilder::ProcessHierarchy(const FSpriterEntity& Entity)
{

	// Build a list of bones (no known parents yet)
	for (const FSpriterObjectInfo& ObjectInfo : Entity.Objects)
	{
		if (ObjectInfo.ObjectType == ESpriterObjectType::Bone)
		{
		}
	}


	// Run thru every key in every animation to make sure things are looking good
	for (const FSpriterAnimation& Animation : Entity.Animations)
	{
		for (int32 MainKeyIndex = 0; MainKeyIndex < Animation.MainlineKeys.Num(); ++MainKeyIndex)
		{
			const FSpriterMainlineKey& MainKey = Animation.MainlineKeys[MainKeyIndex];

			for (int32 BoneRefIndex = 0; BoneRefIndex < MainKey.BoneRefs.Num(); ++BoneRefIndex)
			{
				const FSpriterRef& BoneRef = MainKey.BoneRefs[BoneRefIndex];
				ensure(BoneRef.ParentIndex < BoneRefIndex);

				const FSpriterTimeline& AssociatedTimeline = Animation.Timelines[BoneRef.TimelineIndex];

				if (ensure(AssociatedTimeline.ObjectType == ESpriterObjectType::Bone))
				{
					if (ensure(AssociatedTimeline.ObjectIndex != INDEX_NONE))
					{
						const FSpriterObjectInfo& AssociatedObject = Entity.Objects[AssociatedTimeline.ObjectIndex];

						const FName BoneName(AssociatedObject.ObjectName);

						const FSpriterFatTimelineKey& TimelineKey = AssociatedTimeline.Keys[BoneRef.KeyIndex];
						const FTransform InitialTransform = TimelineKey.Info.ConvertToTransform();

						ProcessBone(BoneName, BoneRef.ParentIndex, InitialTransform, Animation.Name, MainKey.TimeInMS);
					}
				}
			}
		}
	}

	if (RootBones.Num() > 1)
	{
		//@TODO: Handle the case where there are multiple root bones
		UE_LOGF(LogSpriterImporter, Warning, "The spriter entity '%ls' has more than one root bone, which isn't handled correctly yet", *Entity.Name);
	}
}

//////////////////////////////////////////////////////////////////////////

static ESpritePivotMode::Type ConvertNormalizedPivotPointToPivotMode(double PivotX, double PivotY)
{
	// Determine the ideal pivot
	const bool bIsLeft = FMath::IsNearlyEqual(PivotX, 0.0);
	const bool bIsCenterX = FMath::IsNearlyEqual(PivotX, 0.5);
	const bool bIsRight = FMath::IsNearlyEqual(PivotX, 1.0);
	const bool bIsTop = FMath::IsNearlyEqual(PivotY, 0.0);
	const bool bIsCenterY = FMath::IsNearlyEqual(PivotY, 0.5);
	const bool bIsBottom = FMath::IsNearlyEqual(PivotY, 1.0);

	ESpritePivotMode::Type PivotMode = ESpritePivotMode::Custom;
	if (bIsLeft)
	{
		if (bIsTop)
		{
			PivotMode = ESpritePivotMode::Top_Left;
		}
		else if (bIsCenterY)
		{
			PivotMode = ESpritePivotMode::Center_Left;
		}
		else if (bIsBottom)
		{
			PivotMode = ESpritePivotMode::Bottom_Left;
		}
	}
	else if (bIsCenterX)
	{
		if (bIsTop)
		{
			PivotMode = ESpritePivotMode::Top_Center;
		}
		else if (bIsCenterY)
		{
			PivotMode = ESpritePivotMode::Center_Center;
		}
		else if (bIsBottom)
		{
			PivotMode = ESpritePivotMode::Bottom_Center;
		}
	}
	else if (bIsRight)
	{
		if (bIsTop)
		{
			PivotMode = ESpritePivotMode::Top_Right;
		}
		else if (bIsCenterY)
		{
			PivotMode = ESpritePivotMode::Center_Right;
		}
		else if (bIsBottom)
		{
			PivotMode = ESpritePivotMode::Bottom_Right;
		}
	}

	return PivotMode;
}

//////////////////////////////////////////////////////////////////////////
// USpriterImporterFactory

USpriterImporterFactory::USpriterImporterFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	SupportedClass = UPaperSpriterImportData::StaticClass();

	bEditorImport = true;
	bText = true;

	Formats.Add(TEXT("scon;Spriter SCON file"));
}

FText USpriterImporterFactory::GetToolTip() const
{
	return LOCTEXT("SpriterImporterFactoryDescription", "Characters exported from Spriter");
}

bool USpriterImporterFactory::FactoryCanImport(const FString& Filename)
{
	FString FileContent;
	if (FFileHelper::LoadFileToString(/*out*/ FileContent, *Filename))
	{
		TSharedPtr<FJsonObject> DescriptorObject = ParseJSON(FileContent, FString(), /*bSilent=*/ true);
		if (DescriptorObject.IsValid())
		{
			FSpriterSCON GlobalInfo;
			GlobalInfo.ParseFromJSON(DescriptorObject, Filename, /*bSilent=*/ true, /*bPreparseOnly=*/ true);

			return GlobalInfo.IsValid();
		}
	}

	return false;
}

UObject* USpriterImporterFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	Flags |= RF_Transactional;

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

 	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
 
 	bool bLoadedSuccessfully = true;
 
 	const FString CurrentFilename = UFactory::GetCurrentFilename();
 	FString CurrentSourcePath;
 	FString FilenameNoExtension;
 	FString UnusedExtension;
 	FPaths::Split(CurrentFilename, CurrentSourcePath, FilenameNoExtension, UnusedExtension);
 
 	const FString LongPackagePath = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName());
 
 	const FString NameForErrors(InName.ToString());
 	const FString FileContent(BufferEnd - Buffer, Buffer);
 	TSharedPtr<FJsonObject> DescriptorObject = ParseJSON(FileContent, NameForErrors);

	UPaperSpriterImportData* Result = nullptr;
 
	// Parse the file 
	FSpriterSCON DataModel;
	if (DescriptorObject.IsValid())
	{
		DataModel.ParseFromJSON(DescriptorObject, NameForErrors, /*bSilent=*/ false, /*bPreParseOnly=*/ false);
	}

	// Create the new 'hub' asset and convert the data model over
	if (DataModel.IsValid())
	{
		const bool bSilent = false;

		Result = NewObject<UPaperSpriterImportData>(InParent, InName, Flags);
		Result->Modify();

		//@TODO: Do some things here maybe?
		Result->ImportedData = DataModel;


		// Import the assets in the folders
		for (const FSpriterFolder& Folder : DataModel.Folders)
		{
			for (const FSpriterFile& File : Folder.Files)
			{
				const FString RelativeFilename = File.Name.Replace(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
				const FString SourceSpriterFilePath = FPaths::Combine(*CurrentSourcePath, *RelativeFilename);

				FString RelativeDestPath;
				FString JustFilename;
				FString JustExtension;
				FPaths::Split(RelativeFilename, /*out*/ RelativeDestPath, /*out*/ JustFilename, /*out*/ JustExtension);

				if (File.FileType == ESpriterFileType::Sprite)
				{
					const FString TargetTexturePath = LongPackagePath / TEXT("Textures") / RelativeDestPath;
					const FString TargetSpritePath = LongPackagePath / TEXT("Sprites") / RelativeDestPath;

					// Import the texture
					UTexture2D* ImportedTexture = ImportTexture(SourceSpriterFilePath, TargetTexturePath);

					if (ImportTexture == nullptr)
					{
						SPRITER_IMPORT_ERROR(TEXT("Failed to import texture '%s' while importing '%s'"), *SourceSpriterFilePath, *CurrentFilename);
					}

					// Create a sprite from it
					UPaperSprite* ImportedSprite = CastChecked<UPaperSprite>(CreateNewAsset(UPaperSprite::StaticClass(), TargetSpritePath, JustFilename, Flags));

					const ESpritePivotMode::Type PivotMode = ConvertNormalizedPivotPointToPivotMode(File.PivotX, File.PivotY);
					const double PivotInPixelsX = File.Width * File.PivotX;
					const double PivotInPixelsY = File.Height * File.PivotY;

					ImportedSprite->SetPivotMode(PivotMode, FVector2D((float)PivotInPixelsX, (float)PivotInPixelsY));

					FSpriteAssetInitParameters SpriteInitParams;
					SpriteInitParams.SetTextureAndFill(ImportedTexture);
					GetDefault<UPaperImporterSettings>()->ApplySettingsForSpriteInit(SpriteInitParams);
					SpriteInitParams.SetPixelsPerUnrealUnit(1.0f);
					ImportedSprite->InitializeSprite(SpriteInitParams);
				}
				else if (File.FileType == ESpriterFileType::Sound)
				{
					// Import the sound
					const FString TargetAssetPath = LongPackagePath / RelativeDestPath;
					UObject* ImportedSound = ImportAsset(SourceSpriterFilePath, TargetAssetPath);
				}
				else if (File.FileType != ESpriterFileType::INVALID)
				{
					ensureMsgf(false, TEXT("Importer was not updated when a new entry was added to ESpriterFileType"));
				}
					// 		TMap<FString, class UTexture2D*> ImportedTextures;
					// 		TMap<FString, class UPaperSprite> ImportedSprites;

			}
		}

		for (const FSpriterEntity& Entity : DataModel.Entities)
		{
			// Extract the common/shared skeleton
			FBoneHierarchyBuilder HierarchyBuilder;
			HierarchyBuilder.ProcessHierarchy(Entity);

			// Create the skeletal mesh
			const FString TargetMeshName = Entity.Name + TEXT("_SkelMesh");
			const FString TargetMeshPath = LongPackagePath;
			USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(CreateNewAsset(USkeletalMesh::StaticClass(), TargetMeshPath, TargetMeshName, Flags));

			// Create the skeleton
			const FString TargetSkeletonName = Entity.Name + TEXT("_Skeleton");
			const FString TargetSkeletonPath = LongPackagePath;
			USkeleton* EntitySkeleton = CastChecked<USkeleton>(CreateNewAsset(USkeleton::StaticClass(), TargetSkeletonPath, TargetSkeletonName, Flags));

			// Initialize the mesh asset
			FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
			check(ImportedModel->LODModels.Num() == 0);
			ImportedModel->LODModels.Empty();
			ImportedModel->EmptyOriginalReductionSourceMeshData();
			FSkeletalMeshLODModel& LODModel = *new (ImportedModel->LODModels) FSkeletalMeshLODModel();

			SkeletalMesh->ResetLODInfo()
			SkeletalMesh->AddLODInfo();
			SkeletalMesh->GetLODInfo(0)->LODHysteresis = 0.02f;
			FSkeletalMeshOptimizationSettings Settings;
			// set default reduction settings values
			SkeletalMesh->GetLODInfo(0)->ReductionSettings = Settings;

			// Create initial bounding box based on expanded version of reference pose for meshes without physics assets. Can be overridden by artist.
// 			FBox BoundingBox(SkelMeshImportDataPtr->Points.GetData(), SkelMeshImportDataPtr->Points.Num());
// 			FBox Temp = BoundingBox;
// 			FVector MidMesh = 0.5f*(Temp.Min + Temp.Max);
// 			BoundingBox.Min = Temp.Min + 1.0f*(Temp.Min - MidMesh);
// 			BoundingBox.Max = Temp.Max + 1.0f*(Temp.Max - MidMesh);
// 			// Tuck up the bottom as this rarely extends lower than a reference pose's (e.g. having its feet on the floor).
// 			// Maya has Y in the vertical, other packages have Z.
// 			//BEN const int32 CoordToTuck = bAssumeMayaCoordinates ? 1 : 2;
// 			//BEN BoundingBox.Min[CoordToTuck]	= Temp.Min[CoordToTuck] + 0.1f*(Temp.Min[CoordToTuck] - MidMesh[CoordToTuck]);
// 			BoundingBox.Min[2] = Temp.Min[2] + 0.1f*(Temp.Min[2] - MidMesh[2]);
// 			SkeletalMesh->Bounds = FBoxSphereBounds(BoundingBox);

			// Store whether or not this mesh has vertex colors
// 			SkeletalMesh->bHasVertexColors = SkelMeshImportDataPtr->bHasVertexColors;

			// Pass the number of texture coordinate sets to the LODModel.  Ensure there is at least one UV coord
			LODModel.NumTexCoords = 1;// FMath::Max<uint32>(1, SkelMeshImportDataPtr->NumTexCoords);


			// Create the reference skeleton and update LOD0
			FReferenceSkeleton& RefSkeleton = SkeletalMesh->RefSkeleton;
			HierarchyBuilder.CopyToRefSkeleton(RefSkeleton);
			SkeletalMesh->CalculateRequiredBones(LODModel, RefSkeleton, /*BonesToRemove=*/ nullptr);
			SkeletalMesh->CalculateInvRefMatrices();

			// Initialize the skeleton asset
			EntitySkeleton->MergeAllBonesToBoneTree(SkeletalMesh);

			// Point the mesh and skeleton at each other
			SkeletalMesh->Skeleton = EntitySkeleton;
			EntitySkeleton->SetPreviewMesh(SkeletalMesh);

			// Create the animations
			for (const FSpriterAnimation& Animation : Entity.Animations)
			{
				//@TODO: That thing I said...

				const FString TargetAnimationName = Animation.Name;
				const FString TargetAnimationPath = LongPackagePath / TEXT("Animations");
				UAnimSequence* AnimationAsset = CastChecked<UAnimSequence>(CreateNewAsset(UAnimSequence::StaticClass(), TargetAnimationPath, TargetAnimationName, Flags));

				AnimationAsset->SetSkeleton(EntitySkeleton);

				// if you have one pose(thus 0.f duration), it still contains animation, so we'll need to consider that as MINIMUM_ANIMATION_LENGTH time length
				const float DurationInSeconds = Animation.LengthInMS * 0.001f;
				AnimationAsset->SequenceLength = FMath::Max<float>(DurationInSeconds, MINIMUM_ANIMATION_LENGTH);

				const bool bSourceDataExists = (AnimationAsset->SourceRawAnimationData.Num() > 0);
				TArray<struct FRawAnimSequenceTrack>& RawAnimationData = bSourceDataExists ? AnimationAsset->SourceRawAnimationData : AnimationAsset->RawAnimationData;




				int32 TotalNumKeys = 0;
				for (const FSpriterTimeline& Timeline : Animation.Timelines)
				{
					if (Timeline.ObjectType != ESpriterObjectType::Bone)
					{
						continue;
					}

					const FName BoneName = Entity.Objects[Timeline.ObjectIndex].ObjectName;

					const int32 RefBoneIndex = EntitySkeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
					check(RefBoneIndex != INDEX_NONE);

					FRawAnimSequenceTrack RawTrack;
					RawTrack.PosKeys.Empty();
					RawTrack.RotKeys.Empty();
					RawTrack.ScaleKeys.Empty();

					int32 NumKeysForTrack = 0;

					//@TODO: Quick and dirty resampling code that needs to be replaced (totally ignores curve type, edge cases, etc...)
					const float ResampleFPS = 30.0f;
					int32 DesiredNumKeys = FMath::CeilToInt(ResampleFPS * DurationInSeconds);
					const float TimePerKey = 1.0f / ResampleFPS;
					
					float CurrentSampleTime = 0.0f;
					for (int32 FrameIndex = 0; FrameIndex < DesiredNumKeys; ++FrameIndex)
					{
						int32 LowerKeyIndex = 0;
						for (; LowerKeyIndex < Timeline.Keys.Num(); ++LowerKeyIndex)
						{
							if (Timeline.Keys[LowerKeyIndex].TimeInMS * 0.001f > CurrentSampleTime)
							{
								--LowerKeyIndex;
								break;
							}
						}
						if (LowerKeyIndex >= Timeline.Keys.Num())
						{
							LowerKeyIndex = Timeline.Keys.Num() - 1;
						}

						int32 UpperKeyIndex = LowerKeyIndex + 1;
						float UpperKeyTime = 0.0f;
						if (UpperKeyIndex >= Timeline.Keys.Num())
						{
							UpperKeyTime = DurationInSeconds;
							if (Animation.bIsLooping)
							{
								UpperKeyIndex = 0;
							}
							else
							{
								UpperKeyIndex = Timeline.Keys.Num() - 1;
							}
						}
						else
						{
							UpperKeyTime = Timeline.Keys[UpperKeyIndex].TimeInMS * 0.001f;
						}

						const FSpriterFatTimelineKey& TimelineKey0 = Timeline.Keys[LowerKeyIndex];
						const FSpriterFatTimelineKey& TimelineKey1 = Timeline.Keys[UpperKeyIndex];
						const float LowerKeyTime = TimelineKey0.TimeInMS * 0.001f;

						const FTransform LocalTransform0 = TimelineKey0.Info.ConvertToTransform();
						const FTransform LocalTransform1 = TimelineKey1.Info.ConvertToTransform();

						FTransform LocalTransform = LocalTransform0;
						if (LowerKeyIndex != UpperKeyIndex)
						{
							const float Alpha = (CurrentSampleTime - LowerKeyTime) / (UpperKeyTime - LowerKeyTime);

							LocalTransform.Blend(LocalTransform0, LocalTransform1, Alpha);
						}

						RawTrack.ScaleKeys.Add(LocalTransform.GetScale3D());
						RawTrack.PosKeys.Add(LocalTransform.GetTranslation());
						RawTrack.RotKeys.Add(LocalTransform.GetRotation());
						++NumKeysForTrack;

						CurrentSampleTime += TimePerKey;
					}
// 
// 					for (const FSpriterFatTimelineKey& TimelineKey : Timeline.Keys)
// 					{
// 						//@TODO: Ignoring TimeInMS
// 						const FTransform LocalTransform = TimelineKey.Info.ConvertToTransform();
// 
// 						RawTrack.ScaleKeys.Add(LocalTransform.GetScale3D());
// 						RawTrack.PosKeys.Add(LocalTransform.GetTranslation());
// 						RawTrack.RotKeys.Add(LocalTransform.GetRotation());
// 
// 						++NumKeysForTrack;
// 					}
// 



					RawAnimationData.Add(RawTrack);
					AnimationAsset->AnimationTrackNames.Add(BoneName);

					// add mapping to skeleton bone track
					AnimationAsset->TrackToSkeletonMapTable.Add(FTrackToSkeletonMap(RefBoneIndex));

					TotalNumKeys = FMath::Max(TotalNumKeys, NumKeysForTrack);
				}
				AnimationAsset->NumFrames = TotalNumKeys;

				AnimationAsset->MarkRawDataAsModified();

				// compress animation
				{
					GWarn->BeginSlowTask(LOCTEXT("BeginCompressAnimation", "Compress Animation"), true);
					GWarn->StatusForceUpdate(1, 1, LOCTEXT("CompressAnimation", "Compressing Animation"));
					// if source data exists, you should bake it to Raw to apply
					if (bSourceDataExists)
					{
						AnimationAsset->BakeTrackCurvesToRawAnimation();
					}
					else
					{
						// otherwise just compress
						AnimationAsset->PostProcessSequence();
					}

					// run debug mode
					GWarn->EndSlowTask();
				}


// 					NewAnimation = FFbxImporter->ImportAnimations(Skeleton, Outer, SortedLinks, AnimName, TemplateImportData, FBXMeshNodeArray);
// 
// 					if (NewAnimation)
// 					{
// 						// since to know full path, reimport will need to do same
// 						UFbxAnimSequenceImportData* ImportData = UFbxAnimSequenceImportData::GetImportDataForAnimSequence(NewAnimation, TemplateImportData);
// 						ImportData->SourceFilePath = FReimportManager::SanitizeImportFilename(UFactory::CurrentFilename, NewAnimation);
// 						ImportData->SourceFileTimestamp = IFileManager::Get().GetTimeStamp(*UFactory::CurrentFilename).ToString();
// 					}


			}
		}

		Result->PostEditChange();
	}
 	else
 	{
 		// Failed to parse the JSON
 		bLoadedSuccessfully = false;
 	}

	if (Result != nullptr)
	{
		//@TODO: Need to do this
		// Store the current file path and timestamp for re-import purposes
// 		UAssetImportData* ImportData = UTileMapAssetImportData::GetImportDataForTileMap(Result);
// 		ImportData->SourceFilePath = FReimportManager::SanitizeImportFilename(CurrentFilename, Result);
// 		ImportData->SourceFileTimestamp = IFileManager::Get().GetTimeStamp(*CurrentFilename).ToString();
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Result);

	return Result;
}

TSharedPtr<FJsonObject> USpriterImporterFactory::ParseJSON(const FString& FileContents, const FString& NameForErrors, bool bSilent)
{
	// Load the file up (JSON format)
	if (!FileContents.IsEmpty())
	{
		const TSharedRef< TJsonReader<> >& Reader = TJsonReaderFactory<>::Create(FileContents);

		TSharedPtr<FJsonObject> DescriptorObject;
		if (FJsonSerializer::Deserialize(Reader, /*out*/ DescriptorObject) && DescriptorObject.IsValid())
		{
			// File was loaded and deserialized OK!
			return DescriptorObject;
		}
		else
		{
			if (!bSilent)
			{
				//@TODO: PAPER2D: How to correctly surface import errors to the user?
				UE_LOGF(LogSpriterImporter, Warning, "Failed to parse Spriter SCON file '%ls'.  Error: '%ls'", *NameForErrors, *Reader->GetErrorMessage());
			}
			return nullptr;
		}
	}
	else
	{
		if (!bSilent)
		{
			UE_LOGF(LogSpriterImporter, Warning, "Spriter SCON file '%ls' was empty.  This Spriter character cannot be imported.", *NameForErrors);
		}
		return nullptr;
	}
}

UObject* USpriterImporterFactory::CreateNewAsset(UClass* AssetClass, const FString& TargetPath, const FString& DesiredName, EObjectFlags Flags)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	// Create a unique package name and asset name for the frame
	const FString TentativePackagePath = UPackageTools::SanitizePackageName(TargetPath + TEXT("/") + DesiredName);
	FString DefaultSuffix;
	FString AssetName;
	FString PackageName;
	AssetToolsModule.Get().CreateUniqueAssetName(TentativePackagePath, DefaultSuffix, /*out*/ PackageName, /*out*/ AssetName);

	// Create a package for the asset
	UObject* OuterForAsset = CreatePackage(*PackageName);

	// Create a frame in the package
	UObject* NewAsset = NewObject<UObject>(OuterForAsset, AssetClass, *AssetName, Flags);
	FAssetRegistryModule::AssetCreated(NewAsset);

	NewAsset->Modify();
	return NewAsset;
}

UObject* USpriterImporterFactory::ImportAsset(const FString& SourceFilename, const FString& TargetSubPath)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	TArray<FString> FileNames;
	FileNames.Add(SourceFilename);

	TArray<UObject*> ImportedAssets = AssetToolsModule.Get().ImportAssets(FileNames, TargetSubPath);
	return (ImportedAssets.Num() > 0) ? ImportedAssets[0] : nullptr;
}


UTexture2D* USpriterImporterFactory::ImportTexture(const FString& SourceFilename, const FString& TargetSubPath)
{
	UTexture2D* ImportedTexture = Cast<UTexture2D>(ImportAsset(SourceFilename, TargetSubPath));

	if (ImportedTexture != nullptr)
	{
		// Change the compression settings
		GetDefault<UPaperImporterSettings>()->ApplyTextureSettings(ImportedTexture);
	}

	return ImportedTexture;
}

//////////////////////////////////////////////////////////////////////////

#undef SPRITER_IMPORT_ERROR
#undef LOCTEXT_NAMESPACE