// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SpriterDataModel.h"
#include "SpriterImporterFactory.generated.h"

// Imports a rigged sprite character (and associated textures & animations) exported from Spriter (http://www.brashmonkey.com/)
UCLASS()
class USpriterImporterFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual FText GetToolTip() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn) override;
	// End of UFactory interface

protected:
	TSharedPtr<FJsonObject> ParseJSON(const FString& FileContents, const FString& NameForErrors, bool bSilent = false);

	static UObject* CreateNewAsset(UClass* AssetClass, const FString& TargetPath, const FString& DesiredName, EObjectFlags Flags);
	static UObject* ImportAsset(const FString& SourceFilename, const FString& TargetSubPath);
	static UTexture2D* ImportTexture(const FString& SourceFilename, const FString& TargetSubPath);
};
