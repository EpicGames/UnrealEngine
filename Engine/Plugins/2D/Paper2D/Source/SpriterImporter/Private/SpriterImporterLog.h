// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Paper2DModule.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSpriterImporter, Verbose, All);

#define SPRITER_IMPORT_ERROR(FormatString, ...) \
	if (!bSilent) { UE_LOG(LogSpriterImporter, Warning, FormatString, __VA_ARGS__); }
#define SPRITER_IMPORT_WARNING(FormatString, ...) \
	if (!bSilent) { UE_LOG(LogSpriterImporter, Warning, FormatString, __VA_ARGS__); }

