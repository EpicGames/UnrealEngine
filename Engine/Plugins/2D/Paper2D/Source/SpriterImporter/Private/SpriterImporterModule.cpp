// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "SpriterImporterLog.h"
#include "Modules/ModuleManager.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "SpriterImporter"

DEFINE_LOG_CATEGORY(LogSpriterImporter);

//////////////////////////////////////////////////////////////////////////
// FSpriterImporterModule

class FSpriterImporterModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FSpriterImporterModule, SpriterImporter);

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
